#!/bin/bash

set -eu

SCRIPT_DIR=$(cd "$(dirname "$BASH_SOURCE")" ; pwd)
UE_THIRD_PARTY_DIR=$(cd "${SCRIPT_DIR}/../../../.."; pwd)
SYMS_DIR=$(cd "${SCRIPT_DIR}"; pwd)

CFLAGS="-ffunction-sections -fdata-sections -I$SCRIPT_DIR/../../ThirdParty/zlib/1.3"
CXXFLAGS="-ffunction-sections -fdata-sections -I$UE_THIRD_PARTY_DIR/Unix/LibCxx/include -I$UE_THIRD_PARTY_DIR/Unix/LibCxx/include/c++/v1"

# Get num of cores
export CORES=$(getconf _NPROCESSORS_ONLN)
echo "Using ${CORES} cores for building"

BuildLibSyms()
{
    export ARCH=$1
    export FLAVOR=$2
    local EXT=$3
    local BUILD_DIR=/tmp/Build-Syms-${ARCH}-${FLAVOR}

    LIBS="$UE_THIRD_PARTY_DIR/Unix/LibCxx/lib/Unix/${ARCH}/libc++.a $UE_THIRD_PARTY_DIR/Unix/LibCxx/lib/Unix/${ARCH}/libc++abi.a -lm -lc -lgcc_s -lgcc -lpthread"

    echo "Building ${ARCH} ${FLAVOR}"

( cat <<_EOF_
  ## autogenerated by ${BASH_SOURCE} script
  SET(LINUX_MULTIARCH_ROOT \$ENV{UE_SDKS_ROOT}/HostLinux/Linux_x64/v22_clang-16.0.6-centos7)
  SET(ARCHITECTURE_TRIPLE \$ENV{ARCH})

  message (STATUS "LINUX_MULTIARCH_ROOT is '\${LINUX_MULTIARCH_ROOT}'")
  message (STATUS "ARCHITECTURE_TRIPLE is '\${ARCHITECTURE_TRIPLE}'")

  SET(CMAKE_CROSSCOMPILING TRUE)
  SET(CMAKE_SYSTEM_NAME Linux)
  SET(CMAKE_SYSTEM_VERSION 1)

  # sysroot
  SET(CMAKE_SYSROOT \${LINUX_MULTIARCH_ROOT}/\${ARCHITECTURE_TRIPLE})

  SET(CMAKE_LIBRARY_ARCHITECTURE \${ARCHITECTURE_TRIPLE})

  # specify the cross compiler
  SET(CMAKE_C_COMPILER            \${CMAKE_SYSROOT}/bin/clang)
  SET(CMAKE_C_COMPILER_TARGET     \${ARCHITECTURE_TRIPLE})
  SET(CMAKE_C_FLAGS "-target      \${ARCHITECTURE_TRIPLE} ${CFLAGS}")

  SET(CMAKE_CXX_COMPILER          \${CMAKE_SYSROOT}/bin/clang++)
  SET(CMAKE_CXX_COMPILER_TARGET   \${ARCHITECTURE_TRIPLE})
  SET(CMAKE_CXX_FLAGS "-target    \${ARCHITECTURE_TRIPLE} ${CXXFLAGS}")

  SET(CMAKE_SHARED_LINKER_FLAGS   "${LIBS}")

  SET(CMAKE_ASM_COMPILER          \${CMAKE_SYSROOT}/bin/clang)

  SET(CMAKE_FIND_ROOT_PATH        \${LINUX_MULTIARCH_ROOT})

  # hoping to force it to use ar
  set(CMAKE_FIND_ROOT_PATH_MODE_PROGRAM ONLY)
  set(CMAKE_FIND_ROOT_PATH_MODE_LIBRARY ONLY)
  set(CMAKE_FIND_ROOT_PATH_MODE_INCLUDE ONLY)

_EOF_
) > /tmp/__cmake_toolchain.cmake

    rm -rf ${BUILD_DIR}
    mkdir -p ${BUILD_DIR}/Build

    pushd ${BUILD_DIR}/Build

    set -x
    cmake \
      -DCMAKE_TOOLCHAIN_FILE="/tmp/__cmake_toolchain.cmake" \
      -DCMAKE_MAKE_PROGRAM=$(which make) \
      -DCMAKE_BUILD_TYPE=${FLAVOR} \
      -DCMAKE_POSITION_INDEPENDENT_CODE=ON \
      ${SYMS_DIR}/
    set +x

    VERBOSE=1 make -j ${CORES}

    mkdir -p ${SYMS_DIR}/lib/Unix/${ARCH}
    cp -v ${BUILD_DIR}/Build/libsyms.a ${SYMS_DIR}/lib/Unix/${ARCH}/libsyms${EXT}_fPIC.a

    popd
}

BuildLibSyms x86_64-unknown-linux-gnu      Release ""
BuildLibSyms x86_64-unknown-linux-gnu      Debug   "d"
BuildLibSyms aarch64-unknown-linux-gnueabi Release ""
BuildLibSyms aarch64-unknown-linux-gnueabi Debug   "d"
