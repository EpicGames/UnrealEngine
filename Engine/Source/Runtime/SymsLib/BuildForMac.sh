#!/bin/bash

set -eu

SCRIPT_DIR=$(cd "$(dirname "$BASH_SOURCE")" ; pwd)
UE_THIRD_PARTY_DIR=$(cd "${SCRIPT_DIR}/../../../.."; pwd)
SYMS_DIR=$(cd "${SCRIPT_DIR}"; pwd)

CFLAGS="-ffunction-sections -fdata-sections -I$SCRIPT_DIR/../../ThirdParty/zlib/1.3"
CXXFLAGS="-ffunction-sections -fdata-sections"
LIBS="-lm -lc -lpthread"

# Get num of cores
export CORES=$(getconf _NPROCESSORS_ONLN)
echo "Using ${CORES} cores for building"

BuildLibSyms()
{
    export ARCH=$1
    export FLAVOR=$2
    local EXT=$3
    local BUILD_DIR=/tmp/Build-Syms-${ARCH}-${FLAVOR}

    echo "Building ${ARCH} ${FLAVOR}"
    rm -rf ${BUILD_DIR}
    mkdir -p ${BUILD_DIR}/Build

    pushd ${BUILD_DIR}/Build

    set -x
    cmake \
      -DCMAKE_MAKE_PROGRAM=$(which make) \
      -DCMAKE_BUILD_TYPE=${FLAVOR} \
      -DCMAKE_POSITION_INDEPENDENT_CODE=ON \
      -DCMAKE_OSX_DEPLOYMENT_TARGET=11.0 \
      -DCMAKE_OSX_ARCHITECTURES=${ARCH} \
      ${SYMS_DIR}/
    set +x
    
    VERBOSE=1 make -j ${CORES}

    mkdir -p ${SYMS_DIR}/lib/Mac/${ARCH}
    cp -v ${BUILD_DIR}/Build/libsyms.a ${SYMS_DIR}/lib/Mac/${ARCH}/libsyms${EXT}.a

    popd
}

BuildLibSyms x86_64 Release ""
BuildLibSyms x86_64 Debug "d"
BuildLibSyms arm64 Release ""
BuildLibSyms arm64 Debug "d"

lipo -create ${SYMS_DIR}/lib/Mac/{x86_64,arm64}/libsymsd.a -output ${SYMS_DIR}/lib/Mac/libsymsd.a
lipo -create ${SYMS_DIR}/lib/Mac/{x86_64,arm64}/libsyms.a  -output ${SYMS_DIR}/lib/Mac/libsyms.a
rm -rf ${SYMS_DIR}/lib/Mac/{x86_64,arm64}
