cmake_minimum_required(VERSION 3.00)
PROJECT(SYMLIBS)

set(SRC_FILES
    syms/symslib.c
)

add_library(syms STATIC ${SRC_FILES})
target_include_directories(syms PUBLIC ${CMAKE_SOURCE_DIR})
