// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;
using System.IO;

public class SymsLib : ModuleRules
{
	public SymsLib(ReadOnlyTargetRules Target) : base(Target)
	{
		Type = ModuleType.External;

		string LibPathBase =  ModuleDirectory + "/lib";

		PublicSystemIncludePaths.Add(ModuleDirectory);
		PublicSystemIncludePaths.Add(ModuleDirectory + "/syms");

		if (Target.Platform.IsInGroup(UnrealPlatformGroup.Windows))
		{
			bool bUseDebug = Target.Configuration == UnrealTargetConfiguration.Debug;
			string Arch = Target.Architecture == UnrealArch.Arm64 ? "ARM64" : "x64";
			string LibPath = Path.Combine(LibPathBase, Arch, (bUseDebug ? "Debug" : "Release"), "SymsLib.lib");
			PublicAdditionalLibraries.Add(LibPath);
		}
		else if (Target.Platform.IsInGroup(UnrealPlatformGroup.Unix))
		{
			bool bUseDebug = Target.Configuration == UnrealTargetConfiguration.Debug;
			string LibName = bUseDebug ? "libsymsd_fPIC.a" : "libsyms_fPIC.a";
			string LibPath = Path.Combine(LibPathBase, "Unix", Target.Architecture.LinuxName);

			PublicAdditionalLibraries.Add(Path.Combine(LibPath, LibName));
		}
		else if (Target.Platform == UnrealTargetPlatform.Mac)
		{
			bool bUseDebug = Target.Configuration == UnrealTargetConfiguration.Debug;
			string LibName = bUseDebug ? "libsymsd.a" : "libsyms.a";
			string LibPath = Path.Combine(LibPathBase, "Mac");

			PublicAdditionalLibraries.Add(Path.Combine(LibPath, LibName));
		}

        AddEngineThirdPartyPrivateStaticDependencies(Target, "zlib");
	}
}
