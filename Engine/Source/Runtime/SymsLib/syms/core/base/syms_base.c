// Copyright Epic Games, Inc. All Rights Reserved.

#ifndef SYMS_BASE_C
#define SYMS_BASE_C

////////////////////////////////
//~ rjf: Library Metadata

SYMS_API SYMS_String8
syms_version_string(void){
  return syms_str8_lit(SYMS_VERSION_STR);
}

////////////////////////////////
//~ rjf: Basic Type Functions

SYMS_API SYMS_U64Range
syms_make_u64_range(SYMS_U64 min, SYMS_U64 max){
  SYMS_U64Range result = {min, max};
  return(result);
}

SYMS_API SYMS_U64Range
syms_make_u64_inrange(SYMS_U64Range range, SYMS_U64 offset, SYMS_U64 size){
  SYMS_ASSERT(range.min + offset + size <= range.max);
  SYMS_U64Range result = syms_make_u64_range(range.min + offset, range.min + offset + size);
  return result;
}

SYMS_API SYMS_U64
syms_u64_range_size(SYMS_U64Range range){
  SYMS_U64 result = range.max - range.min;
  return(result);
}

////////////////////////////////
//~ allen: Hash Functions

SYMS_API SYMS_U64
syms_hash_djb2(SYMS_String8 string){
  SYMS_U64 result = syms_hash_djb2_continue(string, syms_hash_djb2_initial);
  return(result);
}

SYMS_API SYMS_U64
syms_hash_djb2_continue(SYMS_String8 string, SYMS_U64 intermediate_hash){
  SYMS_U8 *ptr = string.str;
  SYMS_U8 *opl = string.str + string.size;
  SYMS_U64 result = intermediate_hash;
  for (; ptr < opl; ptr += 1){
    result = ((result << 5) + result) + (*ptr);
  }
  return(result);
}

SYMS_API SYMS_U64
syms_hash_u64(SYMS_U64 x){
  SYMS_U64 a = (x >> 10) | (x << 54);
  SYMS_U64 b = a*5381;
  return(b);
}

////////////////////////////////
//~ rjf: Serial Information Functions

SYMS_READ_ONLY SYMS_GLOBAL SYMS_SerialValue syms_null_serial_value;
SYMS_READ_ONLY SYMS_GLOBAL SYMS_SerialFlag syms_null_serial_flag;

SYMS_API SYMS_SerialField*
syms_serial_first_field(SYMS_SerialType *type){
  SYMS_SerialField *result = 0;
  if (type->kind == SYMS_SerialTypeKind_Struct){
    result = (SYMS_SerialField*)type->children;
  }
  return(result);
}

SYMS_API SYMS_SerialValue*
syms_serial_first_value(SYMS_SerialType *type){
  SYMS_SerialValue *result = 0;
  if (type->kind == SYMS_SerialTypeKind_Enum){
    result = (SYMS_SerialValue*)type->children;
  }
  return(result);
}

SYMS_API SYMS_SerialFlag*
syms_serial_first_flag(SYMS_SerialType *type){
  SYMS_SerialFlag *result = 0;
  if (type->kind == SYMS_SerialTypeKind_Flags){
    result = (SYMS_SerialFlag*)type->children;
  }
  return(result);
}

SYMS_API SYMS_SerialValue*
syms_serial_value_from_enum_value(SYMS_SerialType *type, SYMS_U64 value){
  SYMS_SerialValue *result = &syms_null_serial_value;
  if (type->kind == SYMS_SerialTypeKind_Enum){
    SYMS_U64 enum_idx = type->enum_index_from_value(value);
    if(enum_idx < type->child_count)
    {
      SYMS_SerialValue *values = (SYMS_SerialValue*)type->children;
      result = values + type->enum_index_from_value(value);
    }
  }
  return(result);
}

SYMS_API SYMS_SerialFlag*
syms_serial_flag_from_bit_offset(SYMS_SerialType *type, SYMS_U64 bit_off){
  SYMS_SerialFlag *result = &syms_null_serial_flag;
  // NOTE(allen): following should hold: type->basic_size <= 32.
  //              then this condition gives: (bit_off < 32) => (1 << (SYMS_U32)bit_off) defined
  if (type->kind == SYMS_SerialTypeKind_Enum && bit_off < type->basic_size){
    SYMS_U64 child_count = type->child_count;
    SYMS_SerialFlag *flag = (SYMS_SerialFlag*)type->children;
    SYMS_U32 bit = (1 << (SYMS_U32)bit_off);
    for (SYMS_U64 i = 0; i < child_count; i += 1, flag += 1){
      if ((flag->mask & bit) == bit){
        result = flag;
        break;
      }
    }
  }
  return(result);
}

SYMS_API SYMS_String8List
syms_string_list_from_flags(SYMS_Arena *arena, SYMS_SerialType *type, SYMS_U32 flags){
  // TODO(allen): ideas for also exporting multi-bit values?
  //  (1) nothing  (2) value  (3) field=value  (4) abstract model for 3 somehow
  
  SYMS_String8List result = {0};
  if (type != 0){
    SYMS_U32 flag_count = type->child_count;
    SYMS_SerialFlag *flag = syms_serial_first_flag(type);
    for (SYMS_U32 i = 0; i < flag_count; i += 1, flag += 1){
      if (flag->mask == 1 && ((flags >> flag->bitshift) & 1)){
        syms_string_list_push(arena, &result, flag->name);
      }
    }
  }
  return(result);
}

SYMS_API SYMS_U64
syms_enum_index_from_value_identity(SYMS_U64 v){ return(v); }

////////////////////////////////
//~ allen: String Functions

SYMS_API SYMS_B32
syms_codepoint_is_whitespace(SYMS_U32 codepoint){
  return (codepoint == ' '  || codepoint == '\n' || codepoint == '\r' ||
          codepoint == '\t' || codepoint == '\f' || codepoint == '\v');
}

SYMS_API SYMS_U32
syms_lowercase_from_codepoint(SYMS_U32 codepoint){
  if(codepoint >= 'A' && codepoint <= 'Z'){
    codepoint += 'a' - 'A';
  }
  return codepoint;
}


SYMS_API SYMS_String8
syms_str8(SYMS_U8 *str, SYMS_U64 size){
  SYMS_String8 result = {str, size};
  return(result);
}

SYMS_API SYMS_String8
syms_str8_cstring(char *str){
  SYMS_U8 *first = (SYMS_U8*)str;
  SYMS_U8 *ptr = first;
  for (;*ptr != 0; ptr += 1);
  SYMS_String8 result = syms_str8_range(first, ptr);
  return(result);
}

SYMS_API SYMS_String8
syms_str8_range(SYMS_U8 *first, SYMS_U8 *opl){
  SYMS_String8 result = {first, (SYMS_U64)(opl - first)};
  return(result);
}

SYMS_API SYMS_String8
syms_str8_skip_chop_whitespace(SYMS_String8 str){
  SYMS_U8 *opl = str.str + str.size;
  SYMS_U8 *first = str.str;
  for (;first < opl; first += 1){
    if (!syms_codepoint_is_whitespace(*first)){
      break;
    }
  }
  SYMS_U8 *last = opl - 1;
  for (;last >= first; last -= 1){
    if (!syms_codepoint_is_whitespace(*last)){
      break;
    }
  }
  last += 1;
  SYMS_String8 result = syms_str8_range(first, last);
  return(result);
}

SYMS_API SYMS_B32
syms_string_match(SYMS_String8 a, SYMS_String8 b, SYMS_StringMatchFlags flags){
  SYMS_B32 result = syms_false;
  if (a.size == b.size || (flags & SYMS_StringMatchFlag_RightSideSloppy)){
    SYMS_U64 size = SYMS_MIN(a.size, b.size);
    result = syms_true;
    if (size > 0){
      // extract "inner loop flags"
      SYMS_B32 case_insensitive = (flags & SYMS_StringMatchFlag_CaseInsensitive);
      SYMS_B32 slash_insensitive = (flags & SYMS_StringMatchFlag_SlashInsensitive);
      
      // comparison loop
      SYMS_U8 *aptr = a.str;
      SYMS_U8 *bptr = b.str;
      SYMS_U8 *bopl = bptr + size;
      for (;bptr < bopl; aptr += 1, bptr += 1){
        SYMS_U8 at = *aptr;
        SYMS_U8 bt = *bptr;
        if (case_insensitive){
          at = syms_lowercase_from_codepoint(at);
          bt = syms_lowercase_from_codepoint(bt);
        }
        if (slash_insensitive){
          at = (at == '\\' ? '/' : at);
          bt = (bt == '\\' ? '/' : bt);
        }
        if (at != bt){
          result = syms_false;
          break;
        }
      }
    }
  }
  return(result);
}

SYMS_API SYMS_U8*
syms_decode_utf8(SYMS_U8 *p, SYMS_U32 *dst)
{
  SYMS_U32 res, n;
  switch (*p & 0xf0) {
    case 0xf0 :  res = *p & 0x07;  n = 3;  break;
    case 0xe0 :  res = *p & 0x0f;  n = 2;  break;
    case 0xd0 :
    case 0xc0 :  res = *p & 0x1f;  n = 1;  break;
    default   :  res = *p;         n = 0;  break;
  }
  while (n--) {
    res = (res << 6) | (*(++p) & 0x3f);
  }
  *dst = res;
  return p + 1;
}

SYMS_API void
syms_string_list_push_node(SYMS_String8Node *node, SYMS_String8List *list, SYMS_String8 string){
  node->string = string;
  SYMS_QueuePush(list->first, list->last, node);
  list->node_count += 1;
  list->total_size += string.size;
}

SYMS_API void
syms_string_list_push_node_front(SYMS_String8Node *node, SYMS_String8List *list,
                                 SYMS_String8 string)
{
  node->string = string;
  SYMS_QueuePushFront(list->first, list->last, node);
  list->node_count += 1;
  list->total_size += string.size;
}

SYMS_API void
syms_string_list_push(SYMS_Arena *arena, SYMS_String8List *list, SYMS_String8 string){
  SYMS_String8Node *node = syms_push_array(arena, SYMS_String8Node, 1);
  syms_string_list_push_node(node, list, string);
}

SYMS_API void
syms_string_list_push_front(SYMS_Arena *arena, SYMS_String8List *list, SYMS_String8 string){
  SYMS_String8Node *node = syms_push_array(arena, SYMS_String8Node, 1);
  syms_string_list_push_node_front(node, list, string);
}

SYMS_API SYMS_String8List
syms_string_list_concat(SYMS_String8List *left, SYMS_String8List *right){
  SYMS_String8List result = {0};
  result.node_count = left->node_count + right->node_count;
  result.total_size = left->total_size + right->total_size;
  if (left->last != 0){
    left->last->next = right->first;
  }
  result.first = left->first;
  if (result.first == 0){
    result.first = right->first;
  }
  result.last = right->last;
  if (result.last == 0){
    result.last = left->last;
  }
  syms_memzero_struct(left);
  syms_memzero_struct(right);
  return(result);
}

SYMS_API SYMS_String8
syms_string_list_join(SYMS_Arena *arena, SYMS_String8List *list, SYMS_StringJoin *join_ptr){
  // setup join default
  SYMS_StringJoin join = {0};
  if (join_ptr != 0){
    syms_memmove(&join, join_ptr, sizeof(join));
  }
  
  // allocate
  SYMS_String8 result = {0};
  result.size = list->total_size + join.pre.size + join.post.size;
  if (list->node_count >= 2){
    result.size += (list->node_count - 1)*join.sep.size;
  }
  result.str = syms_push_array(arena, SYMS_U8, result.size + 1);
  
  // fill
  SYMS_U8 *ptr = result.str;
  syms_memmove(ptr, join.pre.str, join.pre.size);
  ptr += join.pre.size;
  for (SYMS_String8Node *node = list->first, *next = 0;
       node != 0;
       node = next){
    syms_memmove(ptr, node->string.str, node->string.size);
    ptr += node->string.size;
    next = node->next;
    if (next != 0){
      syms_memmove(ptr, join.sep.str, join.sep.size);
      ptr += join.sep.size;
    }
  }
  syms_memmove(ptr, join.post.str, join.post.size);
  ptr += join.post.size;
  *ptr = 0;
  
  return(result);
}

SYMS_API SYMS_String8
syms_push_string_copy(SYMS_Arena *arena, SYMS_String8 string){
  SYMS_String8 result = syms_str8(syms_push_array(arena, SYMS_U8, string.size+1), string.size);
  syms_memmove(result.str, string.str, string.size);
  result.str[string.size] = 0;
  return result;
}

SYMS_API SYMS_String8
syms_string_trunc_symbol_heuristic(SYMS_String8 string){
  SYMS_U8 *opl = string.str + string.size;
  
  SYMS_U8 *b = string.str;
  for (;b < opl; b += 1){
    SYMS_U8 c = *b;
    SYMS_B32 ident = (('A' <= c && c <= 'Z') ||
                      ('a' <= c && c <= 'z') ||
                      ('0' <= c && c <= '9') ||
                      c == '_' || c == '$' || c == ':' || c >= 128);
    if (!ident){
      break;
    }
  }
  
  SYMS_U8 *a = b;
  for (;a > string.str;){
    a -= 1;
    if (*a == ':'){
      a += 1;
      break;
    }
  }
  
  SYMS_String8 result = syms_str8_range(a, b);
  return(result);
}

SYMS_API SYMS_String8List
syms_string_split(SYMS_Arena *arena, SYMS_String8 input, SYMS_U32 delimiter)
{
  SYMS_String8List result;
  syms_memzero_struct(&result);
  
  SYMS_U8 *ptr = input.str;
  SYMS_U8 *end = input.str + input.size;
  SYMS_U64 size = 0;
  for (;;) {
    SYMS_U32 cp = 0;
    SYMS_U8 *pn = syms_decode_utf8(ptr, &cp);
    if (cp == delimiter) {
      SYMS_String8 split = syms_str8(ptr - size, size);
      syms_string_list_push(arena, &result, split);
      size = 0;
    } else if (pn >= end || cp == 0) {
      SYMS_ASSERT(pn == end);
      SYMS_String8 last = syms_str8(pn - size, size);
      syms_string_list_push(arena, &result, last);
      break;
    }
    size += 1;
    ptr = pn;
  }
  
  return result;
}

////////////////////////////////
//~ allen: String -> Int

SYMS_API SYMS_U64
syms_u64_from_string(SYMS_String8 str, SYMS_U32 radix){
  // First we'll shift the ascii range around with
  //   (x - 0x30)%0x20
  // to map each ascii value to one of the slots in this table.
  // The ascii values for hex digits happen to land on their
  // correct values!
  SYMS_LOCAL SYMS_U8 v_from_c[32] = {
    0x00,0x01,0x02,0x03,0x04,0x05,0x06,0x07,
    0x08,0x09,0xFF,0xFF,0xFF,0xFF,0xFF,0xFF,
    0xFF,0x0A,0x0B,0x0C,0x0D,0x0E,0x0F,0xFF,
    0xFF,0xFF,0xFF,0xFF,0xFF,0xFF,0xFF,0xFF,
  };
  
  // Then we'll squeeze the ascii range down with
  //   (x >> 3)
  // to map each ascii value to one of the masks slots in this table.
  // Each slot covers an 8-value range. The slots with 1s indicate
  // blocks of values that contain valid hex digit asciis.
  // All the ascii values in slots marked 1 that aren't ascii happen
  // to map to the value 0xFF in the v_from_c table!
  SYMS_LOCAL SYMS_B8 m_from_c[32] = {
    0,0,0,0,0,0,1,1,
    1,0,0,0,1,0,0,0,
    0,0,0,0,0,0,0,0,
    0,0,0,0,0,0,0,0,
  };
  
  // Putting it together we can determine for each ascii byte
  // if it is legal in the integer we are parsing, and get it's value!
  
  
  
  SYMS_U64 result = 0;
  if (2 <= radix && radix <= 16){
    SYMS_U8 *ptr = str.str;
    SYMS_U8 *opl = str.str + str.size;
    for (; ptr < opl; ptr += 1){
      SYMS_U8 x = *ptr;
      SYMS_B8 mask = m_from_c[x >> 3];
      if (!mask){
        result = 0;
        break;
      }
      SYMS_U8 value = (v_from_c[(x - 0x30)%0x20]);
      if (value >= radix){
        result = 0;
        break;
      }
      result *= radix;
      result += value;
    }
  }
  return(result);
}

SYMS_API SYMS_S64
syms_s64_from_string_c_rules(SYMS_String8 str){
  SYMS_U8 *ptr = str.str;
  SYMS_U8 *opl = str.str + str.size;
  
  // consume signs
  SYMS_S64 sign = 1;
  for (;ptr < opl; ptr += 1){
    if (*ptr == '-'){
      sign *= -1;
    }
    else if (*ptr == '+'){
      // do nothing
    }
    else{
      break;
    }
  }
  
  // determine radix
  SYMS_U32 radix = 10;
  if (ptr < opl && *ptr == '0'){
    ptr += 1;
    radix = 010;
    if (ptr < opl){
      if (*ptr == 'x'){
        radix = 0x10;
        ptr += 1;
      }
      else if (*ptr == 'b'){
        radix = 2;
        ptr += 1;
      }
    }
  }
  
  // determine integer value
  SYMS_U64 x = syms_u64_from_string(syms_str8_range(ptr, opl), radix);
  
  SYMS_S64 result = ((SYMS_S64)x)*sign;
  return(result);
}

SYMS_API SYMS_String8
syms_string_from_u64(SYMS_Arena *arena, SYMS_U64 x){
  SYMS_U8 buffer[64];
  SYMS_U8 *ptr = buffer;
  if (x == 0){
    *ptr = '0';
    ptr += 1;
  }
  else{
    for (;;){
      *ptr = (x%10) + '0';
      ptr += 1;
      x /= 10;
      if (x == 0){
        break;
      }
    }
    
    SYMS_U8 *a = buffer;
    SYMS_U8 *b = ptr - 1;
    for (;a < b;){
      SYMS_U8 t = *a;
      *a = *b;
      *b = t;
      a += 1;
      b -= 1;
    }
  }
  
  SYMS_String8 result = syms_push_string_copy(arena, syms_str8_range(buffer, ptr));
  return(result);
}

////////////////////////////////
//~ rjf: U64 Range Functions

SYMS_API void
syms_u64_range_list_push_node(SYMS_U64RangeNode *node, SYMS_U64RangeList *list, SYMS_U64Range range){
  SYMS_QueuePush(list->first, list->last, node);
  list->node_count += 1;
  node->range = range;
}

SYMS_API void
syms_u64_range_list_push(SYMS_Arena *arena, SYMS_U64RangeList *list, SYMS_U64Range range){
  SYMS_U64RangeNode *node = syms_push_array(arena, SYMS_U64RangeNode, 1);
  syms_u64_range_list_push_node(node, list, range);
}

SYMS_API void
syms_u64_range_list_concat(SYMS_U64RangeList *list, SYMS_U64RangeList *to_push)
{
  if(list->last == 0)
  {
    *list = *to_push;
  }
  else if(to_push->first != 0)
  {
    list->last->next = to_push->first;
    list->last = to_push->last;
    list->node_count += to_push->node_count;
  }
}

SYMS_API SYMS_U64RangeArray
syms_u64_range_array_from_list(SYMS_Arena *arena, SYMS_U64RangeList *list){
  SYMS_U64RangeArray result = {0};
  result.count = list->node_count;
  result.ranges = syms_push_array(arena, SYMS_U64Range, result.count);
  SYMS_U64Range *range_ptr = result.ranges;
  SYMS_U64Range *range_opl = result.ranges + result.count;
  for (SYMS_U64RangeNode *node = list->first;
       node != 0 && range_ptr < range_opl;
       node = node->next, range_ptr += 1){
    *range_ptr = node->range;
  }
  if (range_ptr < range_opl){
    syms_arena_put_back(arena, (range_opl - range_ptr)*sizeof(*range_ptr));
    result.count = (range_ptr - result.ranges);
  }
  return(result);
}

////////////////////////////////
//~ nick: U64 List Functions

SYMS_API void
syms_u64_list_push_node(SYMS_U64Node *node, SYMS_U64List *list, SYMS_U64 v){
  SYMS_QueuePush(list->first, list->last, node);
  list->count += 1;
  node->u64 = v;
}

SYMS_API void
syms_u64_list_push(SYMS_Arena *arena, SYMS_U64List *list, SYMS_U64 v){
  SYMS_U64Node *node = syms_push_array(arena, SYMS_U64Node, 1);
  syms_u64_list_push_node(node, list, v);
}

SYMS_API void
syms_u64_list_concat_in_place(SYMS_U64List *dst, SYMS_U64List *src){
  if (dst->last == 0){
    *dst = *src;
  }
  else if (src->first != 0){
    dst->last->next = src->first;
    dst->last = src->last;
    dst->count += src->count;
  }
  syms_memzero_struct(src);
}

SYMS_API SYMS_U64Array
syms_u64_array_from_list(SYMS_Arena *arena, SYMS_U64List *list){
  SYMS_U64Array result = {0};
  result.u64 = syms_push_array(arena, SYMS_U64, list->count);
  result.count = list->count;
  SYMS_U64 *ptr = result.u64;
  for (SYMS_U64Node *node = list->first;
       node != 0;
       node = node->next, ptr += 1){
    *ptr = node->u64;
  }
  return(result);
}


////////////////////////////////
//~ allen: Array Functions

SYMS_API SYMS_U64
syms_1based_checked_lookup_u64(SYMS_U64 *u64, SYMS_U64 count, SYMS_U64 n){
  SYMS_U64 result = 0;
  if (1 <= n && n <= count){
    result = u64[n - 1];
  }
  return(result);
}


////////////////////////////////
//~ allen: Arena Functions

SYMS_API void
syms_arena_push_align(SYMS_Arena *arena, SYMS_U64 pow2_boundary){
  SYMS_U64 pos = syms_arena_get_pos(arena);
  SYMS_U64 next_pos = SYMS_AlignPow2(pos, pow2_boundary);
  SYMS_U64 align_amt = next_pos - pos;
  syms_arena_push(arena, align_amt);
}

SYMS_API void
syms_arena_put_back(SYMS_Arena *arena, SYMS_U64 amount){
  SYMS_U64 pos = syms_arena_get_pos(arena);
  SYMS_U64 new_pos = 0;
  if (pos >= amount){
    new_pos = pos - amount;
  }
  syms_arena_pop_to(arena, new_pos);
}

SYMS_API SYMS_ArenaTemp
syms_arena_temp_begin(SYMS_Arena *arena){
  SYMS_U64 pos = syms_arena_get_pos(arena);
  SYMS_ArenaTemp result = {arena, pos};
  return(result);
}

SYMS_API void
syms_arena_temp_end(SYMS_ArenaTemp temp){
  syms_arena_pop_to(temp.arena, temp.pos);
}

SYMS_API SYMS_ArenaTemp
syms_get_scratch(SYMS_Arena **conflicts, SYMS_U64 conflict_count){
  // get compatible arena
  SYMS_Arena *compatible_arena = syms_get_implicit_thread_arena(conflicts, conflict_count);
  
  // construct a temp
  SYMS_ArenaTemp result = {0};
  if (compatible_arena != 0){
    result = syms_arena_temp_begin(compatible_arena);
  }
  return(result);
}

////////////////////////////////
//~ allen: Syms Sort Node

SYMS_API SYMS_SortNode*
syms_sort_node_push(SYMS_Arena *arena, SYMS_SortNode **stack, SYMS_SortNode **free_stack,
                    SYMS_U64 first, SYMS_U64 opl){
  SYMS_SortNode *result = *free_stack;
  if (result != 0){
    SYMS_StackPop(*free_stack);
  }
  else{
    result = syms_push_array(arena, SYMS_SortNode, 1);
  }
  SYMS_StackPush(*stack, result);
  result->first = first;
  result->opl = opl;
  return(result);
}

////////////////////////////////
//~ allen: Thread Lanes

SYMS_THREAD_LOCAL SYMS_U64 syms_thread_lane = 0;

SYMS_API void
syms_set_lane(SYMS_U64 lane){
  syms_thread_lane = lane;
}

SYMS_API SYMS_U64
syms_get_lane(void){
  return(syms_thread_lane);
}

////////////////////////////////
//~ rjf: Based Ranges

SYMS_API void*
syms_based_range_ptr(void *base, SYMS_U64Range range, SYMS_U64 offset){
  void *result = 0;
  if (offset < syms_u64_range_size(range)){
    result = ((char*)base + range.min + offset);
  }
  return(result);
}

SYMS_API SYMS_U64
syms_based_range_read(void *base, SYMS_U64Range range, SYMS_U64 offset, SYMS_U64 out_size, void *out){
  SYMS_U64 result = 0;
  void *ptr = syms_based_range_ptr(base, range, offset);
  if (ptr != 0){
    SYMS_U64 max_size = syms_u64_range_size(range) - offset;
    result = SYMS_ClampTop(out_size, max_size);
    syms_memmove(out, ptr, result);
  }
  return(result);
}

SYMS_API SYMS_U64
syms_based_range_read_uleb128(void *base, SYMS_U64Range range, SYMS_U64 offset, SYMS_U64 *out_value)
{
  SYMS_U64 value = 0;
  SYMS_U64 bytes_read = 0;
  SYMS_U64 shift = 0;
  SYMS_U8 byte = 0;
  for(SYMS_U64 read_offset = offset;
      syms_based_range_read_struct(base, range, read_offset, &byte) == 1;
      read_offset += 1)
  {
    bytes_read += 1;
    SYMS_U8 val = byte & 0x7fu;
    value |= ((SYMS_U64)val) << shift;
    if((byte&0x80u) == 0)
    {
      break;
    }
    shift += 7u;
  }
  if(out_value != 0)
  {
    *out_value = value;
  }
  return bytes_read;
}

SYMS_API SYMS_U64
syms_based_range_read_sleb128(void *base, SYMS_U64Range range, SYMS_U64 offset, SYMS_S64 *out_value)
{
  SYMS_U64 value = 0;
  SYMS_U64 bytes_read = 0;
  SYMS_U64 shift = 0;
  SYMS_U8 byte = 0;
  for(SYMS_U64 read_offset = offset;
      syms_based_range_read_struct(base, range, read_offset, &byte) == 1;
      read_offset += 1)
  {
    bytes_read += 1;
    SYMS_U8 val = byte & 0x7fu;
    value |= ((SYMS_U64)val) << shift;
    shift += 7u;
    if((byte&0x80u) == 0)
    {
      if(shift < sizeof(value) * 8 && (byte & 0x40u) != 0)
      {
        value |= -(SYMS_S64)(1ull << shift);
      }
      break;
    }
  }
  if(out_value != 0)
  {
    *out_value = value;
  }
  return bytes_read;
}

SYMS_API SYMS_String8
syms_based_range_read_string(void *base, SYMS_U64Range range, SYMS_U64 offset){
  SYMS_String8 result = {0};
  char *ptr = (char *)syms_based_range_ptr(base, range, offset);
  if (ptr != 0){
    char *first = ptr;
    char *opl = (char*)base + range.max;
    for (;ptr < opl && *ptr != 0; ptr += 1);
    result.str = (SYMS_U8*)first;
    result.size = (SYMS_U64)(ptr - first);
  }
  return(result);
}

////////////////////////////////
//~ allen: Memory Views

SYMS_API SYMS_MemoryView
syms_memory_view_make(SYMS_String8 data, SYMS_U64 base){
  SYMS_MemoryView result = {0};
  result.data = data.str;
  result.addr_first = base;
  result.addr_opl = base + data.size;
  return(result);
}

SYMS_API SYMS_B32
syms_memory_view_read(SYMS_MemoryView *memview, SYMS_U64 addr, SYMS_U64 size, void *ptr){
  SYMS_B32 result = syms_false;
  if (memview->addr_first <= addr &&
      addr <= addr + size &&
      addr + size <= memview->addr_opl){
    result = syms_true;
    syms_memmove(ptr, (SYMS_U8*)memview->data + addr - memview->addr_first, size);
  }
  return(result);
}

SYMS_API void
syms_unwind_result_missed_read(SYMS_UnwindResult *unwind_result, SYMS_U64 addr){
  unwind_result->dead = syms_true;
  unwind_result->missed_read = syms_true;
  unwind_result->missed_read_addr = addr;
}

////////////////////////////////
//~ nick: Bit manipulations

SYMS_API SYMS_U16
syms_bswap_u16(SYMS_U16 x)
{
  SYMS_U16 result = (((x & 0xFF00) >> 8) |
                     ((x & 0x00FF) << 8));
  return result;
}

SYMS_API SYMS_U32
syms_bswap_u32(SYMS_U32 x)
{
  SYMS_U32 result = (((x & 0xFF000000) >> 24) |
                     ((x & 0x00FF0000) >> 8)  |
                     ((x & 0x0000FF00) << 8)  |
                     ((x & 0x000000FF) << 24));
  return result;
}

SYMS_API SYMS_U64
syms_bswap_u64(SYMS_U64 x)
{
  // TODO(nick): naive bswap, replace with something that is faster like an intrinsic
  SYMS_U64 result = (((x & 0xFF00000000000000ULL) >> 56) |
                     ((x & 0x00FF000000000000ULL) >> 40) |
                     ((x & 0x0000FF0000000000ULL) >> 24) |
                     ((x & 0x000000FF00000000ULL) >> 8)  |
                     ((x & 0x00000000FF000000ULL) << 8)  |
                     ((x & 0x0000000000FF0000ULL) << 24) |
                     ((x & 0x000000000000FF00ULL) << 40) |
                     ((x & 0x00000000000000FFULL) << 56));
  return result;
}

SYMS_API void
syms_bswap_bytes(void *p, SYMS_U64 size)
{
  for(SYMS_U32 i = 0, k = size - 1; i < size / 2; ++i, --k)
  {
    SYMS_Swap(SYMS_U8, ((SYMS_U8*)p)[k], ((SYMS_U8*)p)[i]);
  }
}

////////////////////////////////
//~ rjf: Generated Code

#include "syms/core/generated/syms_meta_base.c"
#include "syms/core/generated/syms_meta_serial_base.c"

////////////////////////////////
//~ allen: Dev Features

#include "syms_dev.c"

#endif // SYMS_BASE_C
