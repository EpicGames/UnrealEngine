// Copyright Epic Games, Inc. All Rights Reserved.
/* date = March 31st 2021 0:31 pm */

#ifndef SYMS_BASE_H
#define SYMS_BASE_H

////////////////////////////////
//~ allen: Version

#define SYMS_VERSION_MAJOR    1
#define SYMS_VERSION_MINOR    0
#define SYMS_VERSION_SUBMINOR 0
#define SYMS_VERSION_STR      "1.0.0"

////////////////////////////////
//~ rjf: Context Cracking

#include "syms_base_context_crack.h"

////////////////////////////////
//~ rjf: Overrideables

#include "syms_base_overrides_check.h"

////////////////////////////////
//~ allen: Linkage Macros

#if !defined(SYMS_API)
# define SYMS_API static
#endif

#if SYMS_LANG_CPP
# define SYMS_C_LINKAGE_BEGIN extern "C" {
# define SYMS_C_LINKAGE_END   }
# define SYMS_EXTERN extern "C"
#else
# define SYMS_C_LINKAGE_BEGIN
# define SYMS_C_LINKAGE_END
# define SYMS_EXTERN extern
#endif

#define SYMS_GLOBAL static
#define SYMS_LOCAL  static

#if !defined(SYMS_READ_ONLY)
# if SYMS_COMPILER_CL || (SYMS_COMPILER_CLANG && SYMS_OS_WINDOWS)
#  pragma section(".roglob", read)
#  define SYMS_READ_ONLY __declspec(allocate(".roglob"))
# elif (SYMS_COMPILER_CLANG && SYMS_OS_LINUX)
#  define SYMS_READ_ONLY __attribute__((section(".rodata")))
# else
// NOTE(rjf): I don't know of a useful way to do this in GCC land.
// __attribute__((section(".rodata"))) looked promising, but it introduces a
// strange warning about malformed section attributes, and it doesn't look
// like writing to that section reliably produces access violations, strangely
// enough. (It does on Clang)
#  define SYMS_READ_ONLY
# endif
#endif

#if SYMS_COMPILER_CL
# define SYMS_THREAD_LOCAL __declspec(thread)
#elif SYMS_COMPILER_CLANG || SYMS_COMPILER_GCC
# define SYMS_THREAD_LOCAL __thread
#else
# error No SYMS_THREAD_LOCAL for this compiler
#endif

////////////////////////////////
//~ allen: Feature Macros

#if !defined(SYMS_PARANOID)
# define SYMS_PARANOID 0
#endif
#if !defined(SYMS_ASSERT_INVARIANTS)
# define SYMS_ASSERT_INVARIANTS 0
#endif
#if !defined(SYMS_DISABLE_NORMAL_ASSERTS)
# define SYMS_DISABLE_NORMAL_ASSERTS 0
#endif
#if !defined(SYMS_ENABLE_DEV_SRCLOC)
# define SYMS_ENABLE_DEV_SRCLOC 0
#endif
#if !defined(SYMS_ENABLE_DEV_STRING)
# define SYMS_ENABLE_DEV_STRING 0
#endif
#if !defined(SYMS_ENABLE_DEV_LOG)
# define SYMS_ENABLE_DEV_LOG 0
#endif
#if !defined(SYMS_ENABLE_DEV_PROFILE)
# define SYMS_ENABLE_DEV_PROFILE 0
#endif

////////////////////////////////
//~ rjf: Base Types and Constants

typedef SYMS_S8  SYMS_B8;
typedef SYMS_S16 SYMS_B16;
typedef SYMS_S32 SYMS_B32;
typedef SYMS_S64 SYMS_B64;
typedef float    SYMS_F32;
typedef double   SYMS_F64;

#define syms_false 0
#define syms_true 1

typedef SYMS_U32 SYMS_RegID;

////////////////////////////////
//~ allen: Macros

#if !defined(syms_write_srcloc__impl)
# define syms_write_srcloc__impl(f,l) ((void)0)
#endif
#if SYMS_ENABLE_DEV_SRCLOC
# define SYMS_WRITE_SRCLOC(f,l) syms_write_srcloc__impl(f,l)
#else
# define SYMS_WRITE_SRCLOC(f,l) ((void)0)
#endif


#define SYMS_ASSERT_RAW(x) do { if(!(x)) { SYMS_ASSERT_BREAK(x); } } while(0)

#if !SYMS_DISABLE_NORMAL_ASSERTS
# define SYMS_ASSERT(x) SYMS_ASSERT_RAW(x)
#else
# define SYMS_ASSERT(x)
#endif
#if SYMS_PARANOID
# define SYMS_ASSERT_PARANOID(x) SYMS_ASSERT_RAW(x)
#else
# define SYMS_ASSERT_PARANOID(x)
#endif

#if SYMS_ASSERT_INVARIANTS
# define SYMS_INVARIANT(r,x) SYMS_ASSERT_RAW(x)
#else
# define SYMS_INVARIANT(r,x) do{ if (!(x)){ (r) = syms_false; goto finish_invariants; } } while(0)
#endif

#define SYMS_NOT_IMPLEMENTED    SYMS_ASSERT_BREAK("not implemented")
#define SYMS_INVALID_CODE_PATH  SYMS_ASSERT_BREAK("invalid code path")

#define SYMS_ARRAY_SIZE(x) (sizeof(x)/sizeof((x)[0]))


#define SYMS_PTR_DIF(a, b) (SYMS_U64)((SYMS_U8*)(a) - (SYMS_U8*)(b))
#define SYMS_MEMBER(type, member) (&((type*)0)->member)
#define SYMS_MEMBER_OFFSET(type, member) (SYMS_PTR_DIF(SYMS_MEMBER(type, member), 0))

#define SYMS_KB(num) ((SYMS_U64)(num) << 10)
#define SYMS_MB(num) ((SYMS_U64)(num) << 20)
#define SYMS_GB(num) ((SYMS_U64)(num) << 30)
#define SYMS_TB(num) ((SYMS_U64)(num) << 40)

#define SYMS_Stringify_(x) #x
#define SYMS_Stringify(x) SYMS_Stringify_(x)
#define SYMS_Glue_(a,b) a ## b
#define SYMS_Glue(a,b) SYMS_Glue_(a,b)

#define syms_memzero_struct(s) syms_memset((s), 0, sizeof(*(s)))
#define syms_memisnull_struct(s) syms_memisnull((s), sizeof(*(s)))

#define SYMS_S8_MIN  0x80 // -128
#define SYMS_S8_MAX  0x7f // +127
#define SYMS_S16_MIN 0x8000 // -32768
#define SYMS_S16_MAX 0x7fff // +32767
#define SYMS_S32_MIN 0x80000000 // -2147483648
#define SYMS_S32_MAX 0x7fffffff // +2147483647
#define SYMS_S64_MIN 0x8000000000000000ull // -9223372036854775808
#define SYMS_S64_MAX 0x7fffffffffffffffull // +9223372036854775807

#define SYMS_U8_MAX  0xFFu
#define SYMS_U16_MAX 0xFFFFu
#define SYMS_U32_MAX 0xFFFFFFFFu
#define SYMS_U64_MAX 0xFFFFFFFFFFFFFFFFu

#define SYMS_STATIC_ASSERT(c) typedef char SYMS_Glue(assert, __LINE__)[(c)?+1:-1]
#define SYMS_MIN(a,b) ((a)<(b)?(a):(b))
#define SYMS_MAX(a,b) ((a)>(b)?(a):(b))

#define SYMS_ClampTop(a,b) SYMS_MIN(a,b)
#define SYMS_ClampBot(a,b) SYMS_MAX(a,b)

#define SYMS_CeilIntegerDiv(a,b) (((a) + (b) - 1)/(b))

#define SYMS_AlignPow2(a,b) (((a) + (b) - 1)&(~((b) - 1)))
#define SYMS_AlignDownPow2(a,b) ((a)&(~((b) - 1)))

#define SYMS_Swap(T,a,b) do{ T t__ = (a); (a) = (b); (b) = t__; }while(0)

#define SYMS_THIS_SRCLOC __FILE__ ":" SYMS_Stringify(__LINE__)

////////////////////////////////
//~ allen: ID Macros

#define SYMS_ID_u32_0(id) (SYMS_U32)(id)
#define SYMS_ID_u32_1(id) (SYMS_U32)((id) >> 32)
#define SYMS_ID_u16_0(id) (SYMS_U16)(id)
#define SYMS_ID_u16_1(id) (SYMS_U16)((id) >> 16)
#define SYMS_ID_u16_2(id) (SYMS_U16)((id) >> 32)
#define SYMS_ID_u16_3(id) (SYMS_U16)((id) >> 48)

#define SYMS_ID_u32_u32(a,b) ((SYMS_U64)(a) | ((SYMS_U64)(b) << 32))
#define SYMS_ID_u16_u16_u32(a,b,c) ((SYMS_U64)(a) | ((SYMS_U64)(b) << 16) | ((SYMS_U64)(c) << 32))

////////////////////////////////
//~ allen: Linked List Macros

#define SYMS_QueuePush_N(f,l,n,next) ( (f)==0?\
((f)=(l)=(n),(n)->next=0):\
((l)->next=(n),(l)=(n),(n)->next=0) )
#define SYMS_QueuePushFront_N(f,l,n,next) ( (f)==0?\
((f)=(l)=(n),(n)->next=0):\
((n)->next=(f),(f)=(n)) )
#define SYMS_QueuePop_N(f,l,next) ( (f)==(l)?\
(f)=(l)=0:\
((f)=(f)->next) )

#define SYMS_QueuePush(f,l,n) SYMS_QueuePush_N(f,l,n,next)
#define SYMS_QueuePushFront(f,l,n) SYMS_QueuePushFront_N(f,l,n,next)
#define SYMS_QueuePop(f,l) SYMS_QueuePop_N(f,l,next)

#define SYMS_StackPush_N(f,n,next) ( (n)->next=(f), (f)=(n) )
#define SYMS_StackPop_N(f,next) ( (f)=(f)->next )

#define SYMS_StackPush(f,n) SYMS_StackPush_N(f,n,next)
#define SYMS_StackPop(f) SYMS_StackPop_N(f,next)

////////////////////////////////
//~ allen: Common Basic Types

typedef struct SYMS_U64Array{
  SYMS_U64 *u64;
  SYMS_U64 count;
} SYMS_U64Array;

typedef struct SYMS_U32Array{
  SYMS_U32 *u32;
  SYMS_U64 count;
} SYMS_U32Array;

typedef struct SYMS_U16Array{
  SYMS_U16 *u16;
  SYMS_U64 count;
} SYMS_U16Array;

typedef struct SYMS_U64ArrayNode{
  struct SYMS_U64ArrayNode *next;
  SYMS_U64 *u64;
  SYMS_U64 count;
} SYMS_U64ArrayNode;

typedef struct SYMS_U64Maybe{
  SYMS_B32 valid;
  SYMS_U64 u64;
} SYMS_U64Maybe;

typedef struct SYMS_U32Range{
  SYMS_U32 min;
  SYMS_U32 max;
} SYMS_U32Range;

typedef struct SYMS_U64Range{
  SYMS_U64 min;
  SYMS_U64 max;
} SYMS_U64Range;

typedef struct SYMS_U64RangeNode{
  struct SYMS_U64RangeNode *next;
  SYMS_U64Range range;
} SYMS_U64RangeNode;

typedef struct SYMS_U64RangeList{
  SYMS_U64RangeNode *first;
  SYMS_U64RangeNode *last;
  SYMS_U64 node_count;
} SYMS_U64RangeList;

typedef struct SYMS_U64RangeArray{
  SYMS_U64Range *ranges;
  SYMS_U64 count;
} SYMS_U64RangeArray;

typedef struct SYMS_U64Node{
  struct SYMS_U64Node *next;
  SYMS_U64 u64;
} SYMS_U64Node;

typedef struct SYMS_U64List{
  SYMS_U64Node *first;
  SYMS_U64Node *last;
  SYMS_U64 count;
} SYMS_U64List;

typedef struct SYMS_String8{
  SYMS_U8 *str;
  SYMS_U64 size;
} SYMS_String8;

typedef struct SYMS_String8Node{
  struct SYMS_String8Node *next;
  SYMS_String8 string;
} SYMS_String8Node;

typedef struct SYMS_String8List{
  SYMS_String8Node *first;
  SYMS_String8Node *last;
  SYMS_U64 node_count;
  SYMS_U64 total_size;
} SYMS_String8List;

typedef struct SYMS_String8Array{
  SYMS_String8 *strings;
  SYMS_U64 count;
} SYMS_String8Array;

typedef SYMS_U32 SYMS_StringMatchFlags;
enum{
  SYMS_StringMatchFlag_CaseInsensitive  = (1<<0),
  SYMS_StringMatchFlag_RightSideSloppy  = (1<<1),
  SYMS_StringMatchFlag_SlashInsensitive = (1<<2),
};

typedef struct SYMS_StringJoin{
  SYMS_String8 pre;
  SYMS_String8 sep;
  SYMS_String8 post;
} SYMS_StringJoin;

////////////////////////////////
//~ allen: Syms Sort Node

typedef struct SYMS_SortNode{
  struct SYMS_SortNode *next;
  SYMS_U64 first;
  SYMS_U64 opl;
} SYMS_SortNode;

////////////////////////////////
//~ allen: Generated Types

#include "syms/core/generated/syms_meta_base.h"

////////////////////////////////
//~ rjf: Serial Information

typedef enum SYMS_SerialWidthKind{
  SYMS_SerialWidthKind_Null,
  SYMS_SerialWidthKind_Fixed,
  SYMS_SerialWidthKind_Array,
  SYMS_SerialWidthKind_NullTerminated,
  SYMS_SerialWidthKind_SequenceNullTerminated,
  SYMS_SerialWidthKind_RestOfStream,
  SYMS_SerialWidthKind_PdbNumeric,
  SYMS_SerialWidthKind_COUNT
} SYMS_SerialWidthKind;

typedef struct SYMS_SerialField{
  SYMS_String8 name;
  struct SYMS_SerialType *type;
  SYMS_SerialWidthKind width_kind;
  SYMS_U32 width_var;
} SYMS_SerialField;

typedef struct SYMS_SerialValue{
  SYMS_String8 name;
  SYMS_U64 value;
} SYMS_SerialValue;

typedef struct SYMS_SerialFlag{
  SYMS_String8 name;
  struct SYMS_SerialType *type;
  SYMS_U32 mask;
  SYMS_U32 bitshift;
} SYMS_SerialFlag;

typedef enum SYMS_SerialTypeKind{
  SYMS_SerialTypeKind_Null,
  SYMS_SerialTypeKind_Integer,
  SYMS_SerialTypeKind_UnsignedInteger,
  SYMS_SerialTypeKind_Character,
  SYMS_SerialTypeKind_BinaryAnnotation,
  SYMS_SerialTypeKind_Enum,
  SYMS_SerialTypeKind_Flags,
  SYMS_SerialTypeKind_Struct,
  SYMS_SerialTypeKind_COUNT
} SYMS_SerialTypeKind;

typedef struct SYMS_SerialType{
  SYMS_String8 name;
  SYMS_SerialTypeKind kind;
  SYMS_U32 child_count;
  void *children;
  SYMS_U64 basic_size;
  SYMS_U64 (*enum_index_from_value)(SYMS_U64 value);
} SYMS_SerialType;

////////////////////////////////
//~ allen: Syms Arena

typedef struct SYMS_ArenaTemp{
  SYMS_Arena *arena;
  SYMS_U64 pos;
} SYMS_ArenaTemp;

////////////////////////////////
//~ allen: Memory Views

typedef struct SYMS_MemoryView{
  // allen: Upgrade path:
  //  1. A list of ranges like this one
  //  2. After building the list put into a binary-searchable format
  //  3. Equip with the ability to request missing memory in-line.
  void *data;
  SYMS_U64 addr_first;
  SYMS_U64 addr_opl;
} SYMS_MemoryView;

typedef struct SYMS_UnwindResult{
  SYMS_B32 dead;
  SYMS_B32 missed_read;
  SYMS_U64 missed_read_addr;
  SYMS_U64 stack_pointer;
} SYMS_UnwindResult;


////////////////////////////////
//~ allen: Generated Serial Info

#include "syms/core/generated/syms_meta_serial_base.h"


SYMS_C_LINKAGE_BEGIN

////////////////////////////////
//~ rjf: Library Metadata

SYMS_API SYMS_String8 syms_version_string(void);

////////////////////////////////
//~ rjf: Basic Type Functions

SYMS_API SYMS_U64Range syms_make_u64_range(SYMS_U64 min, SYMS_U64 max);
SYMS_API SYMS_U64Range syms_make_u64_inrange(SYMS_U64Range range, SYMS_U64 offset, SYMS_U64 size);
SYMS_API SYMS_U64 syms_u64_range_size(SYMS_U64Range range);

////////////////////////////////
//~ allen: Hash Functions

SYMS_GLOBAL SYMS_U64 syms_hash_djb2_initial = 5381;
SYMS_API SYMS_U64 syms_hash_djb2(SYMS_String8 string);
SYMS_API SYMS_U64 syms_hash_djb2_continue(SYMS_String8 string, SYMS_U64 intermediate_hash);

SYMS_API SYMS_U64 syms_hash_u64(SYMS_U64 x);

////////////////////////////////
//~ rjf: Serial Information Functions

#define syms_serial_type(name) (_syms_serial_type_##name)
#define syms_string_from_enum_value(enum_type, value) \
(syms_serial_value_from_enum_value(&syms_serial_type(enum_type), value)->name)
#define syms_bswap_in_place(type, ptr) syms_bswap_in_place__##type((type*)(ptr))

SYMS_API SYMS_SerialField* syms_serial_first_field(SYMS_SerialType *type);
SYMS_API SYMS_SerialValue* syms_serial_first_value(SYMS_SerialType *type);
SYMS_API SYMS_SerialFlag*  syms_serial_first_flag(SYMS_SerialType *type);

SYMS_API SYMS_SerialValue* syms_serial_value_from_enum_value(SYMS_SerialType *type, SYMS_U64 value);
SYMS_API SYMS_SerialFlag*  syms_serial_flag_from_bit_offset(SYMS_SerialType *type, SYMS_U64 bit_off);

SYMS_API SYMS_String8List  syms_string_list_from_flags(SYMS_Arena *arena, SYMS_SerialType *type, SYMS_U32 flags);

SYMS_API SYMS_U64 syms_enum_index_from_value_identity(SYMS_U64 v);

////////////////////////////////
//~ allen: String Functions

SYMS_API SYMS_B32    syms_codepoint_is_whitespace(SYMS_U32 codepoint);
SYMS_API SYMS_U32    syms_lowercase_from_codepoint(SYMS_U32 codepoint);

#define syms_str8_comp(s) { (SYMS_U8 *)(s), sizeof(s) - 1 }
#define syms_str8_lit(s) syms_str8((SYMS_U8 *)(s), sizeof(s) - 1)
#define syms_expand_string(s) (int)((s).size), ((s).str)

SYMS_API SYMS_String8 syms_str8(SYMS_U8 *str, SYMS_U64 size);
SYMS_API SYMS_String8 syms_str8_cstring(char *str);
SYMS_API SYMS_String8 syms_str8_range(SYMS_U8 *first, SYMS_U8 *opl);

SYMS_API SYMS_String8 syms_str8_skip_chop_whitespace(SYMS_String8 str);

SYMS_API SYMS_B32     syms_string_match(SYMS_String8 a, SYMS_String8 b, SYMS_StringMatchFlags flags);
SYMS_API SYMS_U8*     syms_decode_utf8(SYMS_U8 *p, SYMS_U32 *dst);

SYMS_API void         syms_string_list_push_node(SYMS_String8Node *node, SYMS_String8List *list,
                                                 SYMS_String8 string);
SYMS_API void         syms_string_list_push_node_front(SYMS_String8Node *node, SYMS_String8List *list,
                                                       SYMS_String8 string);
SYMS_API void         syms_string_list_push(SYMS_Arena *arena, SYMS_String8List *list, SYMS_String8 string);
SYMS_API void         syms_string_list_push_front(SYMS_Arena *arena, SYMS_String8List *list, SYMS_String8 string);
SYMS_API SYMS_String8List syms_string_list_concat(SYMS_String8List *left, SYMS_String8List *right);


SYMS_API SYMS_String8 syms_string_list_join(SYMS_Arena *arena, SYMS_String8List *list, SYMS_StringJoin *join);
SYMS_API SYMS_String8 syms_push_string_copy(SYMS_Arena *arena, SYMS_String8 string);

SYMS_API SYMS_String8 syms_string_trunc_symbol_heuristic(SYMS_String8 string);

SYMS_API SYMS_String8List syms_string_split(SYMS_Arena *arena, SYMS_String8 input, SYMS_U32 delimiter);

////////////////////////////////
//~ allen: String <-> Integer

SYMS_API SYMS_U64 syms_u64_from_string(SYMS_String8 str, SYMS_U32 radix);
SYMS_API SYMS_S64 syms_s64_from_string_c_rules(SYMS_String8 str);

SYMS_API SYMS_String8 syms_string_from_u64(SYMS_Arena *arena, SYMS_U64 x);

////////////////////////////////
//~ rjf: U64 Range Functions

SYMS_API void syms_u64_range_list_push_node(SYMS_U64RangeNode *node, SYMS_U64RangeList *list, SYMS_U64Range range);
SYMS_API void syms_u64_range_list_push(SYMS_Arena *arena, SYMS_U64RangeList *list, SYMS_U64Range range);
SYMS_API void syms_u64_range_list_concat(SYMS_U64RangeList *list, SYMS_U64RangeList *to_push);

SYMS_API SYMS_U64RangeArray syms_u64_range_array_from_list(SYMS_Arena *arena, SYMS_U64RangeList *list);

////////////////////////////////
//~ nick: U64 List Functions

SYMS_API void syms_u64_list_push_node(SYMS_U64Node *node, SYMS_U64List *list, SYMS_U64 v);
SYMS_API void syms_u64_list_push(SYMS_Arena *arena, SYMS_U64List *list, SYMS_U64 v);
SYMS_API void syms_u64_list_concat_in_place(SYMS_U64List *dst, SYMS_U64List *src);
SYMS_API SYMS_U64Array syms_u64_array_from_list(SYMS_Arena *arena, SYMS_U64List *list);

////////////////////////////////
//~ allen: Array Functions

SYMS_API SYMS_U64 syms_1based_checked_lookup_u64(SYMS_U64 *u64, SYMS_U64 count, SYMS_U64 n);

////////////////////////////////
//~ rjf: Memory/Arena Functions

#define syms_arena_alloc          syms_arena_alloc__impl
#define syms_arena_release        syms_arena_release__impl
#define syms_arena_get_pos        syms_arena_get_pos__impl
#define syms_arena_push           syms_arena_push__impl
#define syms_arena_pop_to         syms_arena_pop_to__impl
#define syms_arena_set_auto_align syms_arena_set_auto_align__impl
#define syms_arena_absorb         syms_arena_absorb__impl
#define syms_arena_tidy           syms_arena_tidy__impl

#define syms_get_implicit_thread_arena syms_get_implicit_thread_arena__impl

SYMS_API void            syms_arena_push_align(SYMS_Arena *arena, SYMS_U64 boundary);
SYMS_API void            syms_arena_put_back(SYMS_Arena *arena, SYMS_U64 amount);

SYMS_API SYMS_ArenaTemp  syms_arena_temp_begin(SYMS_Arena *arena);
SYMS_API void            syms_arena_temp_end(SYMS_ArenaTemp frame);

#define syms_push_array(a,T,c) (SYMS_WRITE_SRCLOC(__FILE__, __LINE__),\
(T*)syms_arena_push((a), sizeof(T)*(c)))

#define syms_push_array_zero(a,T,c) ((T*)syms_memset(syms_push_array(a,T,c), 0, sizeof(T)*(c)))

SYMS_API SYMS_ArenaTemp syms_get_scratch(SYMS_Arena **conflicts, SYMS_U64 count);
#define syms_release_scratch syms_arena_temp_end

#define syms_scratch_pool_tidy syms_scratch_pool_tidy__impl

////////////////////////////////
//~ allen: Syms Sort Node

SYMS_API SYMS_SortNode* syms_sort_node_push(SYMS_Arena *arena, SYMS_SortNode **stack, SYMS_SortNode **free_stack,
                                            SYMS_U64 first, SYMS_U64 opl);

////////////////////////////////
//~ allen: Thread Lanes

SYMS_API void     syms_set_lane(SYMS_U64 lane);
SYMS_API SYMS_U64 syms_get_lane(void);

////////////////////////////////
//~ rjf: Based Ranges

SYMS_API void *   syms_based_range_ptr(void *base, SYMS_U64Range range, SYMS_U64 offset);
SYMS_API SYMS_U64 syms_based_range_read(void *base, SYMS_U64Range range, SYMS_U64 offset, SYMS_U64 out_size, void *out);
SYMS_API SYMS_U64 syms_based_range_read_uleb128(void *base, SYMS_U64Range range, SYMS_U64 offset, SYMS_U64 *out_value);
SYMS_API SYMS_U64 syms_based_range_read_sleb128(void *base, SYMS_U64Range range, SYMS_U64 offset, SYMS_S64 *out_value);

SYMS_API SYMS_String8 syms_based_range_read_string(void *base, SYMS_U64Range range, SYMS_U64 offset);

#define syms_based_range_read_struct(b,r,o,p) syms_based_range_read((b), (r), (o), sizeof(*(p)), p)

////////////////////////////////
//~ allen: Memory Views

SYMS_API SYMS_MemoryView syms_memory_view_make(SYMS_String8 data, SYMS_U64 base);
SYMS_API SYMS_B32        syms_memory_view_read(SYMS_MemoryView *memview, SYMS_U64 addr,
                                               SYMS_U64 size, void *ptr);

#define syms_memory_view_read_struct(s,a,p) syms_memory_view_read((s),(a),sizeof(*(p)),(p))

SYMS_API void syms_unwind_result_missed_read(SYMS_UnwindResult *unwind_result, SYMS_U64 addr);

////////////////////////////////
//~ nick: Bit manipulations

SYMS_API SYMS_U16 syms_bswap_u16(SYMS_U16 x);
SYMS_API SYMS_U32 syms_bswap_u32(SYMS_U32 x);
SYMS_API SYMS_U64 syms_bswap_u64(SYMS_U64 x);
SYMS_API void syms_bswap_bytes(void *p, SYMS_U64 size);

////////////////////////////////
//~ allen: Dev Features

#include "syms_dev.h"

SYMS_C_LINKAGE_END

#endif // SYMS_BASE_H
