// Copyright Epic Games, Inc. All Rights Reserved.
/* date = July 27th 2021 10:53 am */

#ifndef SYMS_BASE_CONTEXT_CRACK_H
#define SYMS_BASE_CONTEXT_CRACK_H

#if defined(__clang__)

# define SYMS_COMPILER_CLANG 1

# if defined(_WIN32)
#  define SYMS_OS_WINDOWS 1
# elif defined(__gnu_linux__) || defined(__linux__)
#  define SYMS_OS_LINUX 1
# elif defined(__APPLE__) && defined(__MACH__)
#  define SYMS_OS_MAC 1
# else
#  error SYMS: Build compiler/platform combo is not supported yet
# endif

# if defined(__amd64__) || defined(__amd64) || defined(__x86_64__) || defined(__x86_64)
#  define SYMS_ARCH_X64 1
# elif defined(i386) || defined(__i386) || defined(__i386__)
#  define SYMS_ARCH_X86 1
# elif defined(__aarch64__)
#  define SYMS_ARCH_ARM64 1
# elif defined(__arm__)
#  define SYMS_ARCH_ARM32 1
# else
#  error SYMS: Build architecture not supported yet
# endif

#elif defined(_MSC_VER)

# define SYMS_COMPILER_CL 1

# if defined(_WIN32)
#  define SYMS_OS_WINDOWS 1
# else
#  error SYMS: Build compiler/platform combo is not supported yet
# endif

# if defined(_M_AMD64)
#  define SYMS_ARCH_X64 1
# elif defined(_M_IX86)
#  define SYMS_ARCH_X86 1
# elif defined(_M_ARM64)
#  define SYMS_ARCH_ARM64 1
# elif defined(_M_ARM)
#  define SYMS_ARCH_ARM32 1
# else
#  error SYMS: Build architecture not supported yet
# endif

#elif defined(__GNUC__) || defined(__GNUG__)

# define SYMS_COMPILER_GCC 1

# if defined(__gnu_linux__) || defined(__linux__)
#  define SYMS_OS_LINUX 1
# else
#  error SYMS: Build compiler/platform combo is not supported yet
# endif

# if defined(__amd64__) || defined(__amd64) || defined(__x86_64__) || defined(__x86_64)
#  define SYMS_ARCH_X64 1
# elif defined(i386) || defined(__i386) || defined(__i386__)
#  define SYMS_ARCH_X86 1
# elif defined(__aarch64__)
#  define SYMS_ARCH_ARM64 1
# elif defined(__arm__)
#  define SYMS_ARCH_ARM32 1
# else
#  error SYMS: Build architecture not supported yet
# endif

#else
# error SYMS: Build compiler is not supported yet
#endif

#if defined(SYMS_ARCH_X64)
# define SYMS_ARCH_64BIT 1
#elif defined(SYMS_ARCH_X86)
# define SYMS_ARCH_32BIT 1
#endif

#if defined(__cplusplus)
# define SYMS_LANG_CPP 1
#else
# define SYMS_LANG_C 1
#endif

// zeroify

#if !defined(SYMS_ARCH_32BIT)
# define SYMS_ARCH_32BIT 0
#endif
#if !defined(SYMS_ARCH_64BIT)
# define SYMS_ARCH_64BIT 0
#endif
#if !defined(SYMS_ARCH_X64)
# define SYMS_ARCH_X64 0
#endif
#if !defined(SYMS_ARCH_X86)
# define SYMS_ARCH_X86 0
#endif
#if !defined(SYMS_ARCH_ARM64)
# define SYMS_ARCH_ARM64 0
#endif
#if !defined(SYMS_ARCH_ARM32)
# define SYMS_ARCH_ARM32 0
#endif
#if !defined(SYMS_COMPILER_CL)
# define SYMS_COMPILER_CL 0
#endif
#if !defined(SYMS_COMPILER_GCC)
# define SYMS_COMPILER_GCC 0
#endif
#if !defined(SYMS_COMPILER_CLANG)
# define SYMS_COMPILER_CLANG 0
#endif
#if !defined(SYMS_OS_WINDOWS)
# define SYMS_OS_WINDOWS 0
#endif
#if !defined(SYMS_OS_LINUX)
# define SYMS_OS_LINUX 0
#endif
#if !defined(SYMS_OS_MAC)
# define SYMS_OS_MAC 0
#endif
#if !defined(SYMS_LANG_CPP)
# define SYMS_LANG_CPP 0
#endif
#if !defined(SYMS_LANG_C)
# define SYMS_LANG_C 0
#endif

#if defined(SYMS_ARCH_X64) || defined(SYMS_ARCH_X86) || defined(SYMS_ARCH_ARM32) || defined(SYMS_ARCH_ARM64)
#  define SYMS_LITTLE_ENDIAN 1
#else
#  error "undefined endianness"
#endif

#endif //SYMS_BASE_CONTEXT_CRACK_H
