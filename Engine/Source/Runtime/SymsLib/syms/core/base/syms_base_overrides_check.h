// Copyright Epic Games, Inc. All Rights Reserved.
/* date = July 27th 2021 10:54 am */

#ifndef SYMS_BASE_OVERRIDES_CHECK_H
#define SYMS_BASE_OVERRIDES_CHECK_H

// This will check every possible overridable that could be provided by the
// user. All overrideables are required at this layer.


// NOTE(allen): basic types

#if !defined(SYMS_S8)
# error "SYMS: SYMS_S8 was not #define'd. It should be #define'd to the name of a signed 8-bit integer type (int8_t in the C Standard Library)."
#endif

#if !defined(SYMS_S16)
# error "SYMS: SYMS_S16 was not #define'd. It should be #define'd to the name of a signed 16-bit integer type (int16_t in the C Standard Library)."
#endif

#if !defined(SYMS_S32)
# error "SYMS: SYMS_S32 was not #define'd. It should be #define'd to the name of a signed 32-bit integer type (int32_t in the C Standard Library)."
#endif

#if !defined(SYMS_S64)
# error "SYMS: SYMS_S64 was not #define'd. It should be #define'd to the name of a signed 64-bit integer type (int64_t in the C Standard Library)."
#endif

#if !defined(SYMS_U8)
# error "SYMS: SYMS_U8 was not #define'd. It should be #define'd to the name of an unsigned 8-bit integer type (uint8_t in the C Standard Library)."
#endif

#if !defined(SYMS_U16)
# error "SYMS: SYMS_U16 was not #define'd. It should be #define'd to the name of an unsigned 16-bit integer type (uint16_t in the C Standard Library)."
#endif

#if !defined(SYMS_U32)
# error "SYMS: SYMS_U32 was not #define'd. It should be #define'd to the name of an unsigned 32-bit integer type (uint32_t in the C Standard Library)."
#endif

#if !defined(SYMS_U64)
# error "SYMS: SYMS_U64 was not #define'd. It should be #define'd to the name of an unsigned 64-bit integer type (uint64_t in the C Standard Library)."
#endif

#if !defined(SYMS_PRIu64)
# error "SYMS: SYMS_PRIu64 was not #define'd. It should be #define'd to a string constant suitable for printing 64-bit unsigned integers in C format strings (PRIu64 in the C Standard Library)."
#endif

#if !defined(SYMS_PRId64)
# error "SYMS: SYMS_PRId64 was not #define'd. It should be #define'd to a string constant suitable for printing 64-bit signed integers in C format strings (PRId64 in the C Standard Library)."
#endif

#if !defined(SYMS_PRIx64)
# error "SYMS: SYMS_PRIx64 was not #define'd. It should be #define'd to a string constant suitable for printing 64-bit hex integers in C format strings (PRIx64 in the C Standard Library)."
#endif


// NOTE(allen): large data operations

#if !defined(syms_memmove)
# error "SYMS: syms_memmove was not #define'd. It should be #define'd to the name of a memmove implementation."
#endif

#if !defined(syms_memset)
# error "SYMS: syms_memset was not #define'd. It should be #define'd to the name of a memset implementation."
#endif

#if !defined(syms_memcmp)
# error "SYMS: syms_memcmp was not #define'd. It should be #define'd to the name of a memcmp implementation."
#endif

#if !defined(syms_strlen)
# error "SYMS: syms_strlen was not #define'd. It should be #define'd to the name of a strlen implementation."
#endif

#if !defined(syms_memisnull)
# error "SYMS: syms_memisnull was not #define'd. It should be #define'd to the name of a memisnull implementation."
#endif


// NOTE(allen): assert

#if !defined(SYMS_ASSERT_BREAK)
# error "SYMS: SYMS_ASSERT_BREAK(message) was not #define'd. It should be #define'd to something that you'd like expanded when an assertion failure occurs."
#endif


// NOTE(allen): arena

#if !defined(SYMS_Arena)
# error "SYMS: SYMS_Arena was not #define'd."
#endif

#if !defined(syms_arena_alloc__impl)
# error "SYMS: syms_arena_alloc__impl was not #define'd."
#endif

#if !defined(syms_arena_release__impl)
# error "SYMS: syms_arena_release__impl was not #define'd."
#endif

#if !defined(syms_arena_get_pos__impl)
# error "SYMS: syms_arena_get_pos__impl was not #define'd."
#endif

#if !defined(syms_arena_push__impl)
# error "SYMS: syms_arena_push__impl was not #define'd."
#endif

#if !defined(syms_arena_pop_to__impl)
# error "SYMS: syms_arena_pop_to__impl was not #define'd."
#endif

#if !defined(syms_arena_set_auto_align__impl)
# error "SYMS: syms_arena_set_auto_align__impl was not #define'd."
#endif

#if !defined(syms_arena_absorb__impl)
# error "SYMS: syms_arena_absorb__impl was not #define'd."
#endif

#if !defined(syms_arena_tidy__impl)
# error "SYMS: syms_arena_tidy__impl was not #define'd."
#endif

#if !defined(syms_get_implicit_thread_arena__impl)
# error "SYMS: syms_get_implicit_thread_arena__impl was not #define'd."
#endif

#if !defined(syms_scratch_pool_tidy__impl)
# error "SYMS: syms_scratch_pool_tidy__impl was not #define'd."
#endif

#endif //SYMS_BASE_OVERRIDES_CHECK_H
