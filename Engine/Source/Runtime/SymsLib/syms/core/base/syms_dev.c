// Copyright Epic Games, Inc. All Rights Reserved.

#ifndef SYMS_DEV_C
#define SYMS_DEV_C

////////////////////////////////
// NOTE(allen): Dev String Functions

#if SYMS_ENABLE_DEV_STRING

SYMS_API SYMS_String8
syms_push_stringfv__dev(SYMS_Arena *arena, char *fmt, va_list args){
  // TODO(allen): optimize with initial guess allocation technique
  va_list args2;
  va_copy(args2, args);
  SYMS_U32 needed_bytes = vsnprintf(0, 0, fmt, args) + 1;
  SYMS_String8 result = {0};
  result.str = syms_push_array(arena, SYMS_U8, needed_bytes);
  result.size = vsnprintf((char*)result.str, needed_bytes, fmt, args2);
  result.str[result.size] = 0;
  va_end(args2);
  return(result);
}

SYMS_API SYMS_String8
syms_push_stringf__dev(SYMS_Arena *arena, char *fmt, ...){
  va_list args;
  va_start(args, fmt);
  SYMS_String8 result = syms_push_stringfv__dev(arena, fmt, args);
  va_end(args);
  return(result);
}

SYMS_API void
syms_string_list_pushfv__dev(SYMS_Arena *arena, SYMS_String8List *list, char *fmt, va_list args){
  SYMS_String8 string = syms_push_stringfv__dev(arena, fmt, args);
  syms_string_list_push(arena, list, string);
}

SYMS_API void
syms_string_list_pushf__dev(SYMS_Arena *arena, SYMS_String8List *list, char *fmt, ...){
  va_list args;
  va_start(args, fmt);
  syms_string_list_pushfv__dev(arena, list, fmt, args);
  va_end(args);
}

#endif

////////////////////////////////
// NOTE(allen): Dev Logging Functions

#if SYMS_ENABLE_DEV_LOG

SYMS_GLOBAL SYMS_LogFeatures syms_log_filter_features__dev = SYMS_LOG_FILTER_FEATURES;
SYMS_GLOBAL SYMS_U64 syms_log_filter_uid__dev = SYMS_LOG_FILTER_UID;

SYMS_API void
syms_log_set_filter__dev(SYMS_LogFeatures features, SYMS_U64 uid){
  syms_log_filter_features__dev = features;
  syms_log_filter_uid__dev = uid;
}


// states: 0 = not opened; 1 = opened disabled; 2 = opened enabled
// transition table: (0/2,enabled) -> 2; (0/2,disabled) -> 1; (1,*) -> 1
// new states are treated as a stack and are "popped" by close
SYMS_THREAD_LOCAL SYMS_U32 syms_log_state__dev = 0;

SYMS_API SYMS_U32
syms_log_open__dev(SYMS_B32 enabled){
  SYMS_U32 prev_state = syms_log_state__dev;
  SYMS_B32 new_is_enabled = (enabled && prev_state != 1);
  SYMS_U32 new_state = new_is_enabled ? 2 : 1;
  syms_log_state__dev = new_state;
  return(prev_state);
}

SYMS_API void
syms_log_close__dev(SYMS_U32 prev_state){
  syms_log_state__dev = prev_state;
}

SYMS_API SYMS_B32
syms_log_is_enabled__dev(void){
  SYMS_B32 result = (syms_log_state__dev == 2);
  return(result);
}

SYMS_API SYMS_U32
syms_log_open_annotated__dev(SYMS_LogFeatures features, SYMS_U64 uid){
  SYMS_LogFeatures filter_features = syms_log_filter_features__dev;
  SYMS_U64 filter_uid = syms_log_filter_uid__dev;
  SYMS_B32 enable = ((filter_features == 0 || (filter_features & features) != 0) &&
                     (uid == 0 || filter_uid == 0 || uid == filter_uid));
  SYMS_U32 result = syms_log_open__dev(enable);
  SYMS_B32 is_enabled = (syms_log_state__dev == 2);
  if (is_enabled){
    SYMS_ArenaTemp scratch = syms_get_scratch(0, 0);
    
    //- features
    SYMS_String8List features_list = {0};
    for (SYMS_U32 i = 0; i < 1; i += 1){
      SYMS_U32 bit = (1 << i);
      if ((features & bit) != 0){
        SYMS_String8 feature_str = {0};
        switch (bit){
          case SYMS_LogFeature_LineTable:       feature_str = syms_str8_lit("line_table");        break;
          case SYMS_LogFeature_DwarfUnitRanges: feature_str = syms_str8_lit("dwarf_unit_ranges"); break;
          case SYMS_LogFeature_DwarfTags:       feature_str = syms_str8_lit("dwarf_tags");        break;
          case SYMS_LogFeature_DwarfUnwind:     feature_str = syms_str8_lit("dwarf_unwind");      break;
          case SYMS_LogFeature_DwarfCFILookup:  feature_str = syms_str8_lit("dwarf_cfi_lookup");  break;
          case SYMS_LogFeature_DwarfCFIDecode:  feature_str = syms_str8_lit("dwarf_cfi_decode");  break;
          case SYMS_LogFeature_DwarfCFIApply:   feature_str = syms_str8_lit("dwarf_cfi_apply");   break;
          case SYMS_LogFeature_PEEpilog:        feature_str = syms_str8_lit("pe_epilog");         break;
        }
        syms_string_list_push(scratch.arena, &features_list, feature_str);
      }
    }
    SYMS_String8 features = syms_str8_lit("misc");
    if (features_list.first != 0){
      SYMS_StringJoin join = {0};
      join.sep = syms_str8_lit(" ");
      features = syms_string_list_join(scratch.arena, &features_list, &join);
    }
    
    //- append block
    SYMS_String8 string = syms_push_stringf__dev(scratch.arena, "logging [%.*s %" SYMS_PRIu64 "]\n",
                                                 syms_expand_string(features), uid);
    SYMS_LOG_RAW_APPEND(string);
    syms_release_scratch(scratch);
  }
  return(result);
}

SYMS_API void
syms_logfv__dev(char *fmt, va_list args){
  SYMS_B32 is_enabled = (syms_log_state__dev == 2);
  if (is_enabled){
    SYMS_ArenaTemp scratch = syms_get_scratch(0, 0);
    SYMS_String8 string = syms_push_stringfv__dev(scratch.arena, fmt, args);
    SYMS_LOG_RAW_APPEND(string);
    syms_release_scratch(scratch);
  }
}

SYMS_API void
syms_logf__dev(char *fmt, ...){
  va_list args;
  va_start(args, fmt);
  syms_logfv__dev(fmt, args);
  va_end(args);
}

#else

#define syms_log_set_filter__dev(...)
#define syms_log_open__dev(...)
#define syms_log_is_enabled__dev(...) syms_false
#define syms_log_open_annotated__dev(...) 0
#define syms_logfv__dev(...)
#define syms_logf__dev(...)

#endif

////////////////////////////////
// NOTE(allen): Dev Profiling Functions

#if SYMS_ENABLE_DEV_PROFILE

SYMS_THREAD_LOCAL SYMS_ProfState *syms_prof_state__dev = 0;

SYMS_API void
syms_prof_equip_thread__dev(SYMS_ProfState *prof_state){
  syms_prof_state__dev = prof_state;
  MemoryZeroStruct(prof_state);
}

SYMS_API void
syms_prof_equip_thread_auto__dev(void){
  SYMS_ProfState *prof_state = (SYMS_ProfState*)SYMS_PROF_ALLOC(sizeof(SYMS_ProfState));
  syms_prof_equip_thread__dev(prof_state);
}

SYMS_API void
syms_prof_unequip_thread__dev(void){
  syms_prof_state__dev = 0;
}

SYMS_API SYMS_U64*
syms_prof_push__dev(void){
  // check for an equipped profiler state
  SYMS_LOCAL SYMS_U64 dummy[2];
  SYMS_U64 *result = dummy;
  SYMS_ProfState *prof_state = syms_prof_state__dev;
  if (prof_state != 0){
    
    // check the current chain; extend if necessary
    SYMS_ProfChain *chain = prof_state->current;
    if (chain == 0 || chain->ptr + 16 > (SYMS_U8*)chain + SYMS_PROF_BLOCK_SIZE){
      chain = prof_state->free;
      if (chain != 0){
        SYMS_StackPop(prof_state->free);
      }
      else{
        chain = (SYMS_ProfChain*)SYMS_PROF_ALLOC(SYMS_PROF_BLOCK_SIZE);
      }
      SYMS_QueuePush(prof_state->first, prof_state->current, chain);
      chain->ptr = (SYMS_U8*)(chain + 1);
    }
    
    // get result pointer and increment
    result = (SYMS_U64*)chain->ptr;
    chain->ptr += 16;
  }
  return(result);
}

SYMS_API void
syms_prof_paste__dev(SYMS_ProfState *sub_state){
  SYMS_ProfState *prof_state = syms_prof_state__dev;
  if (prof_state != 0 && sub_state != 0){
    if (sub_state->first != 0){
      if (prof_state->current == 0){
        prof_state->first = sub_state->first;
      }
      else{
        prof_state->current->next = sub_state->first;
      }
      prof_state->current = sub_state->current;
    }
    if (sub_state->free != 0){
      if (prof_state->free != 0){
        SYMS_ProfChain *last_chain = sub_state->free;
        for (; last_chain->next != 0; last_chain = last_chain->next);
        last_chain->next = prof_state->free;
      }
      prof_state->free = sub_state->free;
    }
    syms_memzero_struct(sub_state);
  }
}

SYMS_API SYMS_ProfLock
syms_prof_lock__dev(SYMS_Arena *arena){
  SYMS_ProfLock result = {syms_prof_state__dev};
  syms_prof_state__dev = 0;
  if (result.state != 0){
    SYMS_U64 chain_count = 0;
    for (SYMS_ProfChain *chain = result.state->first;
         chain != 0;
         chain = chain->next, chain_count += 1);
    
    SYMS_U64 size = chain_count*SYMS_PROF_BLOCK_SIZE;
    SYMS_U8 *str = syms_push_array(arena, SYMS_U8, size);
    SYMS_U8 *ptr = str;
    for (SYMS_ProfChain *chain = result.state->first;
         chain != 0;
         chain = chain->next){
      SYMS_U64 chain_size = (SYMS_U64)(chain->ptr - (SYMS_U8*)(chain + 1));
      syms_memmove(ptr, (chain + 1), chain_size);
      ptr += chain_size;
    }
    result.data = syms_str8_range(str, ptr);
  }
  return(result);
}

SYMS_API void
syms_prof_clear__dev(SYMS_ProfLock lock){
  if (lock.state != 0){
    if (lock.state->current != 0){
      lock.state->current->next = lock.state->free;
      lock.state->free = lock.state->first;
    }
    lock.state->first = 0;
    lock.state->current = 0;
  }
}

SYMS_API void
syms_prof_unlock__dev(SYMS_ProfLock lock){
  syms_prof_state__dev = lock.state;
}

SYMS_API SYMS_ProfTree
syms_prof_tree__dev(SYMS_Arena *arena, SYMS_String8 data){
  SYMS_ProfTree result = {0};
  
  if (data.size > 0){
    // bucket setup
    result.bucket_count = 16381;
    result.buckets = syms_push_array_zero(arena, SYMS_ProfTreeNode*, result.bucket_count);
    
    // setup root
    result.root = syms_push_array_zero(arena, SYMS_ProfTreeNode, 1);
    {
      SYMS_String8 key = syms_str8_lit("<root>");
      SYMS_U64 hash = syms_hash_djb2(key);
      SYMS_U64 index = hash%result.bucket_count;
      SYMS_StackPush(result.buckets[index], result.root);
      result.count += 1;
      result.root->hash = hash;
      result.root->key = key;
    }
    
    // tree walk
    SYMS_ProfTreeNode *tree_walk_ptr = result.root;
    SYMS_U64 depth = 1;
    result.height = 1;
    
    // parse loop
    SYMS_U64 rounded_size = data.size&~15;
    SYMS_U64 *ptr = (SYMS_U64*)data.str;
    SYMS_U64 *opl = ptr + (rounded_size/8);
    for (;ptr < opl; ptr += 2){
      if (*ptr != 0){
        // read begin values
        char *label = (char*)(ptr[0]);
        SYMS_U64 time_min = ptr[1];
        
        // parse key
        SYMS_U64 parent_hash = tree_walk_ptr->hash;
        SYMS_String8 key = syms_str8_cstring(label);
        SYMS_U64 hash = syms_hash_djb2_continue(key, parent_hash);
        SYMS_U64 index = hash%result.bucket_count;
        
        // existing tree node?
        SYMS_ProfTreeNode *tree_node = 0;
        for (SYMS_ProfTreeNode *bucket_node = result.buckets[index];
             bucket_node != 0;
             bucket_node = bucket_node->next){
          if (bucket_node->hash == hash &&
              syms_string_match(bucket_node->key, key, 0) &&
              bucket_node->tree_parent == tree_walk_ptr){
            tree_node = bucket_node;
            break;
          }
        }
        
        // new tree node?
        if (tree_node == 0){
          tree_node = syms_push_array_zero(arena, SYMS_ProfTreeNode, 1);
          
          // insert in hash table
          SYMS_StackPush(result.buckets[index], tree_node);
          result.count += 1;
          tree_node->hash = hash;
          tree_node->key = key;
          
          // insert in tree
          SYMS_QueuePush_N(tree_walk_ptr->tree_first, tree_walk_ptr->tree_last, tree_node, tree_next);
          tree_node->tree_parent = tree_walk_ptr;
        }
        
        // save time_min
        SYMS_ASSERT(tree_node->time_min == 0);
        tree_node->time_min = time_min;
        
        // push stack
        tree_walk_ptr = tree_node;
        depth += 1;
        result.height = SYMS_MAX(depth, result.height);
        
        // track key max
        result.max_key_size = SYMS_MAX(result.max_key_size, key.size);
      }
      else{
        // never "handle" the parent!
        if (tree_walk_ptr->tree_parent == 0){
          tree_walk_ptr = 0;
          break;
        }
        
        // read end values
        SYMS_U64 time_max = ptr[1];
        
        // update tree node
        tree_walk_ptr->count += 1;
        tree_walk_ptr->total_time += time_max - tree_walk_ptr->time_min;
        tree_walk_ptr->time_min = 0;
        
        // pop the stack
        tree_walk_ptr = tree_walk_ptr->tree_parent;
        depth -= 1;
      }
    }
    
    // (stack == 0) -> extra SYMS_ProfEnd
    // (stack != 0 && stack != result.root) -> unclosed SYMS_ProfBegin
    SYMS_ASSERT(tree_walk_ptr == result.root);
  }
  
  return(result);
}

SYMS_API void
syms_prof_tree_sort_in_place__dev(SYMS_ProfTreeNode *root){
  // sort this layer
  if (root->tree_first != 0){
    SYMS_ArenaTemp scratch = syms_get_scratch(0, 0);
    
    // count children
    SYMS_U64 count = 0;
    for (SYMS_ProfTreeNode *node = root->tree_first;
         node != 0;
         node = node->tree_next, count += 1);
    
    // allocate sortable buffer
    SYMS_ProfTreeNode **array = syms_push_array(scratch.arena, SYMS_ProfTreeNode*, count);
    {
      SYMS_ProfTreeNode **ptr = array;
      for (SYMS_ProfTreeNode *node = root->tree_first;
           node != 0;
           node = node->tree_next, ptr += 1){
        *ptr = node;
      }
    }
    
    // sort the array
    syms_prof_tree_sort_pointer_array__dev(array, count);
    
    // reconstruct the linked list
    {
      SYMS_ProfTreeNode **ptr = array;
      root->tree_first = *ptr;
      for (SYMS_U64 i = 1; i < count; i += 1, ptr += 1){
        ptr[0]->tree_next = ptr[1];
      }
      ptr[0]->tree_next = 0;
      root->tree_last = *ptr;
    }
    
    syms_release_scratch(scratch);
  }
  
  // recurse
  for (SYMS_ProfTreeNode *node = root->tree_first;
       node != 0;
       node = node->tree_next){
    syms_prof_tree_sort_in_place__dev(node);
  }
}

SYMS_API void
syms_prof_tree_sort_pointer_array__dev(SYMS_ProfTreeNode **array, SYMS_U64 count){
  if (count > 1){
    SYMS_U64 last = count - 1;
    
    // swap
    SYMS_U64 mid = count/2;
    SYMS_Swap(SYMS_ProfTreeNode*, array[mid], array[last]);
    
    // partition
    SYMS_U64 key = array[last]->total_time;
    SYMS_U64 j = 0;
    for (SYMS_U64 i = 0; i < last; i += 1){
      if (array[i]->total_time > key){
        if (j != i){
          SYMS_Swap(SYMS_ProfTreeNode*, array[i], array[j]);
        }
        j += 1;
      }
    }
    
    SYMS_Swap(SYMS_ProfTreeNode*, array[j], array[last]);
    
    // recurse
    SYMS_U64 pivot = j;
    syms_prof_tree_sort_pointer_array__dev(array, pivot);
    syms_prof_tree_sort_pointer_array__dev(array + pivot + 1, (count - pivot - 1));
  }
}

SYMS_API void
syms_prof_stringize_tree__dev(SYMS_Arena *arena, SYMS_ProfTree *tree, SYMS_String8List *out){
  SYMS_U64 align = tree->max_key_size + tree->height;
  syms_prof_stringize_tree__rec__dev(arena, tree->root, out, align, 0);
}

#if SYMS_ENABLE_DEV_STRING

SYMS_API void
syms_prof_stringize_tree__rec__dev(SYMS_Arena *arena, SYMS_ProfTreeNode *node,
                                   SYMS_String8List *out, SYMS_U64 align, SYMS_U64 indent){
  SYMS_LOCAL const char spaces[] = "                                                ";
  syms_string_list_pushf__dev(arena, out, "%.*s%-*.*s %12lluus [%8llu]\n",
                              (int)indent, spaces,
                              (int)(align - indent), syms_expand_string(node->key),
                              node->total_time, node->count);
  for (SYMS_ProfTreeNode *child = node->tree_first;
       child != 0;
       child = child->tree_next){
    syms_prof_stringize_tree__rec__dev(arena, child, out, align, indent + 1);
  }
}

SYMS_API void
syms_prof_stringize_basic__dev(SYMS_Arena *arena, SYMS_ProfLock lock, SYMS_String8List *out){
  SYMS_ArenaTemp scratch = syms_get_scratch(&arena, 1);
  SYMS_ProfTree tree = syms_prof_tree__dev(scratch.arena, lock.data);
  syms_prof_tree_sort_in_place__dev(tree.root);
  syms_prof_stringize_tree__dev(arena, &tree, out);
  syms_release_scratch(scratch);
}

#endif //SYMS_ENABLE_DEV_STRING

#else

#define syms_prof_equip_thread__dev(...)
#define syms_prof_equip_thread_auto__dev(...)
#define syms_prof_unequip_thread__dev(...)
#define syms_prof_push__dev(...)
#define syms_prof_paste__dev(...)

#define syms_prof_lock__dev(...) {}
#define syms_prof_clear__dev(...)
#define syms_prof_unlock__dev(l) ((void)l)

#define syms_prof_tree__dev(...)
#define syms_prof_tree_sort_in_place__dev(...)
#define syms_prof_tree_sort_pointer_array__dev(...)

#if SYMS_ENABLE_DEV_STRING
# define syms_prof_stringize_tree__dev(...)
# define syms_prof_stringize_tree__rec__dev(...)
# define syms_prof_stringize_basic__dev(...)
#endif

#endif //SYMS_ENABLE_DEV_PROFILE

#endif //SYMS_DEV_C
