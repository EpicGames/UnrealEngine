// Copyright Epic Games, Inc. All Rights Reserved.
/* date = August 6th 2021 4:44 pm */

#ifndef SYMS_DEV_H
#define SYMS_DEV_H

////////////////////////////////
// NOTE(allen): Dev System Dependencies

#if SYMS_ENABLE_DEV_LOG
# undef SYMS_ENABLE_DEV_STRING
# define SYMS_ENABLE_DEV_STRING 1
#endif

////////////////////////////////
// NOTE(allen): Dev Logging Types

typedef SYMS_U32 SYMS_LogFeatures;
enum{
  SYMS_LogFeature_LineTable       = (1 << 0),
  SYMS_LogFeature_DwarfUnitRanges = (1 << 1),
  SYMS_LogFeature_DwarfTags       = (1 << 2),
  SYMS_LogFeature_DwarfUnwind     = (1 << 3),
  SYMS_LogFeature_DwarfCFILookup  = (1 << 4),
  SYMS_LogFeature_DwarfCFIDecode  = (1 << 5),
  SYMS_LogFeature_DwarfCFIApply   = (1 << 6),
  SYMS_LogFeature_PEEpilog        = (1 << 7),
  SYMS_LogFeature_PeResParser     = (1 << 8),
  
  
  // Dummy flag shouldn't be associated to anything.
  // This gives us a way to disable all logging.
  SYMS_LogFeature_Dummy = (1 << 31),
};

////////////////////////////////
// NOTE(allen): Dev Profile Types

typedef struct SYMS_ProfChain{
  struct SYMS_ProfChain *next;
  SYMS_U8 *ptr;
} SYMS_ProfChain;

typedef struct SYMS_ProfState{
  SYMS_ProfChain *first;
  SYMS_ProfChain *current;
  SYMS_ProfChain *free;
} SYMS_ProfState;

typedef struct SYMS_ProfLock{
  SYMS_ProfState *state;
  SYMS_String8 data;
} SYMS_ProfLock;

typedef struct SYMS_ProfTreeNode{
  // hash table
  struct SYMS_ProfTreeNode *next;
  SYMS_U64 hash;
  SYMS_String8 key;
  
  // local counts
  SYMS_U64 count;
  SYMS_U64 total_time;
  
  // tree
  struct SYMS_ProfTreeNode *tree_first;
  struct SYMS_ProfTreeNode *tree_last;
  struct SYMS_ProfTreeNode *tree_next;
  struct SYMS_ProfTreeNode *tree_parent;
  
  // mutable space - not a fixed value for users to rely on
  SYMS_U64 time_min;
} SYMS_ProfTreeNode;

typedef struct SYMS_ProfTree{
  // hash table
  SYMS_ProfTreeNode **buckets;
  SYMS_U64 bucket_count;
  SYMS_U64 count;
  SYMS_U64 max_key_size;
  
  // tree
  SYMS_ProfTreeNode *root;
  SYMS_U64 height;
} SYMS_ProfTree;

////////////////////////////////
// NOTE(allen): Dev String Functions

#if SYMS_ENABLE_DEV_STRING

// TODO(allen): way to override standard library's printf
// (bundle in rad's format string function?)

#include <stdio.h>
#include <stdarg.h>

SYMS_API SYMS_String8 syms_push_stringfv__dev(SYMS_Arena *arena, char *fmt, va_list args);
SYMS_API SYMS_String8 syms_push_stringf__dev(SYMS_Arena *arena, char *fmt, ...);

SYMS_API void         syms_string_list_pushfv__dev(SYMS_Arena *arena, SYMS_String8List *list, char *fmt,
                                                   va_list args);
SYMS_API void         syms_string_list_pushf__dev(SYMS_Arena *arena, SYMS_String8List *list, char *fmt, ...);

#endif

////////////////////////////////
// NOTE(allen): Dev Logging Functions

#if SYMS_ENABLE_DEV_LOG

#if !defined(SYMS_LOG_RAW_APPEND)
# include <stdio.h>
# define SYMS_LOG_RAW_APPEND(s) fwrite((s).str, (s).size, 1, stderr)
#endif
#if !defined(SYMS_LOG_FILTER_FEATURES)
# define SYMS_LOG_FILTER_FEATURES 0
#endif
#if !defined(SYMS_LOG_FILTER_UID)
# define SYMS_LOG_FILTER_UID 0
#endif

SYMS_API void     syms_log_set_filter__dev(SYMS_LogFeatures features, SYMS_U64 uid);

SYMS_API SYMS_U32 syms_log_open__dev(SYMS_B32 enabled);
SYMS_API void     syms_log_close__dev(SYMS_U32 prev_state);
SYMS_API SYMS_B32 syms_log_is_enabled__dev(void);
SYMS_API SYMS_U32 syms_log_open_annotated__dev(SYMS_LogFeatures features, SYMS_U64 uid);

SYMS_API void syms_logfv__dev(char *fmt, va_list args);
SYMS_API void syms_logf__dev(char *fmt, ...);

#endif

#if SYMS_ENABLE_DEV_LOG
# define SYMS_LogOpen(ftr,uid,block) SYMS_U32 syms_log_state__##block = syms_log_open_annotated__dev((ftr),(uid))
# define SYMS_LogClose(block) syms_log_close__dev(syms_log_state__##block)
# define SYMS_Log(...) syms_logf__dev(__VA_ARGS__)
# define SYMS_LogIsEnabled() syms_log_is_enabled__dev()
#else
# define SYMS_LogOpen(ftr,uid,block) ((void)0)
# define SYMS_LogClose(block) ((void)0)
# define SYMS_Log(fmt,...) ((void)0)
# define SYMS_LogIsEnabled() (0)
#endif

////////////////////////////////
// NOTE(allen): Dev Profiling Functions

#if SYMS_ENABLE_DEV_PROFILE

// SYMS_PROF_TIME: () -> SYMS_U64
#if !defined(SYMS_PROF_TIME)
# error "SYMS_PROF_TIME not #define'd"
#endif

// SYMS_PROF_ALLOC: (SYMS_U64) -> void*
#if !defined(SYMS_PROF_ALLOC)
# error "SYMS_PROF_ALLOC not #define'd"
#endif

#if !defined(SYMS_PROF_BLOCK_SIZE)
# define SYMS_PROF_BLOCK_SIZE (1 << 20)
#endif

SYMS_API void      syms_prof_equip_thread__dev(SYMS_ProfState *prof_state);
SYMS_API void      syms_prof_equip_thread_auto__dev(void);
SYMS_API void      syms_prof_unequip_thread__dev(void);
SYMS_API SYMS_U64* syms_prof_push__dev(void);
SYMS_API void      syms_prof_paste__dev(SYMS_ProfState *sub_state);

SYMS_API SYMS_ProfLock syms_prof_lock__dev(SYMS_Arena *arena);
SYMS_API void          syms_prof_clear__dev(SYMS_ProfLock lock);
SYMS_API void          syms_prof_unlock__dev(SYMS_ProfLock lock);

SYMS_API SYMS_ProfTree       syms_prof_tree__dev(SYMS_Arena *arena, SYMS_String8 data);
SYMS_API void                syms_prof_tree_sort_in_place__dev(SYMS_ProfTreeNode *root);
SYMS_API void                syms_prof_tree_sort_pointer_array__dev(SYMS_ProfTreeNode **array, SYMS_U64 count);

#if SYMS_ENABLE_DEV_STRING
SYMS_API void                syms_prof_stringize_tree__dev(SYMS_Arena *arena, SYMS_ProfTree *tree,
                                                           SYMS_String8List *out);
SYMS_API void                syms_prof_stringize_tree__rec__dev(SYMS_Arena *arena, SYMS_ProfTreeNode *node,
                                                                SYMS_String8List *out,
                                                                SYMS_U64 align, SYMS_U64 indent);
SYMS_API void syms_prof_stringize_basic__dev(SYMS_Arena *arena, SYMS_ProfLock lock, SYMS_String8List *out);
#endif

#endif

#if SYMS_ENABLE_DEV_PROFILE

# if !defined(SYMS_ProfBegin)
#  define SYMS_ProfBegin(str) do{ SYMS_U64 t = SYMS_PROF_TIME(); SYMS_U64 *v = syms_prof_push__dev(); \
v[0] = (SYMS_U64)(SYMS_THIS_SRCLOC ": " str); v[1] = t; }while(0)
# endif

# if !defined(SYMS_ProfEnd)
#  define SYMS_ProfEnd() do{ SYMS_U64 *v = syms_prof_push__dev(); \
v[0] = 0; v[1] = SYMS_PROF_TIME(); }while(0)
# endif

# if !defined(SYMS_ProfPasteSubChain)
#  define SYMS_ProfPasteSubChain(s) syms_prof_paste__dev(s)
# endif

#else
# if !defined(SYMS_ProfBegin)
#  define SYMS_ProfBegin(str) ((void)0)
# endif
# if !defined(SYMS_ProfEnd)
#  define SYMS_ProfEnd() ((void)0)
# endif
# if !defined(SYMS_ProfPasteSubChain)
#  define SYMS_ProfPasteSubChain(s) ((void)0)
# endif
#endif


#endif //SYMS_DEV_H
