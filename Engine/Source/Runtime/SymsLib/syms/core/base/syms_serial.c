// Copyright Epic Games, Inc. All Rights Reserved.

#ifndef SYMS_SERIAL_C
#define SYMS_SERIAL_C

////////////////////////////////
// NOTE(allen): Include extended serial info

#include "syms/core/generated/syms_meta_serial_ext.c"

#endif //SYMS_SERIAL_C
