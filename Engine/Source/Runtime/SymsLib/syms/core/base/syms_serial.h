// Copyright Epic Games, Inc. All Rights Reserved.
/* date = April 16th 2021 0:08 pm */

#ifndef SYMS_SERIAL_H
#define SYMS_SERIAL_H

////////////////////////////////
// NOTE(allen): Implement serial control macro features

#if !defined(SYMS_DISABLE_SERIAL_INFO_AUTOMATIC)
# if defined(_SYMS_META_BASE_H)
#  define SYMS_ENABLE_BASE_SERIAL_INFO
# endif
# if defined(_SYMS_META_DEBUG_INFO_H)
#  define SYMS_ENABLE_DEBUG_INFO_SERIAL_INFO
# endif
# if defined(_SYMS_META_CV_H)
#  define SYMS_ENABLE_CV_SERIAL_INFO
# endif
# if defined(_SYMS_META_DWARF_H)
#  define SYMS_ENABLE_DWARF_SERIAL_INFO
# endif
# if defined(_SYMS_META_COFF_H)
#  define SYMS_ENABLE_COFF_SERIAL_INFO
# endif
# if defined(_SYMS_META_PE_H)
#  define SYMS_ENABLE_PE_SERIAL_INFO
# endif
# if defined(_SYMS_META_MACH_H)
#  define SYMS_ENABLE_MACH_SERIAL_INFO
# endif
# if defined(_SYMS_META_ELF_H)
#  define SYMS_ENABLE_ELF_SERIAL_INFO
# endif
#endif

////////////////////////////////
// NOTE(allen): Include extended serial info

#include "syms/core/generated/syms_meta_serial_ext.h"

#endif //SYMS_SERIAL_H
