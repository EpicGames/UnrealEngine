// Copyright Epic Games, Inc. All Rights Reserved.

#if !defined(SYMS_DATA_STRUCTURES_C)
#define SYMS_DATA_STRUCTURES_C

////////////////////////////////
//~ allen: Syms String Cons

SYMS_API SYMS_String8
syms_string_cons(SYMS_Arena *arena, SYMS_StringCons *cons, SYMS_String8 string){
  SYMS_ProfBegin("syms_string_cons");
  SYMS_String8 result = {0};
  if (SYMS_STRING_CONS_BUCKET_COUNT > 0 && string.size > 0){
    SYMS_U64 hash = syms_hash_djb2(string);
    SYMS_U64 bucket_index = hash%SYMS_STRING_CONS_BUCKET_COUNT;
    for (SYMS_StringConsNode *node = cons->buckets[bucket_index];
         node != 0;
         node = node->next){
      if (node->hash == hash && syms_string_match(string, node->string, 0)){
        result = node->string;
        break;
      }
    }
    if (result.str == 0){
      // stabilize the string memory
      SYMS_String8 stable_string = syms_push_string_copy(arena, string);
      // save a cons node
      SYMS_StringConsNode *new_node = syms_push_array(arena, SYMS_StringConsNode, 1);
      SYMS_StackPush(cons->buckets[bucket_index], new_node);
      new_node->string = stable_string;
      new_node->hash = hash;
      result = new_node->string;
    }
  }
  SYMS_ProfEnd();
  return(result);
}


SYMS_API SYMS_DataIdxCons
syms_data_idx_cons_alloc(SYMS_Arena *arena, SYMS_U64 bucket_count){
  SYMS_DataIdxCons result = {0};
  result.bucket_count = bucket_count;
  result.buckets = syms_push_array_zero(arena, SYMS_DataIdxConsNode*, bucket_count);
  return(result);
}

SYMS_API SYMS_U64
syms_data_idx_cons(SYMS_Arena *arena, SYMS_DataIdxCons *cons, SYMS_String8 data){
  SYMS_ProfBegin("syms_data_idx_cons");
  SYMS_U64 result = 0;
  if (cons->bucket_count > 0 && data.size > 0){
    SYMS_U64 hash = syms_hash_djb2(data);
    SYMS_U64 bucket_index = hash%cons->bucket_count;
    for (SYMS_DataIdxConsNode *node = cons->buckets[bucket_index];
         node != 0;
         node = node->bucket_next){
      if (node->hash == hash && syms_string_match(data, node->data, 0)){
        result = node->id;
        break;
      }
    }
    if (result == 0){
      // take a new id
      SYMS_U64 new_id = cons->count + 1;
      // stabilize the data memory
      SYMS_String8 stable_data = syms_push_string_copy(arena, data);
      // save a cons node
      SYMS_DataIdxConsNode *new_node = syms_push_array_zero(arena, SYMS_DataIdxConsNode, 1);
      SYMS_StackPush_N(cons->buckets[bucket_index], new_node, bucket_next);
      SYMS_QueuePush_N(cons->first, cons->last, new_node, all_next);
      cons->count += 1;
      cons->total_size += data.size;
      new_node->data = stable_data;
      new_node->hash = hash;
      new_node->id = new_id;
      result = new_id;
    }
  }
  SYMS_ProfEnd();
  return(result);
}


////////////////////////////////
//~ allen: U64 Set

SYMS_API SYMS_U64Set
syms_u64_set_alloc(SYMS_Arena *arena, SYMS_U64 cap){
  SYMS_U64Set result = {0};
  result.cap = cap;
  result.vals = syms_push_array_zero(arena, SYMS_U64, cap);
  return(result);
}

SYMS_API SYMS_U64
syms_u64_set__bs(SYMS_U64Set *set, SYMS_U64 x){
  SYMS_ProfBegin("syms_u64_set__bs");
  //- binary search:
  //  min index s.t. x <= vals[index]
  //  set->count if no such index exists
  // in this one we assume:
  //  (i != j) implies (vals[i] != vals[j])
  //  thus if (vals[index] == x) then index already satisfies the requirement
  SYMS_U64 count = set->count;
  SYMS_U64 *vals = set->vals;
  SYMS_U64 result = set->count;
  if (count > 0 && x <= vals[count - 1]){
    SYMS_U64 first = 0;
    SYMS_U64 opl   = count;
    for (;;){
      if (first + 1 >= opl){
        break;
      }
      SYMS_U64 mid = (first + opl - 1)/2;
      SYMS_U64 v = vals[mid];
      if (x < v){
        opl = mid + 1;
      }
      else if (x > v){
        first = mid + 1;
      }
      else{
        first = mid;
        break;
      }
    }
    result = first;
  }
  SYMS_ProfEnd();
  return(result);
}

SYMS_API SYMS_B32
syms_u64_set_insert(SYMS_U64Set *set, SYMS_U64 x){
  SYMS_ProfBegin("syms_u64_set_insert");
  SYMS_B32 result = 0;
  SYMS_U64 index = syms_u64_set__bs(set, x);
  SYMS_U64 *vals = set->vals;
  SYMS_U64 count = set->count;
  if (index < set->cap && (index == count || vals[index] != x)){
    syms_memmove(vals + index + 1, vals + index,
                 sizeof(SYMS_U64)*(count - index));
    vals[index] = x;
    set->count = count + 1;
    result = 1;
  }
  SYMS_ProfEnd();
  return(result);
}

SYMS_API void
syms_u64_set_erase(SYMS_U64Set *set, SYMS_U64 x){
  SYMS_ProfBegin("syms_u64_set_erase");
  SYMS_U64 index = syms_u64_set__bs(set, x);
  SYMS_U64 *vals = set->vals;
  SYMS_U64 count = set->count;
  if (index < count && vals[index] == x){
    syms_memmove(vals + index, vals + index + 1,
                 sizeof(SYMS_U64)*(count - index - 1));
    set->count = count - 1;
  }
  SYMS_ProfEnd();
}


////////////////////////////////
//~ allen: Syms Spatial Mapping

//- lookups into spatial maps
SYMS_API SYMS_U64
syms_spatial_map_1d_binary_search(SYMS_SpatialMap1D *map, SYMS_U64 x){
  SYMS_ProfBegin("syms_spatial_map_1d_binary_search");
  //- binary search:
  //   max index s.t. ranges[index].range.min <= x
  //   SYMS_U64_MAX if no such index exists
  //  in this one we assume:
  //   (i != j) implies (ranges[i].range.min != ranges[j].range.min)
  //   thus if (ranges[index].range.min == x) then index already satisfies the requirement
  SYMS_U64 result = SYMS_U64_MAX;
  SYMS_SpatialMap1DRange *ranges = map->ranges;
  SYMS_U64 count = map->count;
  if (count > 0 && ranges[0].range.min <= x){
    SYMS_U64 first = 0;
    SYMS_U64 opl   = count;
    for (;;){
      SYMS_U64 mid = (first + opl)/2;
      SYMS_U64 rmin = ranges[mid].range.min;
      if (x < rmin){
        opl = mid;
      }
      else if (x > rmin){
        first = mid;
      }
      else{
        first = mid;
        break;
      }
      if (first + 1 >= opl){
        break;
      }
    }
    result = first;
  }
  
  SYMS_ProfEnd();
  return(result);
}

SYMS_API SYMS_U64
syms_spatial_map_1d_index_from_point(SYMS_SpatialMap1D *map, SYMS_U64 x){
  SYMS_ProfBegin("syms_spatial_map_1d_index_from_point");
  SYMS_U64 index = syms_spatial_map_1d_binary_search(map, x);
  //- check if the range actually contains x
  //   (if we have a range, we already know that (range.min <= x))
  SYMS_U64 result = SYMS_U64_MAX;
  if (index != SYMS_U64_MAX &&
      x < map->ranges[index].range.max){
    result = index;
  }
  SYMS_ProfEnd();
  return(result);
}

SYMS_API SYMS_U64
syms_spatial_map_1d_value_from_point(SYMS_SpatialMap1D *map, SYMS_U64 x){
  SYMS_ProfBegin("syms_spatial_map_1d_value_from_point");
  SYMS_U64 index = syms_spatial_map_1d_index_from_point(map, x);
  SYMS_U64 result = 0;
  if (index < map->count){
    result = map->ranges[index].val;
  }
  SYMS_ProfEnd();
  return(result);
}

//- copying spatial maps

SYMS_API SYMS_SpatialMap1D
syms_spatial_map_1d_copy(SYMS_Arena *arena, SYMS_SpatialMap1D *map){
  SYMS_SpatialMap1D result = {0};
  result.count = map->count;
  result.ranges = syms_push_array(arena, SYMS_SpatialMap1DRange, result.count);
  syms_memmove(result.ranges, map->ranges, sizeof(SYMS_SpatialMap1DRange)*result.count);
  return(result);
}

//- constructing spatial maps

SYMS_API void
syms_spatial_map_1d_loose_push(SYMS_Arena *arena, SYMS_SpatialMap1DLoose *loose,
                               SYMS_U64 val, SYMS_U64RangeArray ranges){
  SYMS_SpatialMap1DNode *node = syms_push_array(arena, SYMS_SpatialMap1DNode, 1);
  syms_memzero_struct(&node->range);
  node->ranges = ranges;
  node->val = val;
  SYMS_QueuePush(loose->first, loose->last, node);
  loose->total_count += ranges.count;
}

SYMS_API void
syms_spatial_map_1d_loose_push_single(SYMS_Arena *arena, SYMS_SpatialMap1DLoose *loose,
                                      SYMS_U64 val, SYMS_U64Range range){
  SYMS_SpatialMap1DNode *node = syms_push_array(arena, SYMS_SpatialMap1DNode, 1);
  node->range = range;
  syms_memzero_struct(&node->ranges);
  node->val = val;
  SYMS_QueuePush(loose->first, loose->last, node);
  loose->total_count += 1;
}

SYMS_API SYMS_SpatialMap1D
syms_spatial_map_1d_bake(SYMS_Arena *arena, SYMS_SpatialMap1DLoose *loose){
  SYMS_ProfBegin("syms_spatial_map_1d_bake");
  //- fill tight range array
  SYMS_U64 count = loose->total_count;
  SYMS_SpatialMap1DRange *ranges = syms_push_array(arena, SYMS_SpatialMap1DRange, count);
  SYMS_SpatialMap1DRange *range_ptr = ranges;
  for (SYMS_SpatialMap1DNode *node = loose->first;
       node != 0;
       node = node->next){
    SYMS_U64 val = node->val;
    
    {
      SYMS_U64Range range = node->range;
      if (range.min < range.max){
        range_ptr->range = range;
        range_ptr->val = val;
        range_ptr += 1;
      }
    }
    
    SYMS_U64 count = node->ranges.count;
    SYMS_U64Range *range = node->ranges.ranges;
    for (SYMS_U64 i = 0; i < count; i += 1, range += 1){
      if (range->min < range->max){
        range_ptr->range = *range;
        range_ptr->val = val;
        range_ptr += 1;
      }
    }
  }
  SYMS_U64 final_count = (SYMS_U64)(range_ptr - ranges);
  syms_arena_put_back(arena, sizeof(SYMS_SpatialMap1DRange)*(count - final_count));
  
  //- sort
  if (!syms_spatial_map_1d_array_check_sorted(ranges, final_count)){
    syms_spatial_map_1d_array_sort(ranges, final_count);
  }
  
  //- assemble map
  SYMS_SpatialMap1D result = {ranges, final_count};
  
  SYMS_ProfEnd();
  return(result);
}

SYMS_API SYMS_B32
syms_spatial_map_1d_array_check_sorted(SYMS_SpatialMap1DRange *ranges, SYMS_U64 count){
  SYMS_ProfBegin("syms_spatial_map_1d_array_check_sorted");
  SYMS_B32 result = syms_true;
  SYMS_SpatialMap1DRange *range_ptr = ranges;
  for (SYMS_U64 i = 1; i < count; i += 1, range_ptr += 1){
    if (range_ptr->range.min > (range_ptr + 1)->range.min){
      result = syms_false;
      break;
    }
  }
  SYMS_ProfEnd();
  return(result);
}

SYMS_API void
syms_spatial_map_1d_array_sort(SYMS_SpatialMap1DRange *ranges, SYMS_U64 count){
  SYMS_ProfBegin("syms_spatial_map_1d_array_sort");
  syms_spatial_map_1d_array_sort__rec(ranges, count);
  SYMS_ProfEnd();
}

SYMS_API void
syms_spatial_map_1d_array_sort__rec(SYMS_SpatialMap1DRange *ranges, SYMS_U64 count){
  if (count > 4){
    SYMS_U64 last = count - 1;
    
    // swap
    SYMS_U64 mid = count/2;
    SYMS_Swap(SYMS_SpatialMap1DRange, ranges[mid], ranges[last]);
    
    // partition
    SYMS_B32 equal_send_left = 0;
    SYMS_U64 key = ranges[last].range.min;
    SYMS_U64 j = 0;
    for (SYMS_U64 i = 0; i < last; i += 1){
      SYMS_B32 send_left = (ranges[i].range.min < key);
      if (!send_left && ranges[i].range.min == key){
        send_left = equal_send_left;
        equal_send_left = !equal_send_left;
      }
      if (send_left){
        if (j != i){
          SYMS_Swap(SYMS_SpatialMap1DRange, ranges[i], ranges[j]);
        }
        j += 1;
      }
    }
    
    SYMS_Swap(SYMS_SpatialMap1DRange, ranges[j], ranges[last]);
    
    // recurse
    SYMS_U64 pivot = j;
    syms_spatial_map_1d_array_sort__rec(ranges, pivot);
    syms_spatial_map_1d_array_sort__rec(ranges + pivot + 1, (count - pivot - 1));
  }
  else if (count == 2){
    if (ranges[0].range.min > ranges[1].range.min){
      SYMS_Swap(SYMS_SpatialMap1DRange, ranges[0], ranges[1]);
    }
  }
  else if (count == 3){
    if (ranges[0].range.min > ranges[1].range.min){
      SYMS_Swap(SYMS_SpatialMap1DRange, ranges[0], ranges[1]);
    }
    if (ranges[1].range.min > ranges[2].range.min){
      SYMS_Swap(SYMS_SpatialMap1DRange, ranges[1], ranges[2]);
      if (ranges[0].range.min > ranges[1].range.min){
        SYMS_Swap(SYMS_SpatialMap1DRange, ranges[0], ranges[1]);
      }
    }
  }
  else if (count == 4){
    if (ranges[0].range.min > ranges[1].range.min){
      SYMS_Swap(SYMS_SpatialMap1DRange, ranges[0], ranges[1]);
    }
    if (ranges[2].range.min > ranges[3].range.min){
      SYMS_Swap(SYMS_SpatialMap1DRange, ranges[2], ranges[3]);
    }
    if (ranges[0].range.min > ranges[2].range.min){
      SYMS_Swap(SYMS_SpatialMap1DRange, ranges[0], ranges[2]);
    }
    if (ranges[1].range.min > ranges[3].range.min){
      SYMS_Swap(SYMS_SpatialMap1DRange, ranges[1], ranges[3]);
    }
    if (ranges[1].range.min > ranges[2].range.min){
      SYMS_Swap(SYMS_SpatialMap1DRange, ranges[1], ranges[2]);
    }
  }
}

//- support for the overlapping ranges

SYMS_API SYMS_SpatialMultiMap1D
syms_spatial_multi_map_1d_bake(SYMS_Arena *arena, SYMS_SpatialMap1DLoose *loose){
  SYMS_ProfBegin("syms_spatial_multi_map_1d_bake");
  SYMS_SpatialMultiMap1D result = {0};
  
  if (loose->total_count > 0){
    //- gather endpoints
    SYMS_ArenaTemp scratch = syms_get_scratch(&arena, 1);
    SYMS_U64 max_endpoint_count = loose->total_count*2;
    SYMS_1DEndPoint *endpoints = syms_push_array(scratch.arena, SYMS_1DEndPoint, max_endpoint_count);
    
    SYMS_U64 endpoint_count = 0;
    {
      SYMS_1DEndPoint *endpoint_ptr = endpoints;
      for (SYMS_SpatialMap1DNode *node = loose->first;
           node != 0;
           node = node->next){
        SYMS_U64 val = node->val;
        
        {
          SYMS_U64Range range = node->range;
          if (range.min < range.max){
            endpoint_ptr->x = range.min;
            endpoint_ptr->val = val;
            endpoint_ptr->open = 1;
            endpoint_ptr += 1;
            
            endpoint_ptr->x = range.max;
            endpoint_ptr->val = val;
            endpoint_ptr->open = 0;
            endpoint_ptr += 1;
          }
        }
        
        SYMS_U64 count = node->ranges.count;
        SYMS_U64Range *range = node->ranges.ranges;
        for (SYMS_U64 i = 0; i < count; i += 1, range += 1){
          if (range->min < range->max){
            endpoint_ptr->x = range->min;
            endpoint_ptr->val = val;
            endpoint_ptr->open = 1;
            endpoint_ptr += 1;
            
            endpoint_ptr->x = range->max;
            endpoint_ptr->val = val;
            endpoint_ptr->open = 0;
            endpoint_ptr += 1;
          }
        }
      }
      
      endpoint_count = (SYMS_U64)(endpoint_ptr - endpoints);
    }
    SYMS_U64 range_count = endpoint_count/2;
    
    //- sort endpoints
    syms_spatial_map_1d_endpoint_sort(endpoints, endpoint_count);
    
    //- walk endpoints
    SYMS_SpatialMap1D spatial_map = {0};
    SYMS_DataIdxCons data_cons = syms_data_idx_cons_alloc(scratch.arena, loose->total_count);
    if (endpoint_count > 0){
      // allocate simple map ranges
      SYMS_SpatialMap1DRange *ranges = syms_push_array(arena, SYMS_SpatialMap1DRange, range_count);
      
      // setup the walk
      SYMS_SpatialMap1DRange *range_ptr = ranges;
      SYMS_1DEndPoint *endpoint_ptr = endpoints;
      SYMS_1DEndPoint *endpoint_opl = endpoints + endpoint_count;
      SYMS_U64Set set = syms_u64_set_alloc(scratch.arena, range_count);
      
      // walk
      SYMS_U64 prev_x = 0;
      SYMS_B32 has_prev = 0;
      for (;endpoint_ptr < endpoint_opl;){
        // look at the upcoming x
        SYMS_U64 this_x = endpoint_ptr->x;
        
        // emit a range
        if (has_prev && set.count > 0){
          // get an index for this set
          SYMS_String8 data = {0};
          data.str = (SYMS_U8*)set.vals;
          data.size = set.count*sizeof(*set.vals);
          SYMS_U64 set_number = syms_data_idx_cons(scratch.arena, &data_cons, data);
          
          // fill range and advance
          range_ptr->range.min = prev_x;
          range_ptr->range.max = this_x;
          range_ptr->val = set_number;
          range_ptr += 1;
        }
        
        // consume all endpoints at this x
        for (;endpoint_ptr < endpoint_opl && endpoint_ptr->x == this_x;
             endpoint_ptr += 1){
          if (endpoint_ptr->open){
            syms_u64_set_insert(&set, endpoint_ptr->val);
          }
          else{
            syms_u64_set_erase(&set, endpoint_ptr->val);
          }
        }
        
        // update prev info
        prev_x = this_x;
        has_prev = 1;
      }
      
      // since we control emitting endpoints and consuming them,
      // if we have done everything right then the walk should end
      // with an empty set
      SYMS_ASSERT(set.count == 0);
      
      // assemble the simple spatial map
      spatial_map.ranges = ranges;
      spatial_map.count = (SYMS_U64)(range_ptr - ranges);
    }
    
    //- assemble the set data
    SYMS_U64 set_count = data_cons.count;
    SYMS_U64 *set_end_points = syms_push_array(arena, SYMS_U64, set_count + 1);
    SYMS_U8 *set_data = syms_push_array(arena, SYMS_U8, data_cons.total_size);
    {
      // setup cursors
      SYMS_U64 *set_end_point_ptr = set_end_points;
      SYMS_U64 set_data_pos = 0;
      
      // emit loop
      *set_end_point_ptr = 0;
      set_end_point_ptr += 1;
      for (SYMS_DataIdxConsNode *node = data_cons.first;
           node != 0;
           node = node->all_next){
        SYMS_U64 data_size = node->data.size;
        syms_memmove(set_data + set_data_pos, node->data.str, data_size);
        set_data_pos += data_size;
        *set_end_point_ptr = set_data_pos;
        set_end_point_ptr += 1;
      }
    }
    
    //- assemble the multi-map
    result.spatial_map = spatial_map;
    result.set_end_points = set_end_points;
    result.set_data = set_data;
    result.set_count = set_count;
    
    syms_release_scratch(scratch);
  }
  
  SYMS_ProfEnd();
  return(result);
}

SYMS_API SYMS_U64Array
syms_spatial_multi_map_1d_array_from_point(SYMS_SpatialMultiMap1D *map, SYMS_U64 x){
  SYMS_ProfBegin("syms_spatial_multi_map_array_from_point");
  SYMS_U64Array result = {0};
  SYMS_U64 set_number = syms_spatial_map_1d_value_from_point(&map->spatial_map, x);
  if (0 < set_number && set_number <= map->set_count){
    SYMS_U64 *end_points = map->set_end_points;
    SYMS_U64 set_index = set_number - 1;
    SYMS_U64 first = end_points[set_index];
    SYMS_U64 opl = end_points[set_index + 1];
    result.u64 = (SYMS_U64*)(map->set_data + first);
    result.count = (opl - first)/8;
  }
  SYMS_ProfEnd();
  return(result);
}


SYMS_API void
syms_spatial_map_1d_endpoint_sort(SYMS_1DEndPoint *endpoints, SYMS_U64 count){
  if (count > 4){
    SYMS_U64 last = count - 1;
    
    // swap
    SYMS_U64 mid = count/2;
    SYMS_Swap(SYMS_1DEndPoint, endpoints[mid], endpoints[last]);
    
    // partition
    SYMS_B32 equal_send_left = 0;
    SYMS_U64 key = endpoints[last].x;
    SYMS_U64 j = 0;
    for (SYMS_U64 i = 0; i < last; i += 1){
      SYMS_B32 send_left = (endpoints[i].x < key);
      if (!send_left && endpoints[i].x == key){
        send_left = equal_send_left;
        equal_send_left = !equal_send_left;
      }
      if (send_left){
        if (j != i){
          SYMS_Swap(SYMS_1DEndPoint, endpoints[i], endpoints[j]);
        }
        j += 1;
      }
    }
    
    SYMS_Swap(SYMS_1DEndPoint, endpoints[j], endpoints[last]);
    
    // recurse
    SYMS_U64 pivot = j;
    syms_spatial_map_1d_endpoint_sort(endpoints, pivot);
    syms_spatial_map_1d_endpoint_sort(endpoints + pivot + 1, (count - pivot - 1));
  }
  else if (count == 2){
    if (endpoints[0].x > endpoints[1].x){
      SYMS_Swap(SYMS_1DEndPoint, endpoints[0], endpoints[1]);
    }
  }
  else if (count == 3){
    if (endpoints[0].x > endpoints[1].x){
      SYMS_Swap(SYMS_1DEndPoint, endpoints[0], endpoints[1]);
    }
    if (endpoints[1].x > endpoints[2].x){
      SYMS_Swap(SYMS_1DEndPoint, endpoints[1], endpoints[2]);
      if (endpoints[0].x > endpoints[1].x){
        SYMS_Swap(SYMS_1DEndPoint, endpoints[0], endpoints[1]);
      }
    }
  }
  else if (count == 4){
    if (endpoints[0].x > endpoints[1].x){
      SYMS_Swap(SYMS_1DEndPoint, endpoints[0], endpoints[1]);
    }
    if (endpoints[2].x > endpoints[3].x){
      SYMS_Swap(SYMS_1DEndPoint, endpoints[2], endpoints[3]);
    }
    if (endpoints[0].x > endpoints[2].x){
      SYMS_Swap(SYMS_1DEndPoint, endpoints[0], endpoints[2]);
    }
    if (endpoints[1].x > endpoints[3].x){
      SYMS_Swap(SYMS_1DEndPoint, endpoints[1], endpoints[3]);
    }
    if (endpoints[1].x > endpoints[2].x){
      SYMS_Swap(SYMS_1DEndPoint, endpoints[1], endpoints[2]);
    }
  }
}

//- invariants for spatial maps

SYMS_API SYMS_B32
syms_spatial_map_1d_invariants(SYMS_SpatialMap1D *map){
  SYMS_B32 result = syms_true;
  SYMS_U64 count = map->count;
  if (count > 0){
    SYMS_SpatialMap1DRange *range = map->ranges;
    for (SYMS_U64 i = 1; i < count; i += 1){
      SYMS_INVARIANT(result, range->range.min < range->range.max);
      SYMS_INVARIANT(result, range->range.max <= (range + 1)->range.min);
    }
    SYMS_INVARIANT(result, range->range.min < range->range.max);
  }
  finish_invariants:;
  return(result);
}


////////////////////////////////
// NOTE(allen): Mapping Functions ({UnitID,FileID} -> String)

//- shared file id bucket definitions
SYMS_API SYMS_U64
syms_file_id_2_name_map_hash(SYMS_UnitID uid, SYMS_FileID file_id){
  SYMS_U64 result = syms_hash_u64(file_id + uid*97);
  return(result);
}

//- lookups into file id buckets
SYMS_API SYMS_String8
syms_file_id_2_name_map_name_from_id(SYMS_FileID2NameMap *buckets, SYMS_UnitID uid, SYMS_FileID file_id){
  SYMS_ProfBegin("syms_file_id_2_name_map_name_from_id");
  SYMS_U64 hash = syms_file_id_2_name_map_hash(uid, file_id);
  SYMS_U64 bucket_index = hash%SYMS_FILE_ID_TO_NAME_MAP_BUCKET_COUNT;
  SYMS_String8 result = {0};
  for (SYMS_FileID2NameNode *node = buckets->buckets[bucket_index];
       node != 0;
       node = node->next){
    if (node->uid == uid && node->file_id == file_id){
      result = node->name;
      if (result.size == 0){
        result.size = SYMS_U64_MAX;
      }
      break;
    }
  }
  SYMS_ProfEnd();
  return(result);
}

//- copying file id buckets
SYMS_API SYMS_FileID2NameMap
syms_file_id_2_name_map_copy(SYMS_Arena *arena, SYMS_StringCons *cons, SYMS_FileID2NameMap *map){
  SYMS_FileID2NameMap result = {0};
  
  //- allocate
  SYMS_U64 count = map->count;
  SYMS_FileID2NameNode **dst_buckets = result.buckets;
  SYMS_FileID2NameNode *nodes = syms_push_array(arena, SYMS_FileID2NameNode, count);
  
  //- fill memory
  {
    SYMS_FileID2NameNode *dst_node = nodes;
    SYMS_FileID2NameNode **dst_bucket = dst_buckets;
    SYMS_FileID2NameNode **src_bucket = map->buckets;
    SYMS_FileID2NameNode **opl = src_bucket + SYMS_FILE_ID_TO_NAME_MAP_BUCKET_COUNT;
    for (; src_bucket < opl; dst_bucket += 1, src_bucket += 1){
      SYMS_FileID2NameNode **chain_ptr = dst_bucket;
      for (SYMS_FileID2NameNode *node = *src_bucket;
           node != 0;
           node = node->next){
        // name copy
        SYMS_String8 name;
        if (cons != 0){
          name = syms_string_cons(arena, cons, node->name);
        }
        else{
          name = syms_push_string_copy(arena, node->name);
        }
        // fill dst
        dst_node->uid = node->uid;
        dst_node->file_id = node->file_id;
        dst_node->name = name;
        // link into the chain
        *chain_ptr = dst_node;
        chain_ptr = &dst_node->next;
      }
      *chain_ptr = 0;
    }
  }
  
  //- fill result
  result.count = count;
  
  return(result);
}

//- constructing file id buckets

SYMS_API void
syms_file_id_2_name_map_insert(SYMS_Arena *arena, SYMS_FileID2NameMap *map,
                               SYMS_UnitID uid, SYMS_FileID file_id, SYMS_String8 name){
  SYMS_ProfBegin("syms_file_id_2_name_map_insert");
  SYMS_U64 hash = syms_file_id_2_name_map_hash(uid, file_id);
  SYMS_U64 bucket_index = hash%SYMS_FILE_ID_TO_NAME_MAP_BUCKET_COUNT;
  
  SYMS_FileID2NameNode *new_node = syms_push_array(arena, SYMS_FileID2NameNode, 1);
  new_node->uid = uid;
  new_node->file_id = file_id;
  new_node->name = name;
  SYMS_StackPush(map->buckets[bucket_index], new_node);
  map->count += 1;
  SYMS_ProfEnd();
}


////////////////////////////////
//~ allen: Mapping Functions (String -> {UnitID,FileID})

//- copying file maps

SYMS_API SYMS_Name2FileIDMap
syms_name_2_file_id_map_copy(SYMS_Arena *arena, SYMS_StringCons *cons, SYMS_Name2FileIDMap *map){
  SYMS_ProfBegin("syms_name_2_file_id_map_copy");
  
  //- deep copies
  SYMS_U64 file_count = map->file_count;
  SYMS_Name2FileIDMapFile *files = syms_push_array(arena, SYMS_Name2FileIDMapFile, file_count);
  {
    SYMS_Name2FileIDMapFile *file_src_ptr = map->files;
    SYMS_Name2FileIDMapFile *opl = file_src_ptr + file_count;
    SYMS_Name2FileIDMapFile *file_dst_ptr = files;
    for (; file_src_ptr < opl; file_src_ptr += 1, file_dst_ptr += 1){
      // per-unit array copy
      SYMS_U64 unit_count = file_src_ptr->unit_count;
      SYMS_Name2FileIDMapUnit *units = syms_push_array(arena, SYMS_Name2FileIDMapUnit, unit_count);
      syms_memmove(units, file_dst_ptr->units, sizeof(*units)*unit_count);
      // name copy
      SYMS_String8 name;
      if (cons != 0){
        name = syms_string_cons(arena, cons, file_src_ptr->name);
      }
      else{
        name = syms_push_string_copy(arena, file_src_ptr->name);
      }
      // fill dst
      file_dst_ptr->name = name;
      file_dst_ptr->units = units;
      file_dst_ptr->unit_count = unit_count;
    }
  }
  
  //- fill result
  SYMS_Name2FileIDMap result = {0};
  result.files = files;
  result.file_count = file_count;
  
  SYMS_ProfEnd();
  
  return(result);
}

//- constructing file maps

SYMS_API SYMS_Name2FileIDMap
syms_name_2_file_id_map_bake(SYMS_Arena *arena, SYMS_Name2FileIDMapLoose *loose){
  SYMS_ProfBegin("syms_name_2_file_id_map_bake");
  
  //- bake file map down to a tight table
  SYMS_Name2FileIDMapFile *files = syms_push_array(arena, SYMS_Name2FileIDMapFile, loose->count);
  SYMS_Name2FileIDMapFile *file_ptr = files;
  for (SYMS_Name2FileIDMapFileNode *node = loose->first;
       node != 0;
       node = node->next, file_ptr += 1){
    // fill units array
    SYMS_U64 count = node->count;
    SYMS_Name2FileIDMapUnit *units = syms_push_array(arena, SYMS_Name2FileIDMapUnit, count);
    SYMS_Name2FileIDMapUnit *unit_ptr = units;
    for (SYMS_Name2FileIDMapUnitNode *unit_node = node->first;
         unit_node != 0;
         unit_node = unit_node->next, unit_ptr += 1){
      unit_ptr->uid = unit_node->uid;
      unit_ptr->file_id = unit_node->file_id;
    }
    
    // fill file
    file_ptr->name = node->name;
    file_ptr->units = units;
    file_ptr->unit_count = count;
  }
  
  //- assemble baked table type
  SYMS_Name2FileIDMap result = {0};
  result.files = files;
  result.file_count = loose->count;
  
  SYMS_ProfEnd();
  
  return(result);
}

SYMS_API void
syms_name_2_file_id_map_loose_push(SYMS_Arena *arena, SYMS_Name2FileIDMapLoose *map,
                                   SYMS_String8 name_cons, SYMS_UnitID uid, SYMS_FileID file_id){
  SYMS_ProfBegin("syms_name_2_file_id_map_loose_push");
  // find existing name node
  SYMS_Name2FileIDMapFileNode *file_node = 0;
  for (SYMS_Name2FileIDMapFileNode *node = map->first;
       node != 0;
       node = node->next){
    if (name_cons.str == node->name.str){
      file_node = node;
      break;
    }
  }
  
  // insert new node
  if (file_node == 0){
    SYMS_Name2FileIDMapFileNode *new_node = syms_push_array_zero(arena, SYMS_Name2FileIDMapFileNode, 1);
    new_node->name = name_cons;
    SYMS_QueuePush(map->first, map->last, new_node);
    map->count += 1;
    
    file_node = new_node;
  }
  
  // insert unit node
  SYMS_Name2FileIDMapUnitNode *unit_node = syms_push_array(arena, SYMS_Name2FileIDMapUnitNode, 1);
  unit_node->uid = uid;
  unit_node->file_id = file_id;
  
  SYMS_QueuePush(file_node->first, file_node->last, unit_node);
  file_node->count += 1;
  SYMS_ProfEnd();
}


////////////////////////////////
//~ allen: ID Mapping Functions

//- copying id maps

SYMS_API SYMS_IDMap
syms_id_map_copy(SYMS_Arena *arena, SYMS_IDMap *map){
  //- allocate
  SYMS_U64 bucket_count = map->bucket_count;
  SYMS_U64 node_count = map->node_count;
  SYMS_IDMapNode **dst_buckets = syms_push_array(arena, SYMS_IDMapNode*, bucket_count);
  SYMS_IDMapNode *nodes = syms_push_array(arena, SYMS_IDMapNode, node_count);
  
  //- fill memory
  {
    SYMS_IDMapNode *dst_node = nodes;
    SYMS_IDMapNode **dst_bucket = dst_buckets;
    SYMS_IDMapNode **src_bucket = map->buckets;
    SYMS_IDMapNode **opl = src_bucket + bucket_count;
    for (; src_bucket < opl; dst_bucket += 1, src_bucket += 1){
      SYMS_IDMapNode **chain_ptr = dst_bucket;
      for (SYMS_IDMapNode *node = *src_bucket;
           node != 0;
           node = node->next){
        // fill dst
        SYMS_U64 count = node->count;
        dst_node->count = count;
        syms_memmove(dst_node->key, node->key, sizeof(*node->key)*count);
        syms_memmove(dst_node->val, node->val, sizeof(*node->val)*count);
        // link into the chain
        *chain_ptr = dst_node;
        chain_ptr = &dst_node->next;
      }
    }
  }
  
  //- fill result
  SYMS_IDMap result = {0};
  result.buckets = dst_buckets;
  result.bucket_count = bucket_count;
  result.node_count = node_count;
  
  return(result);
}

//- lookups into id map

SYMS_API void*
syms_id_map_ptr_from_u64(SYMS_IDMap *map, SYMS_U64 key){
  SYMS_ProfBegin("syms_id_map_ptr_from_u64");
  void *result = 0;
  if (map->bucket_count > 0){
    SYMS_U64 hash = syms_hash_u64(key);
    SYMS_U64 index = hash%map->bucket_count;
    for (SYMS_IDMapNode *node = map->buckets[index];
         node != 0;
         node = node->next){
      SYMS_U64 count = node->count;
      SYMS_U64 *kptr = node->key;
      for (SYMS_U64 i = 0; i < count; i += 1, kptr += 1){
        if (*kptr == key){
          result = node->val[i];
          goto dbl_break;
        }
      }
    }
    dbl_break:;
  }
  SYMS_ProfEnd();
  return(result);
}

//- constructing id maps

SYMS_API SYMS_IDMap
syms_id_map_alloc(SYMS_Arena *arena, SYMS_U64 bucket_count){
  SYMS_IDMap result = {0};
  result.buckets = syms_push_array_zero(arena, SYMS_IDMapNode*, bucket_count);
  result.bucket_count = bucket_count;
  return(result);
}

SYMS_API void
syms_id_map_insert(SYMS_Arena *arena, SYMS_IDMap *map, SYMS_U64 key, void *val){
  SYMS_ProfBegin("syms_id_map_insert");
  if (map->bucket_count > 0){
    SYMS_U64 hash = syms_hash_u64(key);
    SYMS_U64 index = hash%map->bucket_count;
    
    SYMS_IDMapNode *node = map->buckets[index];
    if (node == 0 || node->count == SYMS_ID_MAP_NODE_CAP){
      syms_arena_push_align(arena, 64);
      node = syms_push_array(arena, SYMS_IDMapNode, 1);
      node->count = 0;
      SYMS_StackPush(map->buckets[index], node);
      map->node_count += 1;
    }
    
    SYMS_U64 i = node->count;
    node->key[i] = key;
    node->val[i] = val;
    node->count += 1;
  }
  SYMS_ProfEnd();
}


////////////////////////////////
//~ allen: Symbol Name Mapping Structure (String -> Array(SID))

//- lookups into symbol name map

SYMS_API SYMS_SymbolIDArray
syms_symbol_name_map_array_from_string(SYMS_SymbolNameMap *map, SYMS_String8 string){
  SYMS_ProfBegin("syms_symbol_name_map_array_from_string");
  SYMS_SymbolIDArray result = {0};
  SYMS_U64 hash = syms_hash_djb2(string);
  SYMS_U64 bucket_index = hash%SYMS_SYMBOL_NAME_MAP_BUCKET_COUNT;
  for (SYMS_SymbolNameNode *node = map->buckets[bucket_index];
       node != 0;
       node = node->next_bucket){
    if (node->hash == hash && syms_string_match(node->name, string, 0)){
      result = node->sid_array;
      break;
    }
  }
  SYMS_ProfEnd();
  return(result);
}

//- constructing symbol name maps

SYMS_API void
syms_symbol_name_map_push(SYMS_Arena *arena, SYMS_SymbolNameMapLoose *map,
                          SYMS_String8 name, SYMS_SymbolID sid){
  SYMS_ProfBegin("syms_symbol_name_map_push");
  
  SYMS_U64 hash = syms_hash_djb2(name);
  SYMS_U64 bucket_index = hash%SYMS_SYMBOL_NAME_MAP_BUCKET_COUNT;
  
  // find existing node
  SYMS_SymbolNameNodeLoose *node = 0;
  for (SYMS_SymbolNameNodeLoose *bucket = map->buckets[bucket_index];
       bucket != 0;
       bucket = bucket->next_bucket){
    if (bucket->hash == hash &&
        syms_string_match(name, bucket->name, 0)){
      node = bucket;
      break;
    }
  }
  
  // create node
  if (node == 0){
    node = syms_push_array_zero(arena, SYMS_SymbolNameNodeLoose, 1);
    SYMS_StackPush_N(map->buckets[bucket_index], node, next_bucket);
    SYMS_QueuePush(map->first, map->last, node);
    map->node_count += 1;
    node->name = syms_push_string_copy(arena, name);
    node->hash = hash;
  }
  
  // insert sid
  {
    SYMS_SymbolIDNode *sid_node = syms_push_array_zero(arena, SYMS_SymbolIDNode, 1);
    SYMS_QueuePush(node->sid_list.first, node->sid_list.last, sid_node);
    node->sid_list.count += 1;
    sid_node->id = sid;
  }
  
  SYMS_ProfEnd();
}

SYMS_API SYMS_SymbolNameMap
syms_symbol_name_map_bake(SYMS_Arena *arena, SYMS_SymbolNameMapLoose *loose){
  SYMS_ProfBegin("syms_symbol_name_map_bake");
  
  SYMS_SymbolNameMap result = {0};
  SYMS_SymbolNameNode **buckets = result.buckets;
  
  //- allocate map memory
  SYMS_U64 node_count = loose->node_count;
  SYMS_SymbolNameNode *nodes = syms_push_array(arena, SYMS_SymbolNameNode, node_count);
  
  //- fill map from loose
  SYMS_SymbolNameNode *node_ptr = nodes;
  for (SYMS_SymbolNameNodeLoose *loose_node = loose->first;
       loose_node != 0;
       loose_node = loose_node->next){
    // copy node's name
    SYMS_String8 name = syms_push_string_copy(arena, loose_node->name);
    
    // grab the node's hash
    SYMS_U64 hash = loose_node->hash;
    
    // fill node's usid array
    SYMS_SymbolIDArray sid_array = {0};
    sid_array.count = loose_node->sid_list.count;
    sid_array.ids = syms_push_array(arena, SYMS_SymbolID, sid_array.count);
    {
      SYMS_SymbolID *sid_ptr = sid_array.ids;
      for (SYMS_SymbolIDNode *sid_node = loose_node->sid_list.first;
           sid_node != 0;
           sid_node = sid_node->next){
        *sid_ptr = sid_node->id;
        sid_ptr += 1;
      }
    }
    
    // insert node to bucket
    SYMS_U64 bucket_index = hash%SYMS_SYMBOL_NAME_MAP_BUCKET_COUNT;
    SYMS_StackPush_N(buckets[bucket_index], node_ptr, next_bucket);
    
    // fill node ptr
    node_ptr->name = name;
    node_ptr->hash = hash;
    node_ptr->sid_array = sid_array;
    node_ptr += 1;
  }
  
  //- assemble result
  result.nodes = nodes;
  result.node_count = node_count;
  
  SYMS_ProfEnd();
  return(result);
}


////////////////////////////////
//~ allen: Line Tables

//- lookups into line tables

SYMS_API SYMS_U64
syms_line_index_from_voff__binary_search(SYMS_Line *lines, SYMS_U64 ender_index, SYMS_U64 voff){
  SYMS_ProfBegin("syms_line_index_from_voff__binary_search");
  SYMS_U64 result = SYMS_U64_MAX;
  if (ender_index > 0 && lines->voff <= voff && voff < (lines + ender_index)->voff){
    //- binary search:
    //   max index s.t. lines[index].virt_off <= x
    //  we must allow for cases where (i != j) and (lines[i].virt_off == lines[j].virt_off)
    //  thus (lines[index].virt_off == x) does not prove that index saatisfies the requiement
    SYMS_U64 first = 0;
    SYMS_U64 opl = ender_index;
    for (;;){
      SYMS_U64 mid = (first + opl)/2;
      SYMS_U64 lvoff = lines[mid].voff;
      if (lvoff > voff){
        opl = mid;
      }
      else{
        first = mid;
      }
      if (first + 1 >= opl){
        break;
      }
    }
    result = first;
  }
  SYMS_ProfEnd();
  return(result);
}

SYMS_API SYMS_Line
syms_line_from_sequence_voff(SYMS_LineTable *line_table, SYMS_U64 seq_number, SYMS_U64 voff){
  SYMS_Line result = {0};
  if (0 < seq_number && seq_number <= line_table->sequence_count){
    SYMS_U64 first = line_table->sequence_index_array[seq_number - 1];
    SYMS_U64 opl = line_table->sequence_index_array[seq_number];
    SYMS_U64 last = opl - 1;
    SYMS_U64 index = syms_line_index_from_voff__binary_search(line_table->line_array + first, last - first, voff);
    result = line_table->line_array[first + index];
  }
  return(result);
}

//- copying and rewriting line tables

SYMS_API SYMS_LineTable
syms_line_table_copy(SYMS_Arena *arena, SYMS_LineTable *line_table){
  SYMS_ProfBegin("syms_line_table_copy");
  
  SYMS_LineTable result = {0};
  if (line_table->sequence_index_array != 0){
    //- copy arrays
    SYMS_U64 sequence_count = line_table->sequence_count;
    SYMS_U64 *sequence_index_array = syms_push_array(arena, SYMS_U64, sequence_count + 1);
    syms_memmove(sequence_index_array, line_table->sequence_index_array, sizeof(SYMS_U64)*(sequence_count + 1));
    
    SYMS_U64 line_count = line_table->line_count;
    SYMS_Line *line_array = syms_push_array(arena, SYMS_Line, line_count);
    syms_memmove(line_array, line_table->line_array, sizeof(*line_array)*line_count);
    
    //- assemble result
    result.sequence_index_array = sequence_index_array;
    result.sequence_count = sequence_count;
    result.line_array = line_array;
    result.line_count = line_count;
  }
  
  SYMS_ProfEnd();
  
  return(result);
}

SYMS_API void
syms_line_table_rewrite_file_ids_in_place(SYMS_FileIDArray *file_ids, SYMS_LineTable *line_table){
  SYMS_ProfBegin("syms_line_table_rewrite_file_ids_in_place");
  
  // check for file ids (no rewrite necessary if this array is empty)
  if (file_ids->count != 0){
    
    // NOTE(allen): if this is slow the first easy step is to build a faster lookup for (file_id -> index)
    // currently this is just a linear scan with a most recently used cache
    
    // most recently used cache
    SYMS_FileID last_file_id = 0;
    SYMS_U64    last_file_index = SYMS_U64_MAX;
    
    // rewrite loop
    SYMS_FileID *file_id_array = file_ids->ids;
    SYMS_FileID *file_id_opl = file_id_array + file_ids->count;
    SYMS_Line *line_ptr = line_table->line_array;
    SYMS_Line *opl = line_ptr + line_table->line_count;
    for (; line_ptr < opl; line_ptr += 1){
      // decide on the index
      if (line_ptr->src_coord.file_id != last_file_id){
        last_file_id = line_ptr->src_coord.file_id;
        last_file_index = 1;
        for (SYMS_FileID *file_id_ptr = file_id_array;
             file_id_ptr < file_id_opl;
             file_id_ptr += 1, last_file_index += 1){
          if (*file_id_ptr == last_file_id){
            break;
          }
        }
      }
      SYMS_U64 index = last_file_index;
      
      // rewrite the file id
      line_ptr->src_coord.file_id = index;
    }
  }
  
  SYMS_ProfEnd();
}

SYMS_API SYMS_LineTable
syms_line_table_with_indexes_from_parse(SYMS_Arena *arena, SYMS_LineParseOut *parse){
  SYMS_LineTable result = syms_line_table_copy(arena, &parse->line_table);
  syms_line_table_rewrite_file_ids_in_place(&parse->file_id_array, &result);
  return(result);
}


////////////////////////////////
//~ allen: Line To Addr Map

SYMS_API SYMS_FileToLineToAddrMap
syms_line_to_addr_map_from_line_table(SYMS_Arena *arena, SYMS_LineTable *line_table){
  SYMS_ArenaTemp scratch = syms_get_scratch(0, 0);
  
  //- setup loose map
  SYMS_FileToLineToAddrLoose loose = {0};
  
  //- last-used cache for file nodes
  SYMS_FileToLineToAddrLooseFile *cached_file_node = 0;
  SYMS_U64 cached_file_id = 0;
  
  //- read lines
  SYMS_U64 *seq_idx_ptr = line_table->sequence_index_array;
  SYMS_U64 seq_count = line_table->sequence_count;
  for (SYMS_U64 i = 0; i < seq_count; i += 1){
    // get sequence range & inc
    SYMS_U64 first = *seq_idx_ptr;
    seq_idx_ptr += 1;
    SYMS_U64 opl = *seq_idx_ptr;
    
    // iterate lines
    if (first < opl){
      SYMS_Line *line_ptr = line_table->line_array + first;
      SYMS_U64 last = opl - 1;
      for (SYMS_U64 j = first; j < last; j += 1, line_ptr += 1){
        // grab line data
        SYMS_U64 line_file_id = line_ptr->src_coord.file_id;
        SYMS_U32 line_number = line_ptr->src_coord.line;
        
        // get file node
        SYMS_FileToLineToAddrLooseFile *file_node = 0;
        if (line_file_id == cached_file_id){
          file_node = cached_file_node;
        }
        else{
          for (SYMS_FileToLineToAddrLooseFile *node = loose.first;
               node != 0;
               node = node->next){
            if (node->file_id == line_file_id){
              file_node = node;
              break;
            }
          }
        }
        if (file_node == 0){
          file_node = syms_push_array_zero(scratch.arena, SYMS_FileToLineToAddrLooseFile, 1);
          SYMS_QueuePush(loose.first, loose.last, file_node);
          loose.count += 1;
          file_node->file_id = line_file_id;
        }
        
        // update the file node cache slot
        cached_file_id = line_file_id;
        cached_file_node = file_node;
        
        // get line node
        SYMS_FileToLineToAddrLooseLine *line_node = 0;
        for (SYMS_FileToLineToAddrLooseLine *node = file_node->first;
             node != 0;
             node = node->next){
          if (node->line == line_number){
            line_node = node;
            break;
          }
        }
        if (line_node == 0){
          line_node = syms_push_array_zero(scratch.arena, SYMS_FileToLineToAddrLooseLine, 1);
          SYMS_QueuePush(file_node->first, file_node->last, line_node);
          file_node->line_count += 1;
          line_node->line = line_number;
        }
        
        // push range
        SYMS_U64Range range = syms_make_u64_range(line_ptr->voff, (line_ptr + 1)->voff);
        syms_u64_range_list_push(scratch.arena, &line_node->ranges, range);
        file_node->range_count += 1;
      }
    }
  }
  
  //- convert loose to buckets & maps
  SYMS_U64 bucket_count = 0;
  if (loose.count > 0){
    bucket_count = ((loose.count + 1)*3/2) | 1;
  }
  
  SYMS_FileToLineToAddrNode **buckets = syms_push_array_zero(arena, SYMS_FileToLineToAddrNode*, bucket_count);
  for (SYMS_FileToLineToAddrLooseFile *loose_file_node = loose.first;
       loose_file_node != 0;
       loose_file_node = loose_file_node->next){
    SYMS_ArenaTemp temp = syms_arena_temp_begin(scratch.arena);
    
    // grab counts
    SYMS_U64 line_count = loose_file_node->line_count;
    SYMS_U64 range_count = loose_file_node->range_count;
    
    // create sorted node array
    SYMS_FileToLineToAddrLooseLine **array = syms_push_array(scratch.arena, SYMS_FileToLineToAddrLooseLine*,
                                                             line_count);
    {
      SYMS_FileToLineToAddrLooseLine **line_ptr = array;
      for (SYMS_FileToLineToAddrLooseLine *loose_line_node = loose_file_node->first;
           loose_line_node != 0;
           loose_line_node = loose_line_node->next, line_ptr += 1){
        *line_ptr = loose_line_node;
      }
    }
    syms_line_to_addr_line_sort(array, line_count);
    
    // fill line map arrays
    SYMS_U64Range *ranges = syms_push_array(arena, SYMS_U64Range, range_count);
    SYMS_U32 *line_range_indexes = syms_push_array(arena, SYMS_U32, line_count + 1);
    SYMS_U32 *line_numbers = syms_push_array(arena, SYMS_U32, line_count);
    
    SYMS_U32 line_range_index = 0;
    SYMS_U32 *line_range_index_ptr = line_range_indexes;
    SYMS_U32 *line_number_ptr = line_numbers;
    
    {
      SYMS_FileToLineToAddrLooseLine **line_ptr = array;
      for (SYMS_U64 i = 0; i < line_count; i += 1, line_ptr += 1){
        // fills
        *line_number_ptr = (**line_ptr).line;
        *line_range_index_ptr = line_range_index;
        SYMS_U64Range *first_range_ptr = ranges + line_range_index;
        SYMS_U64Range *range_ptr = first_range_ptr;
        for (SYMS_U64RangeNode *node = (**line_ptr).ranges.first;
             node != 0;
             node = node->next, range_ptr += 1){
          *range_ptr = node->range;
        }
        
        // incs
        line_number_ptr += 1;
        line_range_index_ptr += 1;
        line_range_index += (SYMS_U64)(range_ptr - first_range_ptr);
      }
      
      // fill ender index
      *line_range_index_ptr = line_range_index;
    }
    
    // assemble the line map
    SYMS_LineToAddrMap *new_map = syms_push_array(arena, SYMS_LineToAddrMap, 1);
    new_map->ranges = ranges;
    new_map->line_range_indexes = line_range_indexes;
    new_map->line_numbers = line_numbers;
    new_map->line_count = line_count;
    
    // insert bucket
    SYMS_FileToLineToAddrNode *new_file_node = syms_push_array(arena, SYMS_FileToLineToAddrNode, 1);
    new_file_node->file_id = loose_file_node->file_id;
    new_file_node->map = new_map;
    SYMS_U64 bucket_index = loose_file_node->file_id%bucket_count;
    SYMS_StackPush(buckets[bucket_index], new_file_node);
    
    syms_arena_temp_end(temp);
  }
  
  syms_release_scratch(scratch);
  
  //- build result
  SYMS_FileToLineToAddrMap result = {0};
  result.buckets = buckets;
  result.bucket_count = bucket_count;
  
  return(result);
}

SYMS_API SYMS_LineToAddrMap*
syms_line_to_addr_map_lookup_file_id(SYMS_FileToLineToAddrMap *map, SYMS_FileID file_id){
  SYMS_LineToAddrMap *result = &syms_line_to_addr_map_nil;
  if (map->bucket_count > 0){
    SYMS_U64 bucket_index = file_id%map->bucket_count;
    for (SYMS_FileToLineToAddrNode *node = map->buckets[bucket_index];
         node != 0;
         node = node->next){
      if (node->file_id == file_id){
        result = node->map;
        break;
      }
    }
  }
  return(result);
}

SYMS_API SYMS_U64RangeArray
syms_line_to_addr_map_lookup_nearest_line_number(SYMS_LineToAddrMap *map, SYMS_U32 line,
                                                 SYMS_U32 *actual_line_out){
  SYMS_U64 count = map->line_count;
  SYMS_U32 *numbers = map->line_numbers;
  SYMS_U64 index = syms_index_from_n__u32__binary_search_round_up(numbers, count, line);
  if (count > 0 && index >= count){
    index = count - 1;
  }
  SYMS_U64RangeArray result = {0};
  if (index < count){
    SYMS_U32 *range_indexes = map->line_range_indexes;
    result.ranges = map->ranges + range_indexes[index];
    result.count = range_indexes[index + 1] - range_indexes[index];
    *actual_line_out = numbers[index];
  }
  return(result);
}


SYMS_API void
syms_line_to_addr_line_sort(SYMS_FileToLineToAddrLooseLine **array, SYMS_U64 count){
  SYMS_ProfBegin("syms_line_to_addr_line_sort");
  syms_line_to_addr_line_sort__rec(array, count);
  SYMS_ProfEnd();
}

SYMS_API void
syms_line_to_addr_line_sort__rec(SYMS_FileToLineToAddrLooseLine **array, SYMS_U64 count){
  if (count > 4){
    SYMS_U64 last = count - 1;
    
    // swap
    SYMS_U64 mid = count/2;
    SYMS_Swap(SYMS_FileToLineToAddrLooseLine*, array[mid], array[last]);
    
    // partition
    SYMS_U64 key = array[last]->line;
    SYMS_U64 j = 0;
    for (SYMS_U64 i = 0; i < last; i += 1){
      if (array[i]->line < key){
        if (j != i){
          SYMS_Swap(SYMS_FileToLineToAddrLooseLine*, array[i], array[j]);
        }
        j += 1;
      }
    }
    
    SYMS_Swap(SYMS_FileToLineToAddrLooseLine*, array[j], array[last]);
    
    // recurse
    SYMS_U64 pivot = j;
    syms_line_to_addr_line_sort__rec(array, pivot);
    syms_line_to_addr_line_sort__rec(array + pivot + 1, (count - pivot - 1));
  }
  else if (count == 2){
    if (array[0]->line > array[1]->line){
      SYMS_Swap(SYMS_FileToLineToAddrLooseLine*, array[0], array[1]);
    }
  }
  else if (count == 3){
    if (array[0]->line > array[1]->line){
      SYMS_Swap(SYMS_FileToLineToAddrLooseLine*, array[0], array[1]);
    }
    if (array[1]->line > array[2]->line){
      SYMS_Swap(SYMS_FileToLineToAddrLooseLine*, array[1], array[2]);
      if (array[0]->line > array[1]->line){
        SYMS_Swap(SYMS_FileToLineToAddrLooseLine*, array[0], array[1]);
      }
    }
  }
  else if (count == 4){
    if (array[0]->line > array[1]->line){
      SYMS_Swap(SYMS_FileToLineToAddrLooseLine*, array[0], array[1]);
    }
    if (array[2]->line > array[3]->line){
      SYMS_Swap(SYMS_FileToLineToAddrLooseLine*, array[2], array[3]);
    }
    if (array[0]->line > array[2]->line){
      SYMS_Swap(SYMS_FileToLineToAddrLooseLine*, array[0], array[2]);
    }
    if (array[1]->line > array[3]->line){
      SYMS_Swap(SYMS_FileToLineToAddrLooseLine*, array[1], array[3]);
    }
    if (array[1]->line > array[2]->line){
      SYMS_Swap(SYMS_FileToLineToAddrLooseLine*, array[1], array[2]);
    }
  }
}


////////////////////////////////
//~ allen: Copies & Operators for Other Data Structures

SYMS_API SYMS_String8Array
syms_string_array_copy(SYMS_Arena *arena, SYMS_StringCons *cons, SYMS_String8Array *array){
  SYMS_ProfBegin("syms_string_array_copy");
  
  //- allocate
  SYMS_U64 count = array->count;
  SYMS_String8 *strings = syms_push_array(arena, SYMS_String8, count);
  
  //- fill
  SYMS_String8 *dst_ptr = strings;
  SYMS_String8 *src_ptr = array->strings;
  SYMS_String8 *opl = src_ptr + count;
  for (; src_ptr < opl; src_ptr += 1, dst_ptr += 1){
    SYMS_String8 string;
    if (cons != 0){
      string = syms_string_cons(arena, cons, *src_ptr);
    }
    else{
      string = syms_push_string_copy(arena, *src_ptr);
    }
    *dst_ptr = string;
  }
  
  //- assemble result
  SYMS_String8Array result = {0};
  result.strings = strings;
  result.count = count;
  
  SYMS_ProfEnd();
  
  return(result);
}

SYMS_API SYMS_LinkNameRecArray
syms_link_name_record_copy(SYMS_Arena *arena, SYMS_LinkNameRecArray *array){
  SYMS_ProfBegin("syms_link_name_record_copy");
  SYMS_LinkNameRecArray result = {0};
  result.count = array->count;
  result.recs = syms_push_array(arena, SYMS_LinkNameRec, result.count);
  
  SYMS_LinkNameRec *src = array->recs;
  SYMS_LinkNameRec *dst = result.recs;
  for (SYMS_U64 i = 0; i < result.count; i += 1, src += 1, dst += 1){
    dst->name = syms_push_string_copy(arena, src->name);
    dst->voff = src->voff;
  }
  
  SYMS_ProfEnd();
  return(result);
}

////////////////////////////////
//~ allen: Binary Search Functions

SYMS_API SYMS_U64
syms_index_from_n__u32__binary_search_round_up(SYMS_U32 *v, SYMS_U64 count, SYMS_U32 n){
  SYMS_ProfBegin("syms_index_from_n__u32__binary_search_round_up");
  SYMS_U64 result = SYMS_U64_MAX;
  if (count > 0 && n <= v[count - 1]){
    //- binary search:
    //   minimum index s.t. v[index] >= n
    //  in this one we assume:
    //   (i != j) implies (v[i] != v[j])
    //   thus if (v[index] == n) then index already satisfies the requirement
    SYMS_U64 first = 0;
    SYMS_U64 opl = count;
    for (;;){
      SYMS_U64 mid = (first + opl - 1)/2;
      SYMS_U64 w = v[mid];
      if (w < n){
        first = mid + 1;
      }
      else if (w > n){
        opl = mid + 1;
      }
      else{
        first = mid;
        break;
      }
      if (first + 1 >= opl){
        break;
      }
    }
    result = first;
  }
  SYMS_ProfEnd();
  return(result);
}

#endif //SYMS_DATA_STRUCTURES_C
