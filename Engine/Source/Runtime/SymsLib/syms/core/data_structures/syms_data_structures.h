// Copyright Epic Games, Inc. All Rights Reserved.
/* date = September 16th 2021 1:52 pm */

#ifndef SYMS_DATA_STRUCTURES_H
#define SYMS_DATA_STRUCTURES_H

////////////////////////////////
//~ allen: String Cons Structure

// deduplicates equivalent strings

#define SYMS_STRING_CONS_BUCKET_COUNT 1024

typedef struct SYMS_StringConsNode{
  struct SYMS_StringConsNode *next;
  SYMS_String8 string;
  SYMS_U64 hash;
} SYMS_StringConsNode;

typedef struct SYMS_StringCons{
  SYMS_StringConsNode *buckets[SYMS_STRING_CONS_BUCKET_COUNT];
} SYMS_StringCons;

// assign sequential indexes to small variable length blobs

typedef struct SYMS_DataIdxConsNode{
  struct SYMS_DataIdxConsNode *bucket_next;
  struct SYMS_DataIdxConsNode *all_next;
  SYMS_String8 data;
  SYMS_U64 hash;
  SYMS_U64 id;
} SYMS_DataIdxConsNode;

typedef struct SYMS_DataIdxCons{
  SYMS_DataIdxConsNode **buckets;
  SYMS_U64 bucket_count;
  SYMS_DataIdxConsNode *first;
  SYMS_DataIdxConsNode *last;
  SYMS_U64 count;
  SYMS_U64 total_size;
} SYMS_DataIdxCons;

////////////////////////////////
//~ allen: U64 Set

typedef struct SYMS_U64Set{
  SYMS_U64 *vals;
  SYMS_U64 count;
  SYMS_U64 cap;
} SYMS_U64Set;

////////////////////////////////
//~ allen: 1D Spatial Mapping Structure

// assigns a value to ranges of unsigned 64-bit values
// ranges specified in half-open intervals: [min,max)
// ranges must be non-overlapping

typedef struct SYMS_SpatialMap1DRange{
  SYMS_U64Range range;
  SYMS_U64 val;
} SYMS_SpatialMap1DRange;

// TODO(allen): optimize this by using end points instead of ranges
typedef struct SYMS_SpatialMap1D{
  SYMS_SpatialMap1DRange *ranges;
  SYMS_U64 count;
} SYMS_SpatialMap1D;

//- loose version
typedef struct SYMS_SpatialMap1DNode{
  struct SYMS_SpatialMap1DNode *next;
  SYMS_U64Range range;
  SYMS_U64RangeArray ranges;
  SYMS_U64 val;
} SYMS_SpatialMap1DNode;

typedef struct SYMS_SpatialMap1DLoose{
  SYMS_SpatialMap1DNode *first;
  SYMS_SpatialMap1DNode *last;
  SYMS_U64 total_count;
} SYMS_SpatialMap1DLoose;

//- version with support for overlapping ranges
typedef struct SYMS_SpatialMultiMap1D{
  SYMS_SpatialMap1D spatial_map;
  // set_end_points length: (set_count + 1)
  SYMS_U64 *set_end_points; 
  // set_data length: sizeof(SYMS_U64)*(set_end_points[set_count])
  SYMS_U8 *set_data;
  SYMS_U64 set_count;
} SYMS_SpatialMultiMap1D;

typedef struct SYMS_1DEndPoint{
  SYMS_U64 x;
  SYMS_U64 val;
  SYMS_B32 open;
} SYMS_1DEndPoint;

////////////////////////////////
//~ allen: File Mapping Structure ({UnitID,FileID} -> String)

// maps a unit-id,file-id pair to a string
// organized as a hash table to opimize for key based lookups

#define SYMS_FILE_ID_TO_NAME_MAP_BUCKET_COUNT 1024

typedef struct SYMS_FileID2NameNode{
  struct SYMS_FileID2NameNode *next;
  // key
  SYMS_UnitID uid;
  SYMS_FileID file_id;
  // value
  SYMS_String8 name;
} SYMS_FileID2NameNode;

typedef struct SYMS_FileID2NameMap{
  SYMS_FileID2NameNode *buckets[SYMS_FILE_ID_TO_NAME_MAP_BUCKET_COUNT];
  SYMS_U64 count;
} SYMS_FileID2NameMap;

////////////////////////////////
//~ allen: File Mapping Structure (String -> {UnitID,FileID})

// maps strings to a set of unit-id,file-id pairs
// oragnized in an array of strings with each string
//  equipped with an array of unit-id,file-id pairs
//
// we don't organize this as a hash table because having a
//  list of all known source files is useful and there are
//  many string matching rules that might want to be used
//  for lookups into this data

typedef struct SYMS_Name2FileIDMapUnit{
  SYMS_UnitID uid;
  SYMS_FileID file_id;
} SYMS_Name2FileIDMapUnit;

typedef struct SYMS_Name2FileIDMapFile{
  SYMS_String8 name;
  SYMS_Name2FileIDMapUnit *units;
  SYMS_U64 unit_count;
} SYMS_Name2FileIDMapFile;

typedef struct SYMS_Name2FileIDMap{
  SYMS_Name2FileIDMapFile *files;
  SYMS_U64 file_count;
} SYMS_Name2FileIDMap;

//- loose version
typedef struct SYMS_Name2FileIDMapUnitNode{
  struct SYMS_Name2FileIDMapUnitNode *next;
  SYMS_UnitID uid;
  SYMS_FileID file_id;
} SYMS_Name2FileIDMapUnitNode;

typedef struct SYMS_Name2FileIDMapFileNode{
  struct SYMS_Name2FileIDMapFileNode *next;
  SYMS_String8 name;
  SYMS_Name2FileIDMapUnitNode *first;
  SYMS_Name2FileIDMapUnitNode *last;
  SYMS_U64 count;
} SYMS_Name2FileIDMapFileNode;

typedef struct SYMS_Name2FileIDMapLoose{
  SYMS_Name2FileIDMapFileNode *first;
  SYMS_Name2FileIDMapFileNode *last;
  SYMS_U64 count;
} SYMS_Name2FileIDMapLoose;


////////////////////////////////
//~ allen: ID Mapping Structure

// maps unsigned 64-bit values to arbitrary user pointers
// organized as a hash table to opimize key based lookups

#define SYMS_ID_MAP_NODE_CAP 3

typedef struct SYMS_IDMapNode{
  struct SYMS_IDMapNode *next;
  SYMS_U64 count;
  SYMS_U64 key[SYMS_ID_MAP_NODE_CAP];
  void *val[SYMS_ID_MAP_NODE_CAP];
} SYMS_IDMapNode;

typedef struct SYMS_IDMap{
  SYMS_IDMapNode **buckets;
  SYMS_U64 bucket_count;
  SYMS_U64 node_count;
} SYMS_IDMap;


////////////////////////////////
//~ allen: Symbol Name Mapping Structure (String -> Array(SID))

// maps strings to lists of SIDs
// organized as an array of nodes and a hash table simultaneously
// so that single name lookups are accelerated, and filter matching
// is also possible.

#define SYMS_SYMBOL_NAME_MAP_BUCKET_COUNT 1024

typedef struct SYMS_SymbolNameNode{
  struct SYMS_SymbolNameNode *next_bucket;
  SYMS_String8 name;
  SYMS_U64 hash;
  SYMS_SymbolIDArray sid_array;
} SYMS_SymbolNameNode;

typedef struct SYMS_SymbolNameMap{
  SYMS_SymbolNameNode *buckets[SYMS_SYMBOL_NAME_MAP_BUCKET_COUNT];
  SYMS_SymbolNameNode *nodes;
  SYMS_U64 node_count;
} SYMS_SymbolNameMap;

//- loose version
typedef struct SYMS_SymbolNameNodeLoose{
  struct SYMS_SymbolNameNodeLoose *next;
  struct SYMS_SymbolNameNodeLoose *next_bucket;
  SYMS_String8 name;
  SYMS_U64 hash;
  SYMS_SymbolIDList sid_list;
} SYMS_SymbolNameNodeLoose;

typedef struct SYMS_SymbolNameMapLoose{
  SYMS_SymbolNameNodeLoose *buckets[SYMS_SYMBOL_NAME_MAP_BUCKET_COUNT];
  SYMS_SymbolNameNodeLoose *first;
  SYMS_SymbolNameNodeLoose *last;
  SYMS_U64 node_count;
} SYMS_SymbolNameMapLoose;


////////////////////////////////
//~ allen: Line To Addr Map Structure

typedef struct SYMS_LineToAddrMap{
  SYMS_U64Range *ranges;
  // line_range_indexes ranges from [0,line_count] inclusive so that:
  // for-all i in [0,line_count):
  //   (line_range_indexes[i + 1] - line_range_indexes[i]) == # of ranges for line at index i
  SYMS_U32 *line_range_indexes;
  SYMS_U32 *line_numbers;
  SYMS_U64 line_count;
} SYMS_LineToAddrMap;

typedef struct SYMS_FileToLineToAddrNode{
  struct SYMS_FileToLineToAddrNode *next;
  SYMS_FileID file_id;
  SYMS_LineToAddrMap *map;
} SYMS_FileToLineToAddrNode;

typedef struct SYMS_FileToLineToAddrMap{
  SYMS_FileToLineToAddrNode **buckets;
  SYMS_U64 bucket_count;
} SYMS_FileToLineToAddrMap;

//- loose version
typedef struct SYMS_FileToLineToAddrLooseLine{
  struct SYMS_FileToLineToAddrLooseLine *next;
  SYMS_U32 line;
  SYMS_U64RangeList ranges;
} SYMS_FileToLineToAddrLooseLine;

typedef struct SYMS_FileToLineToAddrLooseFile{
  struct SYMS_FileToLineToAddrLooseFile *next;
  SYMS_FileID file_id;
  SYMS_FileToLineToAddrLooseLine *first;
  SYMS_FileToLineToAddrLooseLine *last;
  SYMS_U64 line_count;
  SYMS_U64 range_count;
} SYMS_FileToLineToAddrLooseFile;

typedef struct SYMS_FileToLineToAddrLoose{
  SYMS_FileToLineToAddrLooseFile *first;
  SYMS_FileToLineToAddrLooseFile *last;
  SYMS_U64 count;
} SYMS_FileToLineToAddrLoose;


////////////////////////////////
//~ allen: String Cons Functions

SYMS_API SYMS_String8     syms_string_cons(SYMS_Arena *arena, SYMS_StringCons *cons, SYMS_String8 string);

SYMS_API SYMS_DataIdxCons syms_data_idx_cons_alloc(SYMS_Arena *arena, SYMS_U64 bucket_count);
SYMS_API SYMS_U64         syms_data_idx_cons(SYMS_Arena *arena, SYMS_DataIdxCons *cons, SYMS_String8 data);

////////////////////////////////
//~ allen: U64 Set

SYMS_API SYMS_U64Set syms_u64_set_alloc(SYMS_Arena *arena, SYMS_U64 cap);
SYMS_API SYMS_U64    syms_u64_set__bs(SYMS_U64Set *set, SYMS_U64 x);
SYMS_API SYMS_B32    syms_u64_set_insert(SYMS_U64Set *set, SYMS_U64 x);
SYMS_API void        syms_u64_set_erase(SYMS_U64Set *set, SYMS_U64 x);

////////////////////////////////
//~ allen: 1D Spatial Mapping Functions (Overlaps Not Allowed)

//- lookups into spatial maps
SYMS_API SYMS_U64          syms_spatial_map_1d_binary_search(SYMS_SpatialMap1D *map, SYMS_U64 x);
SYMS_API SYMS_U64          syms_spatial_map_1d_index_from_point(SYMS_SpatialMap1D *map, SYMS_U64 x);
SYMS_API SYMS_U64          syms_spatial_map_1d_value_from_point(SYMS_SpatialMap1D *map, SYMS_U64 x);

//- copying spatial maps
SYMS_API SYMS_SpatialMap1D syms_spatial_map_1d_copy(SYMS_Arena *arena, SYMS_SpatialMap1D *map);

//- constructing spatial maps
SYMS_API void              syms_spatial_map_1d_loose_push(SYMS_Arena *arena, SYMS_SpatialMap1DLoose *loose,
                                                          SYMS_U64 val, SYMS_U64RangeArray ranges);
SYMS_API void              syms_spatial_map_1d_loose_push_single(SYMS_Arena *arena, SYMS_SpatialMap1DLoose *loose,
                                                                 SYMS_U64 val, SYMS_U64Range range);
SYMS_API SYMS_SpatialMap1D syms_spatial_map_1d_bake(SYMS_Arena *arena, SYMS_SpatialMap1DLoose *loose);

SYMS_API SYMS_B32          syms_spatial_map_1d_array_check_sorted(SYMS_SpatialMap1DRange *ranges, SYMS_U64 count);
SYMS_API void              syms_spatial_map_1d_array_sort(SYMS_SpatialMap1DRange *ranges, SYMS_U64 count);
SYMS_API void              syms_spatial_map_1d_array_sort__rec(SYMS_SpatialMap1DRange *ranges, SYMS_U64 count);

//- support for the overlapping ranges
SYMS_API SYMS_SpatialMultiMap1D syms_spatial_multi_map_1d_bake(SYMS_Arena *arena, SYMS_SpatialMap1DLoose *loose);

SYMS_API SYMS_U64Array syms_spatial_multi_map_1d_array_from_point(SYMS_SpatialMultiMap1D *map, SYMS_U64 x);

SYMS_API void              syms_spatial_map_1d_endpoint_sort(SYMS_1DEndPoint *endpoints, SYMS_U64 count);

// TODO(allen): copying spatial multi maps

//- invariants for spatial maps
SYMS_API SYMS_B32          syms_spatial_map_1d_invariants(SYMS_SpatialMap1D *map);

////////////////////////////////
//~ allen: File Mapping Functions ({UnitID,FileID} -> String)

//- shared file id bucket definitions
SYMS_API SYMS_U64            syms_file_id_2_name_map_hash(SYMS_UnitID uid, SYMS_FileID file_id);

//- lookups into file id buckets
SYMS_API SYMS_String8        syms_file_id_2_name_map_name_from_id(SYMS_FileID2NameMap *map,
                                                                  SYMS_UnitID uid, SYMS_FileID file_id);

//- copying file id buckets
SYMS_API SYMS_FileID2NameMap syms_file_id_2_name_map_copy(SYMS_Arena *arena, SYMS_StringCons *cons_optional,
                                                          SYMS_FileID2NameMap *map);

//- constructing file id buckets
SYMS_API void                syms_file_id_2_name_map_insert(SYMS_Arena *arena, SYMS_FileID2NameMap *map,
                                                            SYMS_UnitID uid, SYMS_FileID file_id,
                                                            SYMS_String8 name);


////////////////////////////////
//~ allen: File Mapping Functions (String -> {UnitID,FileID})

//- copying file maps
SYMS_API SYMS_Name2FileIDMap syms_name_2_file_id_map_copy(SYMS_Arena *arena, SYMS_StringCons *cons_optional,
                                                          SYMS_Name2FileIDMap *file_map);

//- constructing file maps
SYMS_API SYMS_Name2FileIDMap syms_name_2_file_id_map_bake(SYMS_Arena *arena, SYMS_Name2FileIDMapLoose *loose);
// allen: Strings passed to this function should all be cons'ed in the same cons structure first.
SYMS_API void syms_name_2_file_id_map_loose_push(SYMS_Arena *arena, SYMS_Name2FileIDMapLoose *map,
                                                 SYMS_String8 name_cons,
                                                 SYMS_UnitID uid, SYMS_FileID file_id);


////////////////////////////////
//~ allen: ID Mapping Functions

//- copying id maps
SYMS_API SYMS_IDMap   syms_id_map_copy(SYMS_Arena *arena, SYMS_IDMap *map);

//- lookups into id map
SYMS_API void*        syms_id_map_ptr_from_u64(SYMS_IDMap *map, SYMS_U64 key);

//- constructing id maps
SYMS_API SYMS_IDMap   syms_id_map_alloc(SYMS_Arena *arena, SYMS_U64 bucket_count);
SYMS_API void         syms_id_map_insert(SYMS_Arena *arena, SYMS_IDMap *map, SYMS_U64 key, void *val);


////////////////////////////////
//~ allen: Symbol Name Mapping Structure (String -> Array(SID))

// TODO(allen): copying

//- lookups into symbol name map
SYMS_API SYMS_SymbolIDArray syms_symbol_name_map_array_from_string(SYMS_SymbolNameMap *map, SYMS_String8 string);

//- constructing symbol name maps
SYMS_API void                    syms_symbol_name_map_push(SYMS_Arena *arena, SYMS_SymbolNameMapLoose *map,
                                                           SYMS_String8 name, SYMS_SymbolID sid);
SYMS_API SYMS_SymbolNameMap      syms_symbol_name_map_bake(SYMS_Arena *arena, SYMS_SymbolNameMapLoose *loose);



////////////////////////////////
//~ allen: Line Tables

//- lookups into line tables
SYMS_API SYMS_U64  syms_line_index_from_voff__binary_search(SYMS_Line *lines, SYMS_U64 ender_index, SYMS_U64 voff);
SYMS_API SYMS_Line syms_line_from_sequence_voff(SYMS_LineTable *line_table, SYMS_U64 seq_index, SYMS_U64 voff);

//- copying and rewriting line tables
SYMS_API SYMS_LineTable    syms_line_table_copy(SYMS_Arena *arena, SYMS_LineTable *line_table);
SYMS_API void              syms_line_table_rewrite_file_ids_in_place(SYMS_FileIDArray *file_ids,
                                                                     SYMS_LineTable *line_table_in_out);
SYMS_API SYMS_LineTable    syms_line_table_with_indexes_from_parse(SYMS_Arena *arena, SYMS_LineParseOut *parse);

////////////////////////////////
//~ allen: Line To Addr Map

//- line-to-addr map
SYMS_API SYMS_FileToLineToAddrMap
syms_line_to_addr_map_from_line_table(SYMS_Arena *arena, SYMS_LineTable *table);

//- line-to-addr query
SYMS_API SYMS_LineToAddrMap* syms_line_to_addr_map_lookup_file_id(SYMS_FileToLineToAddrMap *map,
                                                                  SYMS_FileID file_id);

SYMS_API SYMS_U64RangeArray syms_line_to_addr_map_lookup_nearest_line_number(SYMS_LineToAddrMap *map,
                                                                             SYMS_U32 line,
                                                                             SYMS_U32 *actual_line_out);

//- line-to-addr map helpers
SYMS_API void syms_line_to_addr_line_sort(SYMS_FileToLineToAddrLooseLine **array, SYMS_U64 count);
SYMS_API void syms_line_to_addr_line_sort__rec(SYMS_FileToLineToAddrLooseLine **array, SYMS_U64 count);

////////////////////////////////
//~ allen: Copies & Operators for Other Data Structures

SYMS_API SYMS_String8Array syms_string_array_copy(SYMS_Arena *arena, SYMS_StringCons *cons_optional,
                                                  SYMS_String8Array *array);

SYMS_API SYMS_LinkNameRecArray syms_link_name_record_copy(SYMS_Arena *arena, SYMS_LinkNameRecArray *array);

////////////////////////////////
//~ allen: Binary Search Functions

SYMS_API SYMS_U64 syms_index_from_n__u32__binary_search_round_up(SYMS_U32 *v, SYMS_U64 count, SYMS_U32 n);

#endif //SYMS_DATA_STRUCTURES_H
