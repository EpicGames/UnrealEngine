// Copyright Epic Games, Inc. All Rights Reserved.

#ifndef SYMS_DWARF_C
#define SYMS_DWARF_C

////////////////////////////////
//~ rjf: Generated Enums

#include "syms/core/generated/syms_meta_dwarf.c"

////////////////////////////////
//~ rjf: Functions

// TODO(allen): how can we cleanup the dw name strings arrays?

SYMS_API SYMS_String8
syms_dw_name_string_from_section_kind(SYMS_DwSectionKind kind)
{
  static char *strs[] =
  {
    "",
    ".debug_abbrev",
    ".debug_aranges",
    ".debug_frame",
    ".debug_info",
    ".debug_line",
    ".debug_loc",
    ".debug_macinfo",
    ".debug_pubnames",
    ".debug_pubtypes",
    ".debug_ranges",
    ".debug_str",
    ".debug_addr",
    ".debug_loclists",
    ".debug_rnglists",
    ".debug_str_offsets",
    ".debug_line_str",
    ".debug_names"
  };
  SYMS_String8 str =
  {
    (SYMS_U8 *)strs[kind],
    syms_strlen(strs[kind]),
  };
  return str;
}

SYMS_API SYMS_String8
syms_dw_mach_name_string_from_section_kind(SYMS_DwSectionKind kind)
{
  static char *strs[] =
  {
    "",
    "__debug_abbrev",
    "__debug_aranges",
    "__debug_frame",
    "__debug_info",
    "__debug_line",
    "__debug_loc",
    "__debug_macinfo",
    "__debug_pubnames_DWARF",
    "__debug_pubtypes_DWARF",
    "__debug_ranges",
    "__debug_str",
    "__debug_addr",
    "__debug_loclists",
    "__debug_rnglists",
    "__debug_str_offsets",
    "__debug_line_str",
    "__debug_names"
  };
  SYMS_String8 str =
  {
    (SYMS_U8 *)strs[kind],
    syms_strlen(strs[kind]),
  };
  return str;
}

SYMS_API SYMS_String8
syms_dw_dwo_name_string_from_section_kind(SYMS_DwSectionKind kind)
{
  static char *strs[] =
  {
    "",
    ".debug_abbrev.dwo",
    ".debug_aranges.dwo",
    ".debug_frame.dwo",
    ".debug_info.dwo",
    ".debug_line.dwo",
    ".debug_loc.dwo",
    ".debug_macinfo.dwo",
    ".debug_pubnames.dwo",
    ".debug_pubtypes.dwo",
    ".debug_ranges.dwo",
    ".debug_str.dwo",
    ".debug_addr.dwo",
    ".debug_loclists.dwo",
    ".debug_rnglists.dwo",
    ".debug_str_offsets.dwo",
    ".debug_line_str.dwo",
    ".debug_names.dwo"
  };
  SYMS_String8 str =
  {
    (SYMS_U8 *)strs[kind],
    syms_strlen(strs[kind]),
  };
  return str;
}

SYMS_API SYMS_U64
syms_dw_offset_size_from_mode(SYMS_DwMode mode)
{
  SYMS_U64 result = 0;
  switch(mode)
  {
    default: SYMS_INVALID_CODE_PATH; break;
    case SYMS_DwMode_32Bit: { result = 4; break; }
    case SYMS_DwMode_64Bit: { result = 8; break; }
  }
  return result;
}

SYMS_API SYMS_B32
syms_dw_are_attrib_class_and_form_kind_compatible(SYMS_DwAttribClass attrib_class, SYMS_DwFormKind form_kind)
{
  SYMS_B32 result = syms_false;
  
  switch(attrib_class)
  {
    case SYMS_DwAttribClass_ADDRESS:
    {
      result = (form_kind == SYMS_DwFormKind_ADDRX  || form_kind == SYMS_DwFormKind_ADDRX1 || form_kind == SYMS_DwFormKind_ADDRX2 ||
                form_kind == SYMS_DwFormKind_ADDRX3 || form_kind == SYMS_DwFormKind_ADDRX4 || form_kind == SYMS_DwFormKind_ADDR);
    }break;
    
    case SYMS_DwAttribClass_ADDRPTR: case SYMS_DwAttribClass_LOCLIST: case SYMS_DwAttribClass_LINEPTR:
    {
      result = (form_kind == SYMS_DwFormKind_SEC_OFFSET);
    }break;
    
    case SYMS_DwAttribClass_BLOCK:
    {
      result = (form_kind == SYMS_DwFormKind_BLOCK  ||  form_kind == SYMS_DwFormKind_BLOCK1 ||
                form_kind == SYMS_DwFormKind_BLOCK2 ||  form_kind == SYMS_DwFormKind_BLOCK4);
    }break;
    
    case SYMS_DwAttribClass_CONST:
    {
      result = (form_kind == SYMS_DwFormKind_DATA1 || form_kind == SYMS_DwFormKind_DATA2 || form_kind == SYMS_DwFormKind_DATA4 ||
                form_kind == SYMS_DwFormKind_DATA8 || form_kind == SYMS_DwFormKind_SDATA || form_kind == SYMS_DwFormKind_UDATA ||
                form_kind == SYMS_DwFormKind_DATA16);
    }break;
    
    case SYMS_DwAttribClass_EXPRLOC:
    {
      result = (form_kind == SYMS_DwFormKind_EXPRLOC);
    }break;
    
    case SYMS_DwAttribClass_FLAG:
    {
      result = (form_kind == SYMS_DwFormKind_FLAG_PRESENT || form_kind == SYMS_DwFormKind_FLAG);
    }break;
    
    case SYMS_DwAttribClass_LOCLISTPTR:
    {
      result = (form_kind == SYMS_DwFormKind_LOCLISTX);
      result = syms_true; // NOTE(rjf): this did not seem to have a failure case.
    }break;
    
    case SYMS_DwAttribClass_MACPTR:
    {
      result = (form_kind == SYMS_DwFormKind_RNGLISTX || form_kind == SYMS_DwFormKind_SEC_OFFSET);
      result = syms_true; // NOTE(rjf): this did not seem to have a failure case.
    }break;
    
    case SYMS_DwAttribClass_RNGLIST:
    {
      result = (form_kind == SYMS_DwFormKind_RNGLISTX || form_kind == SYMS_DwFormKind_SEC_OFFSET);
      result = syms_true; // NOTE(rjf): this did not seem to have a failure case.
    }break;
    
    case SYMS_DwAttribClass_RNGLISTPTR:
    {
      result = (form_kind == SYMS_DwFormKind_RNGLISTX || form_kind == SYMS_DwFormKind_SEC_OFFSET);
      result = syms_true; // NOTE(rjf): this did not seem to have a failure case.
    }break;
    
    case SYMS_DwAttribClass_REFERENCE:
    {
      result = (form_kind == SYMS_DwFormKind_REF1      || form_kind == SYMS_DwFormKind_REF2     ||
                form_kind == SYMS_DwFormKind_REF4      || form_kind == SYMS_DwFormKind_REF8     ||
                form_kind == SYMS_DwFormKind_REF_UDATA || form_kind == SYMS_DwFormKind_REF_ADDR ||
                form_kind == SYMS_DwFormKind_REF_SUP4  || form_kind == SYMS_DwFormKind_REF_SUP8 ||
                form_kind == SYMS_DwFormKind_REF_SIG8);
    }break;
    
    case SYMS_DwAttribClass_STRING:
    {
      result = (form_kind == SYMS_DwFormKind_STRING   ||
                form_kind == SYMS_DwFormKind_STRX     ||
                form_kind == SYMS_DwFormKind_STRX1    ||
                form_kind == SYMS_DwFormKind_STRX2    ||
                form_kind == SYMS_DwFormKind_STRX3    ||
                form_kind == SYMS_DwFormKind_STRX4    ||
                form_kind == SYMS_DwFormKind_STRP     ||
                form_kind == SYMS_DwFormKind_STRP_SUP ||
                form_kind == SYMS_DwFormKind_LINE_STRP);
    }break;
    
    case SYMS_DwAttribClass_STROFFSETSPTR:
    {
      result = form_kind == SYMS_DwFormKind_SEC_OFFSET;
    }break;
    
    case SYMS_DwAttribClass_UNDEFINED:
    {
      result = syms_true;
    }break;
    
    default:break;
  }
  
  return result;
}

SYMS_API SYMS_TypeKind
syms_dw_type_kind_from_tag_encoding_size(SYMS_DwTagKind tag_kind, SYMS_DwAttribTypeEncoding encoding, SYMS_U64 size)
{
  SYMS_TypeKind result = SYMS_TypeKind_Null;
  switch(tag_kind)
  {
    default: break;
    case SYMS_DwTagKind_NULL:              result = SYMS_TypeKind_Null;      break;
    
    //- rjf: modifiers
    // {
    case SYMS_DwTagKind_CONST_TYPE:
    case SYMS_DwTagKind_PACKED_TYPE:
    case SYMS_DwTagKind_REFERENCE_TYPE:
    case SYMS_DwTagKind_RESTRICT_TYPE:
    case SYMS_DwTagKind_RVALUE_REFERENCE_TYPE:
    case SYMS_DwTagKind_SHARED_TYPE:
    case SYMS_DwTagKind_VOLATILE_TYPE:     result = SYMS_TypeKind_Modifier;  break;
    // }
    
    case SYMS_DwTagKind_ENUMERATION_TYPE:  result = SYMS_TypeKind_Enum;      break;
    case SYMS_DwTagKind_CLASS_TYPE:        result = SYMS_TypeKind_Class;     break;
    case SYMS_DwTagKind_STRUCTURE_TYPE:    result = SYMS_TypeKind_Struct;    break;
    case SYMS_DwTagKind_UNION_TYPE:        result = SYMS_TypeKind_Union;     break;
    case SYMS_DwTagKind_TYPEDEF:           result = SYMS_TypeKind_Typedef;   break;
    case SYMS_DwTagKind_POINTER_TYPE:      result = SYMS_TypeKind_Ptr;       break;
    case SYMS_DwTagKind_PTR_TO_MEMBER_TYPE:result = SYMS_TypeKind_MemberPtr; break;
    case SYMS_DwTagKind_SUBROUTINE_TYPE:   result = SYMS_TypeKind_Proc;      break;
    case SYMS_DwTagKind_ARRAY_TYPE:
    case SYMS_DwTagKind_SUBRANGE_TYPE:     result = SYMS_TypeKind_Array;     break;
    case SYMS_DwTagKind_BASE_TYPE:
    {
      switch (encoding)
      {
        case SYMS_DwAttribTypeEncoding_SIGNED_CHAR:
        case SYMS_DwAttribTypeEncoding_SIGNED:
        {
          switch(size)
          {
            case 1: result = SYMS_TypeKind_Int8;  break;
            case 2: result = SYMS_TypeKind_Int16; break;
            case 4: result = SYMS_TypeKind_Int32; break;
            case 8: result = SYMS_TypeKind_Int64; break;
            default: break;
          }
        }break;
        case SYMS_DwAttribTypeEncoding_UNSIGNED_CHAR:
        case SYMS_DwAttribTypeEncoding_UNSIGNED:
        {
          switch(size)
          {
            case 1: result = SYMS_TypeKind_UInt8;  break;
            case 2: result = SYMS_TypeKind_UInt16; break;
            case 4: result = SYMS_TypeKind_UInt32; break;
            case 8: result = SYMS_TypeKind_UInt64; break;
            default: break;
          }
        }break;
        case SYMS_DwAttribTypeEncoding_FLOAT:
        {
          switch(size)
          {
            case 4: result = SYMS_TypeKind_Float32;  break;
            case 8: result = SYMS_TypeKind_Float64;  break;
            default: break;
          }
        }break;
        case SYMS_DwAttribTypeEncoding_BOOLEAN: result = SYMS_TypeKind_Bool; break;
        
        case SYMS_DwAttribTypeEncoding_COMPLEX_FLOAT:
        case SYMS_DwAttribTypeEncoding_IMAGINARY_FLOAT:
        case SYMS_DwAttribTypeEncoding_PACKED_DECIMAL:
        case SYMS_DwAttribTypeEncoding_NUMERIC_STRING:
        case SYMS_DwAttribTypeEncoding_EDITED:
        case SYMS_DwAttribTypeEncoding_SIGNED_FIXED:
        case SYMS_DwAttribTypeEncoding_UNSIGNED_FIXED:
        case SYMS_DwAttribTypeEncoding_UTF:
        case SYMS_DwAttribTypeEncoding_UCS:
        case SYMS_DwAttribTypeEncoding_ASCII:
        {
          // TODO(nick): types to export
        }break;
        
        default: break;
      }
    }break;
  }
  return result;
}

SYMS_API SYMS_TypeModifiers
syms_dw_type_modifiers_from_tag_kind(SYMS_DwTagKind tag_kind)
{
  SYMS_TypeModifiers mods = 0;
  switch(tag_kind)
  {
    case SYMS_DwTagKind_CONST_TYPE:            mods |= SYMS_TypeModifier_Const; break;
    case SYMS_DwTagKind_PACKED_TYPE:           mods |= SYMS_TypeModifier_Packed; break;
    case SYMS_DwTagKind_REFERENCE_TYPE:        mods |= SYMS_TypeModifier_Reference; break;
    case SYMS_DwTagKind_RESTRICT_TYPE:         mods |= SYMS_TypeModifier_Restrict; break;
    case SYMS_DwTagKind_RVALUE_REFERENCE_TYPE: mods |= SYMS_TypeModifier_RValueReference; break;
    case SYMS_DwTagKind_SHARED_TYPE:           mods |= SYMS_TypeModifier_Shared; break;
    case SYMS_DwTagKind_VOLATILE_TYPE:         mods |= SYMS_TypeModifier_Volatile; break;
    default: break;
  }
  return mods;
}

SYMS_API SYMS_DwAttribClass
syms_dw_attrib_class_from_attrib_kind_extensions(SYMS_DwAttribKind attrib)
{
  SYMS_DwAttribClass result = 0;
  switch (attrib) {
  case SYMS_DwAttribKind_GNU_VECTOR:                     result = SYMS_DwAttribClass_FLAG;      break;
  case SYMS_DwAttribKind_GNU_GUARDED_BY:                 result = SYMS_DwAttribClass_UNDEFINED; break;
  case SYMS_DwAttribKind_GNU_PT_GUARDED_BY:              result = SYMS_DwAttribClass_UNDEFINED; break;
  case SYMS_DwAttribKind_GNU_GUARDED:                    result = SYMS_DwAttribClass_UNDEFINED; break;
  case SYMS_DwAttribKind_GNU_LOCKS_EXCLUDED:             result = SYMS_DwAttribClass_UNDEFINED; break;
  case SYMS_DwAttribKind_GNU_EXCLUSIVE_LOCKS_REQUIRED:   result = SYMS_DwAttribClass_UNDEFINED; break;
  case SYMS_DwAttribKind_GNU_SHARED_LOCKS_REQUIRED:      result = SYMS_DwAttribClass_UNDEFINED; break;
  case SYMS_DwAttribKind_GNU_ODR_SIGNATURE:              result = SYMS_DwAttribClass_UNDEFINED; break;
  case SYMS_DwAttribKind_GNU_TEMPLATE_NAME:              result = SYMS_DwAttribClass_UNDEFINED; break;
  case SYMS_DwAttribKind_GNU_CALL_SITE_VALUE:            result = SYMS_DwAttribClass_EXPRLOC;   break;
  case SYMS_DwAttribKind_GNU_CALL_SITE_DATA_VALUE:       result = SYMS_DwAttribClass_EXPRLOC;   break;
  case SYMS_DwAttribKind_GNU_CALL_SITE_TARGET:           result = SYMS_DwAttribClass_EXPRLOC;   break;
  case SYMS_DwAttribKind_GNU_CALL_SITE_TARGET_CLOBBERED: result = SYMS_DwAttribClass_EXPRLOC;   break;
  case SYMS_DwAttribKind_GNU_TAIL_CALL:                  result = SYMS_DwAttribClass_FLAG;      break;
  case SYMS_DwAttribKind_GNU_ALL_TAIL_CALL_SITES:        result = SYMS_DwAttribClass_FLAG;      break;
  case SYMS_DwAttribKind_GNU_ALL_CALL_SITES:             result = SYMS_DwAttribClass_FLAG;      break;
  case SYMS_DwAttribKind_GNU_ALL_SOURCE_CALL_SITES:      result = SYMS_DwAttribClass_FLAG;      break;
  case SYMS_DwAttribKind_GNU_MACROS:                     result = SYMS_DwAttribClass_FLAG;      break;
  case SYMS_DwAttribKind_GNU_DELETED:                    result = SYMS_DwAttribClass_UNDEFINED; break;
  case SYMS_DwAttribKind_GNU_DWO_NAME:                   result = SYMS_DwAttribClass_STRING;    break;
  case SYMS_DwAttribKind_GNU_DWO_ID:                     result = SYMS_DwAttribClass_CONST;     break;
  case SYMS_DwAttribKind_GNU_RANGES_BASE:                result = SYMS_DwAttribClass_UNDEFINED; break;
  case SYMS_DwAttribKind_GNU_ADDR_BASE:                  result = SYMS_DwAttribClass_ADDRPTR;   break;
  case SYMS_DwAttribKind_GNU_PUBNAMES:                   result = SYMS_DwAttribClass_FLAG;      break;
  case SYMS_DwAttribKind_GNU_PUBTYPES:                   result = SYMS_DwAttribClass_UNDEFINED; break;
  case SYMS_DwAttribKind_GNU_DISCRIMINATOR:              result = SYMS_DwAttribClass_CONST;     break;
  case SYMS_DwAttribKind_GNU_LOCVIEWS:                   result = SYMS_DwAttribClass_UNDEFINED; break;
  case SYMS_DwAttribKind_GNU_ENTRY_VIEW:                 result = SYMS_DwAttribClass_UNDEFINED; break;
  case SYMS_DwAttribKind_VMS_RTNBEG_PD_ADDRESS:          result = SYMS_DwAttribClass_UNDEFINED; break;
  case SYMS_DwAttribKind_USE_GNAT_DESCRIPTIVE_TYPE:      result = SYMS_DwAttribClass_UNDEFINED; break;
  case SYMS_DwAttribKind_GNAT_DESCRIPTIVE_TYPE:          result = SYMS_DwAttribClass_UNDEFINED; break;
  case SYMS_DwAttribKind_GNU_NUMERATOR:                  result = SYMS_DwAttribClass_UNDEFINED; break;
  case SYMS_DwAttribKind_GNU_DENOMINATOR:                result = SYMS_DwAttribClass_UNDEFINED; break;
  case SYMS_DwAttribKind_GNU_BIAS:                       result = SYMS_DwAttribClass_UNDEFINED; break;
  case SYMS_DwAttribKind_UPC_THREADS_SCALED:             result = SYMS_DwAttribClass_UNDEFINED; break;
  case SYMS_DwAttribKind_PGI_LBASE:                      result = SYMS_DwAttribClass_UNDEFINED; break;
  case SYMS_DwAttribKind_PGI_SOFFSET:                    result = SYMS_DwAttribClass_UNDEFINED; break;
  case SYMS_DwAttribKind_PGI_LSTRIDE:                    result = SYMS_DwAttribClass_UNDEFINED; break;
  case SYMS_DwAttribKind_LLVM_INCLUDE_PATH:              result = SYMS_DwAttribClass_STRING;    break;
  case SYMS_DwAttribKind_LLVM_CONFIG_MACROS:             result = SYMS_DwAttribClass_STRING;    break;
  case SYMS_DwAttribKind_LLVM_SYSROOT:                   result = SYMS_DwAttribClass_STRING;    break;
  case SYMS_DwAttribKind_LLVM_API_NOTES:                 result = SYMS_DwAttribClass_STRING;    break;
  case SYMS_DwAttribKind_LLVM_TAG_OFFSET:                result = SYMS_DwAttribClass_UNDEFINED; break;
  case SYMS_DwAttribKind_APPLE_OPTIMIZED:                result = SYMS_DwAttribClass_FLAG;      break;
  case SYMS_DwAttribKind_APPLE_FLAGS:                    result = SYMS_DwAttribClass_FLAG;      break;
  case SYMS_DwAttribKind_APPLE_ISA:                      result = SYMS_DwAttribClass_FLAG;      break;
  case SYMS_DwAttribKind_APPLE_BLOCK:                    result = SYMS_DwAttribClass_UNDEFINED; break;
  case SYMS_DwAttribKind_APPLE_MAJOR_RUNTIME_VERS:       result = SYMS_DwAttribClass_UNDEFINED; break;
  case SYMS_DwAttribKind_APPLE_RUNTIME_CLASS:            result = SYMS_DwAttribClass_UNDEFINED; break;
  case SYMS_DwAttribKind_APPLE_OMIT_FRAME_PTR:           result = SYMS_DwAttribClass_FLAG;      break;
  case SYMS_DwAttribKind_APPLE_PROPERTY_NAME:            result = SYMS_DwAttribClass_UNDEFINED; break;
  case SYMS_DwAttribKind_APPLE_PROPERTY_GETTER:          result = SYMS_DwAttribClass_UNDEFINED; break;
  case SYMS_DwAttribKind_APPLE_PROPERTY_SETTER:          result = SYMS_DwAttribClass_UNDEFINED; break;
  case SYMS_DwAttribKind_APPLE_PROPERTY_ATTRIBUTE:       result = SYMS_DwAttribClass_UNDEFINED; break;
  case SYMS_DwAttribKind_APPLE_OBJC_COMPLETE_TYPE:       result = SYMS_DwAttribClass_UNDEFINED; break;
  case SYMS_DwAttribKind_APPLE_PROPERTY:                 result = SYMS_DwAttribClass_UNDEFINED; break;
  case SYMS_DwAttribKind_APPLE_OBJ_DIRECT:               result = SYMS_DwAttribClass_UNDEFINED; break;
  case SYMS_DwAttribKind_APPLE_SDK:                      result = SYMS_DwAttribClass_STRING;    break;
  }
  return result;
}

SYMS_API SYMS_DwAttribClass
syms_dw_attrib_class_from_attrib_kind_v2(SYMS_DwAttribKind attrib)
{
  SYMS_DwAttribClass result = 0;
  switch (attrib) {
  case SYMS_DwAttribKind_SIBLING:               result = SYMS_DwAttribClass_REFERENCE; break;
  case SYMS_DwAttribKind_LOCATION:              result = SYMS_DwAttribClass_BLOCK|SYMS_DwAttribClass_CONST; break;
  case SYMS_DwAttribKind_NAME:                  result = SYMS_DwAttribClass_STRING; break;
  case SYMS_DwAttribKind_ORDERING:              result = SYMS_DwAttribClass_CONST; break;
  case SYMS_DwAttribKind_BYTE_SIZE:             result = SYMS_DwAttribClass_CONST; break;
  case SYMS_DwAttribKind_BIT_OFFSET:            result = SYMS_DwAttribClass_CONST; break;
  case SYMS_DwAttribKind_BIT_SIZE:              result = SYMS_DwAttribClass_CONST; break;
  case SYMS_DwAttribKind_STMT_LIST:             result = SYMS_DwAttribClass_CONST; break;
  case SYMS_DwAttribKind_LOW_PC:                result = SYMS_DwAttribClass_ADDRESS; break;
  case SYMS_DwAttribKind_HIGH_PC:               result = SYMS_DwAttribClass_ADDRESS; break;
  case SYMS_DwAttribKind_LANGUAGE:              result = SYMS_DwAttribClass_CONST; break;
  case SYMS_DwAttribKind_DISCR:                 result = SYMS_DwAttribClass_REFERENCE; break;
  case SYMS_DwAttribKind_DISCR_VALUE:           result = SYMS_DwAttribClass_CONST; break;
  case SYMS_DwAttribKind_VISIBILITY:            result = SYMS_DwAttribClass_CONST; break;
  case SYMS_DwAttribKind_IMPORT:                result = SYMS_DwAttribClass_REFERENCE; break;
  case SYMS_DwAttribKind_STRING_LENGTH:         result = SYMS_DwAttribClass_BLOCK|SYMS_DwAttribClass_CONST; break;
  case SYMS_DwAttribKind_COMMON_REFERENCE:      result = SYMS_DwAttribClass_REFERENCE; break;
  case SYMS_DwAttribKind_COMP_DIR:              result = SYMS_DwAttribClass_STRING; break;
  case SYMS_DwAttribKind_CONST_VALUE:           result = SYMS_DwAttribClass_STRING|SYMS_DwAttribClass_CONST|SYMS_DwAttribClass_BLOCK; break;
  case SYMS_DwAttribKind_CONTAINING_TYPE:       result = SYMS_DwAttribClass_REFERENCE; break;
  case SYMS_DwAttribKind_DEFAULT_VALUE:         result = SYMS_DwAttribClass_REFERENCE; break;
  case SYMS_DwAttribKind_INLINE:                result = SYMS_DwAttribClass_CONST; break;
  case SYMS_DwAttribKind_IS_OPTIONAL:           result = SYMS_DwAttribClass_FLAG; break;
  case SYMS_DwAttribKind_LOWER_BOUND:           result = SYMS_DwAttribClass_CONST|SYMS_DwAttribClass_REFERENCE; break;
  case SYMS_DwAttribKind_PRODUCER:              result = SYMS_DwAttribClass_STRING; break;
  case SYMS_DwAttribKind_PROTOTYPED:            result = SYMS_DwAttribClass_FLAG; break;
  case SYMS_DwAttribKind_RETURN_ADDR:           result = SYMS_DwAttribClass_BLOCK|SYMS_DwAttribClass_CONST; break;
  case SYMS_DwAttribKind_START_SCOPE:           result = SYMS_DwAttribClass_CONST; break;
  case SYMS_DwAttribKind_BIT_STRIDE:            result = SYMS_DwAttribClass_CONST; break;
  case SYMS_DwAttribKind_UPPER_BOUND:           result = SYMS_DwAttribClass_CONST|SYMS_DwAttribClass_REFERENCE; break;
  case SYMS_DwAttribKind_ABSTRACT_ORIGIN:       result = SYMS_DwAttribClass_REFERENCE; break;
  case SYMS_DwAttribKind_ACCESSIBILITY:         result = SYMS_DwAttribClass_CONST; break;
  case SYMS_DwAttribKind_ADDRESS_CLASS:         result = SYMS_DwAttribClass_CONST; break;
  case SYMS_DwAttribKind_ARTIFICIAL:            result = SYMS_DwAttribClass_FLAG; break;
  case SYMS_DwAttribKind_BASE_TYPES:            result = SYMS_DwAttribClass_REFERENCE; break;
  case SYMS_DwAttribKind_CALLING_CONVENTION:    result = SYMS_DwAttribClass_CONST; break;
  case SYMS_DwAttribKind_COUNT:                 result = SYMS_DwAttribClass_CONST|SYMS_DwAttribClass_REFERENCE; break;
  case SYMS_DwAttribKind_DATA_MEMBER_LOCATION:  result = SYMS_DwAttribClass_BLOCK|SYMS_DwAttribClass_REFERENCE; break;
  case SYMS_DwAttribKind_DECL_COLUMN:           result = SYMS_DwAttribClass_CONST; break;
  case SYMS_DwAttribKind_DECL_FILE:             result = SYMS_DwAttribClass_CONST; break;
  case SYMS_DwAttribKind_DECL_LINE:             result = SYMS_DwAttribClass_CONST; break;
  case SYMS_DwAttribKind_DECLARATION:           result = SYMS_DwAttribClass_FLAG; break;
  case SYMS_DwAttribKind_DISCR_LIST:            result = SYMS_DwAttribClass_BLOCK; break;
  case SYMS_DwAttribKind_ENCODING:              result = SYMS_DwAttribClass_CONST; break;
  case SYMS_DwAttribKind_EXTERNAL:              result = SYMS_DwAttribClass_FLAG; break;
  case SYMS_DwAttribKind_FRAME_BASE:            result = SYMS_DwAttribClass_BLOCK|SYMS_DwAttribClass_CONST; break;
  case SYMS_DwAttribKind_FRIEND:                result = SYMS_DwAttribClass_REFERENCE; break;
  case SYMS_DwAttribKind_IDENTIFIER_CASE:       result = SYMS_DwAttribClass_CONST; break;
  case SYMS_DwAttribKind_MACRO_INFO:            result = SYMS_DwAttribClass_CONST; break;
  case SYMS_DwAttribKind_NAMELIST_ITEM:         result = SYMS_DwAttribClass_BLOCK; break;
  case SYMS_DwAttribKind_PRIORITY:              result = SYMS_DwAttribClass_REFERENCE; break;
  case SYMS_DwAttribKind_SEGMENT:               result = SYMS_DwAttribClass_BLOCK|SYMS_DwAttribClass_CONST; break;
  case SYMS_DwAttribKind_SPECIFICATION:         result = SYMS_DwAttribClass_REFERENCE; break;
  case SYMS_DwAttribKind_STATIC_LINK:           result = SYMS_DwAttribClass_BLOCK|SYMS_DwAttribClass_CONST; break;
  case SYMS_DwAttribKind_TYPE:                  result = SYMS_DwAttribClass_REFERENCE; break;
  case SYMS_DwAttribKind_USE_LOCATION:          result = SYMS_DwAttribClass_BLOCK|SYMS_DwAttribClass_CONST; break;
  case SYMS_DwAttribKind_VARIABLE_PARAMETER:    result = SYMS_DwAttribClass_FLAG; break;
  case SYMS_DwAttribKind_VIRTUALITY:            result = SYMS_DwAttribClass_CONST; break;
  case SYMS_DwAttribKind_VTABLE_ELEM_LOCATION:  result = SYMS_DwAttribClass_BLOCK|SYMS_DwAttribClass_REFERENCE; break;
  default:                                      result = syms_dw_attrib_class_from_attrib_kind_extensions(attrib); break;
  }
  return result;
}

SYMS_API SYMS_DwAttribClass
syms_dw_attrib_class_from_form_kind_v2(SYMS_DwFormKind form)
{
  SYMS_DwAttribClass result = 0;
  switch (form) {
  case SYMS_DwFormKind_ADDR:      result = SYMS_DwAttribClass_ADDRESS; break;
  case SYMS_DwFormKind_BLOCK2:    result = SYMS_DwAttribClass_BLOCK; break;
  case SYMS_DwFormKind_BLOCK4:    result = SYMS_DwAttribClass_BLOCK; break;
  case SYMS_DwFormKind_DATA2:     result = SYMS_DwAttribClass_CONST; break;
  case SYMS_DwFormKind_DATA4:     result = SYMS_DwAttribClass_CONST; break;
  case SYMS_DwFormKind_DATA8:     result = SYMS_DwAttribClass_CONST; break;
  case SYMS_DwFormKind_STRING:    result = SYMS_DwAttribClass_STRING; break;
  case SYMS_DwFormKind_BLOCK:     result = SYMS_DwAttribClass_BLOCK; break;
  case SYMS_DwFormKind_BLOCK1:    result = SYMS_DwAttribClass_BLOCK; break;
  case SYMS_DwFormKind_DATA1:     result = SYMS_DwAttribClass_CONST; break;
  case SYMS_DwFormKind_FLAG:      result = SYMS_DwAttribClass_FLAG; break;
  case SYMS_DwFormKind_SDATA:     result = SYMS_DwAttribClass_CONST; break;
  case SYMS_DwFormKind_STRP:      result = SYMS_DwAttribClass_STRING; break;
  case SYMS_DwFormKind_UDATA:     result = SYMS_DwAttribClass_CONST; break;
  case SYMS_DwFormKind_REF_ADDR:  result = SYMS_DwAttribClass_REFERENCE; break;
  case SYMS_DwFormKind_REF1:      result = SYMS_DwAttribClass_REFERENCE; break;
  case SYMS_DwFormKind_REF2:      result = SYMS_DwAttribClass_REFERENCE; break;
  case SYMS_DwFormKind_REF4:      result = SYMS_DwAttribClass_REFERENCE; break;
  case SYMS_DwFormKind_REF8:      result = SYMS_DwAttribClass_REFERENCE; break;
  case SYMS_DwFormKind_REF_UDATA: result = SYMS_DwAttribClass_REFERENCE; break;
  case SYMS_DwFormKind_INDIRECT:  result = SYMS_DwAttribClass_UNDEFINED;
  }
  return result;
}

#endif // SYMS_DWARF_C
