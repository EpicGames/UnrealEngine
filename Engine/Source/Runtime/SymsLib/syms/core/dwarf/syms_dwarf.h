// Copyright Epic Games, Inc. All Rights Reserved.
/* date = May 11th 2021 4:56 pm */

#ifndef SYMS_DWARF_H
#define SYMS_DWARF_H

////////////////////////////////
//~ rjf: Generated Enums

#include "syms/core/generated/syms_meta_dwarf.h"

////////////////////////////////
//~ rjf: Types

typedef enum SYMS_DwCompUnitKind
{
  SYMS_DwCompUnitKind_RESERVED      = 0x00,
  SYMS_DwCompUnitKind_COMPILE       = 0x01, 
  SYMS_DwCompUnitKind_TYPE          = 0x02,
  SYMS_DwCompUnitKind_PARTIAL       = 0x03, 
  SYMS_DwCompUnitKind_SKELETON      = 0x04,
  SYMS_DwCompUnitKind_SPLIT_COMPILE = 0x05, 
  SYMS_DwCompUnitKind_SPLIT_TYPE    = 0x06,
  SYMS_DwCompUnitKind_LO_USER       = 0x80, 
  SYMS_DwCompUnitKind_HI_USER       = 0xff
}
SYMS_DwCompUnitKind;

typedef enum SYMS_DwLNCT
{
  SYMS_DwLNCT_PATH            = 0x1,
  SYMS_DwLNCT_DIRECTORY_INDEX = 0x2,
  SYMS_DwLNCT_TIMESTAMP       = 0x3,
  SYMS_DwLNCT_SIZE            = 0x4,
  SYMS_DwLNCT_MD5             = 0x5,
  SYMS_DwLNCT_USER_LO         = 0x2000,
  SYMS_DwLNCT_USER_HI         = 0x3fff
}
SYMS_DwLNCT;

////////////////////////////////
//~ rjf: DWARF Format Functions

SYMS_C_LINKAGE_BEGIN

SYMS_API SYMS_String8  syms_dw_name_string_from_section_kind(SYMS_DwSectionKind kind);
SYMS_API SYMS_String8  syms_dw_mach_name_string_from_section_kind(SYMS_DwSectionKind kind);

SYMS_API SYMS_String8  syms_dw_dwo_name_string_from_section_kind(SYMS_DwSectionKind kind);
SYMS_API SYMS_U64      syms_dw_offset_size_from_mode(SYMS_DwMode mode);
SYMS_API SYMS_B32      syms_dw_are_attrib_class_and_form_kind_compatible(SYMS_DwAttribClass attrib_class, SYMS_DwFormKind form_kind);
SYMS_API SYMS_TypeKind syms_dw_type_kind_from_tag_encoding_size(SYMS_DwTagKind tag_kind, SYMS_DwAttribTypeEncoding encoding, SYMS_U64 size);
SYMS_API SYMS_TypeModifiers syms_dw_type_modifiers_from_tag_kind(SYMS_DwTagKind tag_kind);

SYMS_API SYMS_DwAttribClass syms_dw_attrib_class_from_attrib_kind_extensions(SYMS_DwAttribKind attrib);
SYMS_API SYMS_DwAttribClass syms_dw_attrib_class_from_attrib_kind_v2(SYMS_DwAttribKind attrib);
SYMS_API SYMS_DwAttribClass syms_dw_attrib_class_from_form_kind_v2(SYMS_DwFormKind form);

SYMS_C_LINKAGE_END

#endif // SYMS_DWARF_H
