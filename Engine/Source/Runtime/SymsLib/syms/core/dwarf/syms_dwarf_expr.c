// Copyright Epic Games, Inc. All Rights Reserved.

#if !defined(SYMS_DWARF_EXPR_C)
#define SYMS_DWARF_EXPR_C

////////////////////////////////
//~ NOTE(allen): Include Generated Functions/Tables

#include "syms/core/generated/syms_meta_dwarf_expr.c"

////////////////////////////////
//~ NOTE(allen): Dwarf Expression Analysis & Eval Functions

//- analyzers

SYMS_API SYMS_DwSimpleLoc
syms_dw_expr__analyze_fast(void *base, SYMS_U64Range range, SYMS_U64 text_section_base){
  SYMS_DwSimpleLoc result = {SYMS_DwSimpleLocKind_Empty};
  
  SYMS_U8 op = 0;
  if (syms_based_range_read(base, range, 0, 1, &op)){
    
    // step params
    SYMS_U64 size_param = 0;
    SYMS_B8  is_signed = 0;
    
    // step
    SYMS_U64 step_cursor = 1;
    switch (op){
      
      //// literal encodings ////
      
      case SYMS_DwOp_LIT0:  case SYMS_DwOp_LIT1:  case SYMS_DwOp_LIT2:
      case SYMS_DwOp_LIT3:  case SYMS_DwOp_LIT4:  case SYMS_DwOp_LIT5:
      case SYMS_DwOp_LIT6:  case SYMS_DwOp_LIT7:  case SYMS_DwOp_LIT8:
      case SYMS_DwOp_LIT9:  case SYMS_DwOp_LIT10: case SYMS_DwOp_LIT11:
      case SYMS_DwOp_LIT12: case SYMS_DwOp_LIT13: case SYMS_DwOp_LIT14:
      case SYMS_DwOp_LIT15: case SYMS_DwOp_LIT16: case SYMS_DwOp_LIT17:
      case SYMS_DwOp_LIT18: case SYMS_DwOp_LIT19: case SYMS_DwOp_LIT20:
      case SYMS_DwOp_LIT21: case SYMS_DwOp_LIT22: case SYMS_DwOp_LIT23:
      case SYMS_DwOp_LIT24: case SYMS_DwOp_LIT25: case SYMS_DwOp_LIT26:
      case SYMS_DwOp_LIT27: case SYMS_DwOp_LIT28: case SYMS_DwOp_LIT29:
      case SYMS_DwOp_LIT30: case SYMS_DwOp_LIT31:
      {
        SYMS_U64 x = op - SYMS_DwOp_LIT0;
        result.kind = SYMS_DwSimpleLocKind_Address;
        result.addr = x;
      }break;
      
      case SYMS_DwOp_CONST1U:size_param = 1; goto const_n;
      case SYMS_DwOp_CONST2U:size_param = 2; goto const_n;
      case SYMS_DwOp_CONST4U:size_param = 4; goto const_n;
      case SYMS_DwOp_CONST8U:size_param = 8; goto const_n;
      case SYMS_DwOp_CONST1S:size_param = 1; is_signed = 1; goto const_n;
      case SYMS_DwOp_CONST2S:size_param = 2; is_signed = 1; goto const_n;
      case SYMS_DwOp_CONST4S:size_param = 4; is_signed = 1; goto const_n;
      case SYMS_DwOp_CONST8S:size_param = 8; is_signed = 1; goto const_n;
      const_n:
      {
        SYMS_U64 x = 0;
        step_cursor += syms_based_range_read(base, range, step_cursor, size_param, &x);
        // sign extend
        if (is_signed){
          SYMS_U64 bit_shift = (size_param << 3) - 1;
          if ((x >> bit_shift) != 0){
            x |= ~((1 << bit_shift) - 1);
          }
        }
        result.kind = SYMS_DwSimpleLocKind_Address;
        result.addr = x;
      }break;
      
      case SYMS_DwOp_ADDR:
      {
        SYMS_U64 offset = 0;
        step_cursor += syms_based_range_read(base, range, step_cursor, 8, &offset);
        SYMS_U64 x = text_section_base + offset;
        result.kind = SYMS_DwSimpleLocKind_Address;
        result.addr = x;
      }break;
      
      case SYMS_DwOp_CONSTU:
      {
        SYMS_U64 x = 0;
        step_cursor += syms_based_range_read_uleb128(base, range, step_cursor, &x);
        result.kind = SYMS_DwSimpleLocKind_Address;
        result.addr = x;
      }break;
      
      case SYMS_DwOp_CONSTS:
      {
        SYMS_U64 x = 0;
        step_cursor += syms_based_range_read_sleb128(base, range, step_cursor, (SYMS_S64*)&x);
        result.kind = SYMS_DwSimpleLocKind_Address;
        result.addr = x;
      }break;
      
      
      //// register location descriptions ////
      
      case SYMS_DwOp_REG0:  case SYMS_DwOp_REG1:  case SYMS_DwOp_REG2:
      case SYMS_DwOp_REG3:  case SYMS_DwOp_REG4:  case SYMS_DwOp_REG5:
      case SYMS_DwOp_REG6:  case SYMS_DwOp_REG7:  case SYMS_DwOp_REG8:
      case SYMS_DwOp_REG9:  case SYMS_DwOp_REG10: case SYMS_DwOp_REG11:
      case SYMS_DwOp_REG12: case SYMS_DwOp_REG13: case SYMS_DwOp_REG14:
      case SYMS_DwOp_REG15: case SYMS_DwOp_REG16: case SYMS_DwOp_REG17:
      case SYMS_DwOp_REG18: case SYMS_DwOp_REG19: case SYMS_DwOp_REG20:
      case SYMS_DwOp_REG21: case SYMS_DwOp_REG22: case SYMS_DwOp_REG23:
      case SYMS_DwOp_REG24: case SYMS_DwOp_REG25: case SYMS_DwOp_REG26:
      case SYMS_DwOp_REG27: case SYMS_DwOp_REG28: case SYMS_DwOp_REG29:
      case SYMS_DwOp_REG30: case SYMS_DwOp_REG31:
      {
        SYMS_U64 reg_idx = op - SYMS_DwOp_REG0;
        result.kind = SYMS_DwSimpleLocKind_Register;
        result.reg_idx = reg_idx;
      }break;
      
      case SYMS_DwOp_REGX:
      {
        SYMS_U64 reg_idx = 0;
        step_cursor += syms_based_range_read_uleb128(base, range, step_cursor, &reg_idx);
        result.kind = SYMS_DwSimpleLocKind_Register;
        result.reg_idx = reg_idx;
      }break;
      
      
      //// implicit location descriptions ////
      
      case SYMS_DwOp_IMPLICIT_VALUE:
      {
        SYMS_U64 size = 0;
        step_cursor += syms_based_range_read_uleb128(base, range, step_cursor, &size);
        if (step_cursor + size <= range.max){
          void *data = (SYMS_U8*)base + range.min + step_cursor;
          result.kind = SYMS_DwSimpleLocKind_ValueLong;
          result.val_long.str  = (SYMS_U8*)data;
          result.val_long.size = size;
        }
        step_cursor += size;
      }break;
      
      case SYMS_DwOp_STACK_VALUE:
      {
        // NOTE(allen): This op pops from the value stack, so if it comes
        // first the dwarf expression is bad.
        result.kind = SYMS_DwSimpleLocKind_Fail;
        result.fail_kind = SYMS_DwLocFailKind_BadData;
      }break;
      
      
      //// composite location descriptions ////
      
      // NOTE(allen): if the first and only op is a piece, the expression is empty
      
      case SYMS_DwOp_PIECE:
      {
        SYMS_U64 size = 0;
        step_cursor += syms_based_range_read_uleb128(base, range, step_cursor, &size);
        result.kind = SYMS_DwSimpleLocKind_Empty;
      }break;
      
      case SYMS_DwOp_BIT_PIECE:
      {
        SYMS_U64 bit_size = 0;
        step_cursor += syms_based_range_read_uleb128(base, range, step_cursor, &bit_size);
        SYMS_U64 bit_off = 0;
        step_cursor += syms_based_range_read_uleb128(base, range, step_cursor, &bit_off);
        result.kind = SYMS_DwSimpleLocKind_Empty;
      }break;
      
      
      //// final fallback ////
      
      default:
      {
        result.kind = SYMS_DwSimpleLocKind_Fail;
        result.fail_kind = SYMS_DwLocFailKind_TooComplicated;
      }break;
    }
    
    // check this was the whole expression
    if (range.min + step_cursor < range.max){
      result.kind = SYMS_DwSimpleLocKind_Fail;
      result.fail_kind = SYMS_DwLocFailKind_TooComplicated;
    }
  }
  
  return(result);
}

SYMS_API SYMS_DwExprAnalysis
syms_dw_expr__analyze_details(void *in_base, SYMS_U64Range in_range, SYMS_DwExprMachineCallConfig *call_config){
  SYMS_DwExprAnalysis result = {0};
  
  // are we resolving calls?
  SYMS_B32 has_call_func = (call_config != 0 && call_config->func != 0);
  
  // tasks
  SYMS_ArenaTemp scratch = syms_get_scratch(0, 0);
  SYMS_DwExprAnalysisTask *unfinished_tasks = 0;
  SYMS_DwExprAnalysisTask *finished_tasks = 0;
  
  // convert range input to string
  SYMS_String8 in_data = syms_str8((SYMS_U8*)in_base + in_range.min, in_range.max - in_range.min);
  
  // put input task onto the list
  {
    SYMS_DwExprAnalysisTask *new_task = syms_push_array(scratch.arena, SYMS_DwExprAnalysisTask, 1);
    new_task->p = SYMS_U64_MAX;
    new_task->data = in_data;
    SYMS_StackPush(unfinished_tasks, new_task);
  }
  
  // state for checking implicit locations
  SYMS_B32 last_was_implicit_loc = syms_false;
  
  // task loop
  for (;;){
    // get next task to handle
    SYMS_DwExprAnalysisTask *task = unfinished_tasks;
    if (task == 0){
      break;
    }
    
    SYMS_String8 task_data = task->data;
    SYMS_U8 *task_base = task_data.str;
    SYMS_U64Range task_range = {0, task_data.size};
    
    // move the task to finished now
    SYMS_StackPop(unfinished_tasks);
    SYMS_StackPush(finished_tasks, task);
    
    // analysis loop
    SYMS_U64 cursor = 0;
    for (;;){
      
      // decode op
      SYMS_U64 op_offset = cursor;
      SYMS_U8 op = 0;
      if (syms_based_range_read(task_base, task_range, op_offset, 1, &op)){
        SYMS_U64 after_op_off = cursor + 1;
        
        // require piece op after 'implicit' location descriptions
        if (last_was_implicit_loc){
          if (op != SYMS_DwOp_PIECE && op != SYMS_DwOp_BIT_PIECE){
            result.flags |= SYMS_DwExprFlag_BadData;
            goto finish;
          }
        }
        
        // step params
        SYMS_U64 size_param = 0;
        SYMS_B8  is_signed = 0;
        
        // step
        SYMS_U64 step_cursor = after_op_off;
        switch (op){
          
          //// literal encodings ////
          
          case SYMS_DwOp_LIT0:  case SYMS_DwOp_LIT1:  case SYMS_DwOp_LIT2:
          case SYMS_DwOp_LIT3:  case SYMS_DwOp_LIT4:  case SYMS_DwOp_LIT5:
          case SYMS_DwOp_LIT6:  case SYMS_DwOp_LIT7:  case SYMS_DwOp_LIT8:
          case SYMS_DwOp_LIT9:  case SYMS_DwOp_LIT10: case SYMS_DwOp_LIT11:
          case SYMS_DwOp_LIT12: case SYMS_DwOp_LIT13: case SYMS_DwOp_LIT14:
          case SYMS_DwOp_LIT15: case SYMS_DwOp_LIT16: case SYMS_DwOp_LIT17:
          case SYMS_DwOp_LIT18: case SYMS_DwOp_LIT19: case SYMS_DwOp_LIT20:
          case SYMS_DwOp_LIT21: case SYMS_DwOp_LIT22: case SYMS_DwOp_LIT23:
          case SYMS_DwOp_LIT24: case SYMS_DwOp_LIT25: case SYMS_DwOp_LIT26:
          case SYMS_DwOp_LIT27: case SYMS_DwOp_LIT28: case SYMS_DwOp_LIT29:
          case SYMS_DwOp_LIT30: case SYMS_DwOp_LIT31:
          break;
          
          case SYMS_DwOp_CONST1U:size_param = 1; goto const_n;
          case SYMS_DwOp_CONST2U:size_param = 2; goto const_n;
          case SYMS_DwOp_CONST4U:size_param = 4; goto const_n;
          case SYMS_DwOp_CONST8U:size_param = 8; goto const_n;
          case SYMS_DwOp_CONST1S:size_param = 1; is_signed = 1; goto const_n;
          case SYMS_DwOp_CONST2S:size_param = 2; is_signed = 1; goto const_n;
          case SYMS_DwOp_CONST4S:size_param = 4; is_signed = 1; goto const_n;
          case SYMS_DwOp_CONST8S:size_param = 8; is_signed = 1; goto const_n;
          const_n:
          {
            SYMS_U64 x = 0;
            step_cursor += syms_based_range_read(task_base, task_range, step_cursor, size_param, &x);
          }break;
          
          case SYMS_DwOp_ADDR:
          {
            SYMS_U64 offset = 0;
            step_cursor += syms_based_range_read(task_base, task_range, step_cursor, 8, &offset);
            result.flags |= SYMS_DwExprFlag_UsesTextBase;
          }break;
          
          case SYMS_DwOp_CONSTU:
          {
            SYMS_U64 x = 0;
            step_cursor += syms_based_range_read_uleb128(task_base, task_range, step_cursor, &x);
          }break;
          
          case SYMS_DwOp_CONSTS:
          {
            SYMS_U64 x = 0;
            step_cursor += syms_based_range_read_sleb128(task_base, task_range, step_cursor, (SYMS_S64*)&x);
          }break;
          
          
          //// register based addressing ////
          
          case SYMS_DwOp_FBREG:
          {
            SYMS_S64 offset = 0;
            step_cursor += syms_based_range_read_sleb128(task_base, task_range, step_cursor, &offset);
            result.flags |= SYMS_DwExprFlag_UsesFrameBase;
          }break;
          
          case SYMS_DwOp_BREG0:  case SYMS_DwOp_BREG1:  case SYMS_DwOp_BREG2:
          case SYMS_DwOp_BREG3:  case SYMS_DwOp_BREG4:  case SYMS_DwOp_BREG5:
          case SYMS_DwOp_BREG6:  case SYMS_DwOp_BREG7:  case SYMS_DwOp_BREG8:
          case SYMS_DwOp_BREG9:  case SYMS_DwOp_BREG10: case SYMS_DwOp_BREG11:
          case SYMS_DwOp_BREG12: case SYMS_DwOp_BREG13: case SYMS_DwOp_BREG14:
          case SYMS_DwOp_BREG15: case SYMS_DwOp_BREG16: case SYMS_DwOp_BREG17:
          case SYMS_DwOp_BREG18: case SYMS_DwOp_BREG19: case SYMS_DwOp_BREG20:
          case SYMS_DwOp_BREG21: case SYMS_DwOp_BREG22: case SYMS_DwOp_BREG23:
          case SYMS_DwOp_BREG24: case SYMS_DwOp_BREG25: case SYMS_DwOp_BREG26:
          case SYMS_DwOp_BREG27: case SYMS_DwOp_BREG28: case SYMS_DwOp_BREG29:
          case SYMS_DwOp_BREG30: case SYMS_DwOp_BREG31:
          {
            SYMS_S64 offset = 0;
            step_cursor += syms_based_range_read_sleb128(task_base, task_range, step_cursor, &offset);
            result.flags |= SYMS_DwExprFlag_UsesRegisters;
          }break;
          
          case SYMS_DwOp_BREGX:
          {
            SYMS_U64 reg_idx = 0;
            step_cursor += syms_based_range_read_uleb128(task_base, task_range, step_cursor, &reg_idx);
            SYMS_S64 offset = 0;
            step_cursor += syms_based_range_read_sleb128(task_base, task_range, step_cursor, &offset);
            result.flags |= SYMS_DwExprFlag_UsesRegisters;
          }break;
          
          
          //// stack operations ////
          
          case SYMS_DwOp_DUP:
          case SYMS_DwOp_DROP:
          break;
          
          case SYMS_DwOp_PICK:
          {
            SYMS_U64 idx = 0;
            step_cursor += syms_based_range_read(task_base, task_range, step_cursor, 1, &idx);
          }break;
          
          case SYMS_DwOp_OVER:
          case SYMS_DwOp_SWAP:
          case SYMS_DwOp_ROT:
          break;
          
          case SYMS_DwOp_DEREF:
          {
            result.flags |= SYMS_DwExprFlag_UsesMemory;
          }break;
          
          case SYMS_DwOp_DEREF_SIZE:
          {
            SYMS_U64 size = 0;
            step_cursor += syms_based_range_read(task_base, task_range, step_cursor, 1, &size);
            result.flags |= SYMS_DwExprFlag_UsesMemory;
          }break;
          
          case SYMS_DwOp_XDEREF:
          case SYMS_DwOp_XDEREF_SIZE:
          {
            result.flags |= SYMS_DwExprFlag_NotSupported;
            goto finish;
          }break;
          
          case SYMS_DwOp_PUSH_OBJECT_ADDRESS:
          {
            result.flags |= SYMS_DwExprFlag_UsesObjectAddress;
          }break;
          
          case SYMS_DwOp_GNU_PUSH_TLS_ADDRESS:
          case SYMS_DwOp_FORM_TLS_ADDRESS:
          {
            result.flags |= SYMS_DwExprFlag_UsesTLSAddress;
          }break;
          
          case SYMS_DwOp_CALL_FRAME_CFA:
          {
            result.flags |= SYMS_DwExprFlag_UsesCFA;
          }break;
          
          
          //// arithmetic and logical operations ////
          
          case SYMS_DwOp_ABS:
          case SYMS_DwOp_AND:
          case SYMS_DwOp_DIV:
          case SYMS_DwOp_MINUS:
          case SYMS_DwOp_MOD:
          case SYMS_DwOp_MUL:
          case SYMS_DwOp_NEG:
          case SYMS_DwOp_NOT:
          case SYMS_DwOp_OR:
          case SYMS_DwOp_PLUS:
          break;
          
          case SYMS_DwOp_PLUS_UCONST:
          {
            SYMS_U64 y = 0;
            step_cursor += syms_based_range_read_uleb128(task_base, task_range, step_cursor, &y);
          }break;
          
          case SYMS_DwOp_SHL:
          case SYMS_DwOp_SHR:
          case SYMS_DwOp_SHRA:
          case SYMS_DwOp_XOR:
          break;
          
          
          //// control flow operations ////
          
          case SYMS_DwOp_LE:
          case SYMS_DwOp_GE:
          case SYMS_DwOp_EQ:
          case SYMS_DwOp_LT:
          case SYMS_DwOp_GT:
          case SYMS_DwOp_NE:
          break;
          
          case SYMS_DwOp_SKIP:
          case SYMS_DwOp_BRA:
          {
            SYMS_S16 d = 0;
            step_cursor += syms_based_range_read(task_base, task_range, step_cursor, 2, &d);
            result.flags |= SYMS_DwExprFlag_NonLinearFlow;
          }break;
          
          case SYMS_DwOp_CALL2:size_param = 2; goto callN;
          case SYMS_DwOp_CALL4:size_param = 4; goto callN;
          callN:
          {
            SYMS_U64 p = 0;
            step_cursor += syms_based_range_read(task_base, task_range, step_cursor, size_param, &p);
            result.flags |= SYMS_DwExprFlag_UsesCallResolution|SYMS_DwExprFlag_NonLinearFlow;
            
            // add to task list
            if (has_call_func){
              SYMS_DwExprAnalysisTask *existing = syms_dw_expr__analysis_task_from_p(unfinished_tasks, p);
              if (existing == 0){
                existing = syms_dw_expr__analysis_task_from_p(finished_tasks, p);;
              }
              if (existing == 0){
                SYMS_DwExprAnalysisTask *new_task = syms_push_array(scratch.arena, SYMS_DwExprAnalysisTask, 1);
                new_task->p = p;
                new_task->data = call_config->func(call_config->user_ptr, p);
                SYMS_StackPush(unfinished_tasks, new_task);
              }
            }
          }break;
          
          case SYMS_DwOp_CALL_REF:
          {
            result.flags |= SYMS_DwExprFlag_NotSupported;
            goto finish;
          }break;
          
          
          //// special operations ////
          
          case SYMS_DwOp_NOP:break;
          
          
          //// register location descriptions ////
          
          case SYMS_DwOp_REG0:  case SYMS_DwOp_REG1:  case SYMS_DwOp_REG2:
          case SYMS_DwOp_REG3:  case SYMS_DwOp_REG4:  case SYMS_DwOp_REG5:
          case SYMS_DwOp_REG6:  case SYMS_DwOp_REG7:  case SYMS_DwOp_REG8:
          case SYMS_DwOp_REG9:  case SYMS_DwOp_REG10: case SYMS_DwOp_REG11:
          case SYMS_DwOp_REG12: case SYMS_DwOp_REG13: case SYMS_DwOp_REG14:
          case SYMS_DwOp_REG15: case SYMS_DwOp_REG16: case SYMS_DwOp_REG17:
          case SYMS_DwOp_REG18: case SYMS_DwOp_REG19: case SYMS_DwOp_REG20:
          case SYMS_DwOp_REG21: case SYMS_DwOp_REG22: case SYMS_DwOp_REG23:
          case SYMS_DwOp_REG24: case SYMS_DwOp_REG25: case SYMS_DwOp_REG26:
          case SYMS_DwOp_REG27: case SYMS_DwOp_REG28: case SYMS_DwOp_REG29:
          case SYMS_DwOp_REG30: case SYMS_DwOp_REG31:
          {
            last_was_implicit_loc = syms_true;
          }break;
          
          case SYMS_DwOp_REGX:
          {
            SYMS_U64 reg_idx = 0;
            step_cursor += syms_based_range_read(task_base, task_range, step_cursor, size_param, &reg_idx);
            last_was_implicit_loc = syms_true;
          }break;
          
          
          //// implicit location descriptions ////
          
          case SYMS_DwOp_IMPLICIT_VALUE:
          {
            SYMS_U64 size = 0;
            step_cursor += syms_based_range_read(task_base, task_range, step_cursor, size_param, &size);
            if (step_cursor + size > task_range.max){
              result.flags |= SYMS_DwExprFlag_BadData;
              goto finish;
            }
            step_cursor += size;
            last_was_implicit_loc = syms_true;
          }break;
          
          case SYMS_DwOp_STACK_VALUE:
          {
            last_was_implicit_loc = syms_true;
          }break;
          
          
          //// composite location descriptions ////
          
          case SYMS_DwOp_PIECE:
          {
            SYMS_U64 size = 0;
            step_cursor += syms_based_range_read_uleb128(task_base, task_range, step_cursor, &size);
            result.flags |= SYMS_DwExprFlag_UsesComposite;
            
            last_was_implicit_loc = syms_false;
          }break;
          
          case SYMS_DwOp_BIT_PIECE:
          {
            SYMS_U64 bit_size = 0;
            step_cursor += syms_based_range_read_uleb128(task_base, task_range, step_cursor, &bit_size);
            SYMS_U64 bit_off = 0;
            step_cursor += syms_based_range_read_uleb128(task_base, task_range, step_cursor, &bit_off);
            result.flags |= SYMS_DwExprFlag_UsesComposite;
            
            last_was_implicit_loc = syms_false;
          }break;
          
          
          //// final fallback ////
          
          default:
          {
            result.flags |= SYMS_DwExprFlag_NotSupported;
            goto finish;
          }break;
        }
        
        // increment cursor
        cursor = step_cursor;
      }
      
      // check for end of task
      if (cursor < task_data.size){
        goto finish_task;
      }
    }
    
    finish_task:;
  }
  
  finish:;
  
  syms_release_scratch(scratch);
  
  return(result);
}


//- full eval

SYMS_API SYMS_DwLocation
syms_dw_expr__eval(SYMS_Arena *arena_optional, void *expr_base, SYMS_U64Range expr_range,
                   SYMS_DwExprMachineConfig *config){
  SYMS_DwLocation result = {0};
  
  // setup stack
  SYMS_ArenaTemp scratch = syms_get_scratch(&arena_optional, 1);
  SYMS_DwExprStack stack = syms_dw_expr__stack_make(scratch.arena);
  
  // adjust expr range
  void *expr_ptr = (SYMS_U8*)expr_base + expr_range.min;
  SYMS_U64 expr_size = expr_range.max - expr_range.min;
  
  // setup call stack
  SYMS_DwExprCallStack call_stack = {0};
  syms_dw_expr__call_push(scratch.arena, &call_stack, expr_ptr, expr_size);
  
  // state variables
  SYMS_DwSimpleLoc stashed_loc = {SYMS_DwSimpleLocKind_Address};
  
  
  // run loop
  SYMS_U64 max_step_count = config->max_step_count;
  SYMS_U64 step_counter = 0;
  for (;;){
    // check top of stack
    SYMS_DwExprCall *call = syms_dw_expr__call_top(&call_stack);
    if (call == 0){
      goto finish;
    }
    
    // grab top of stack details
    void *base = call->ptr;
    SYMS_U64Range range = {0, call->size};
    SYMS_U64 cursor = call->cursor;
    
    // decode op
    SYMS_U64 op_offset = cursor;
    SYMS_U8 op = 0;
    if (syms_based_range_read(base, range, op_offset, 1, &op)){
      SYMS_U64 after_op_off = cursor + 1;
      
      // require piece op after 'implicit' location descriptions
      if (stashed_loc.kind != SYMS_DwSimpleLocKind_Address){
        if (op != SYMS_DwOp_PIECE && op != SYMS_DwOp_BIT_PIECE){
          stashed_loc.kind = SYMS_DwSimpleLocKind_Fail;
          stashed_loc.fail_kind = SYMS_DwLocFailKind_BadData;
          goto finish;
        }
      }
      
      // step params
      SYMS_U64 size_param = 0;
      SYMS_B8  is_signed = 0;
      
      // step
      SYMS_U64 step_cursor = after_op_off;
      switch (op){
        
        //// literal encodings ////
        
        case SYMS_DwOp_LIT0:  case SYMS_DwOp_LIT1:  case SYMS_DwOp_LIT2:
        case SYMS_DwOp_LIT3:  case SYMS_DwOp_LIT4:  case SYMS_DwOp_LIT5:
        case SYMS_DwOp_LIT6:  case SYMS_DwOp_LIT7:  case SYMS_DwOp_LIT8:
        case SYMS_DwOp_LIT9:  case SYMS_DwOp_LIT10: case SYMS_DwOp_LIT11:
        case SYMS_DwOp_LIT12: case SYMS_DwOp_LIT13: case SYMS_DwOp_LIT14:
        case SYMS_DwOp_LIT15: case SYMS_DwOp_LIT16: case SYMS_DwOp_LIT17:
        case SYMS_DwOp_LIT18: case SYMS_DwOp_LIT19: case SYMS_DwOp_LIT20:
        case SYMS_DwOp_LIT21: case SYMS_DwOp_LIT22: case SYMS_DwOp_LIT23:
        case SYMS_DwOp_LIT24: case SYMS_DwOp_LIT25: case SYMS_DwOp_LIT26:
        case SYMS_DwOp_LIT27: case SYMS_DwOp_LIT28: case SYMS_DwOp_LIT29:
        case SYMS_DwOp_LIT30: case SYMS_DwOp_LIT31:
        {
          SYMS_U64 x = op - SYMS_DwOp_LIT0;
          syms_dw_expr__stack_push(scratch.arena, &stack, x);
        }break;
        
        case SYMS_DwOp_CONST1U:size_param = 1; goto const_n;
        case SYMS_DwOp_CONST2U:size_param = 2; goto const_n;
        case SYMS_DwOp_CONST4U:size_param = 4; goto const_n;
        case SYMS_DwOp_CONST8U:size_param = 8; goto const_n;
        case SYMS_DwOp_CONST1S:size_param = 1; is_signed = 1; goto const_n;
        case SYMS_DwOp_CONST2S:size_param = 2; is_signed = 1; goto const_n;
        case SYMS_DwOp_CONST4S:size_param = 4; is_signed = 1; goto const_n;
        case SYMS_DwOp_CONST8S:size_param = 8; is_signed = 1; goto const_n;
        const_n:
        {
          SYMS_U64 x = 0;
          step_cursor += syms_based_range_read(base, range, step_cursor, size_param, &x);
          // sign extend
          if (is_signed){
            SYMS_U64 bit_shift = (size_param << 3) - 1;
            if ((x >> bit_shift) != 0){
              x |= ~((1 << bit_shift) - 1);
            }
          }
          syms_dw_expr__stack_push(scratch.arena, &stack, x);
        }break;
        
        case SYMS_DwOp_ADDR:
        {
          SYMS_U64 offset = 0;
          step_cursor += syms_based_range_read(base, range, step_cursor, 8, &offset);

          // NOTE(nick): Earlier versions of GCC emit TLS offset with SYMS_DwOp_ADDR.
          SYMS_B32 is_text_relative;
          {
            SYMS_U8 next_op = 0;
            syms_based_range_read_struct(base, range, step_cursor, &next_op);
            is_text_relative = (next_op != SYMS_DwOp_GNU_PUSH_TLS_ADDRESS);
          }

          SYMS_U64 addr = offset;

          if (is_text_relative){
            if (config->text_section_base != 0){
              addr += *config->text_section_base;
            }
            else{
              stashed_loc.kind = SYMS_DwSimpleLocKind_Fail;
              stashed_loc.fail_kind = SYMS_DwLocFailKind_MissingTextBase;
              goto finish;
            }
          }

          syms_dw_expr__stack_push(scratch.arena, &stack, addr);
        }break;
        
        case SYMS_DwOp_CONSTU:
        {
          SYMS_U64 x = 0;
          step_cursor += syms_based_range_read_uleb128(base, range, step_cursor, &x);
          syms_dw_expr__stack_push(scratch.arena, &stack, x);
        }break;
        
        case SYMS_DwOp_CONSTS:
        {
          SYMS_U64 x = 0;
          step_cursor += syms_based_range_read_sleb128(base, range, step_cursor, (SYMS_S64*)&x);
          syms_dw_expr__stack_push(scratch.arena, &stack, x);
        }break;
        
        
        //// register based addressing ////
        
        case SYMS_DwOp_FBREG:
        {
          SYMS_S64 offset = 0;
          step_cursor += syms_based_range_read_sleb128(base, range, step_cursor, &offset);
          if (config->frame_base != 0){
            SYMS_U64 x = *config->frame_base + offset;
            syms_dw_expr__stack_push(scratch.arena, &stack, x);
          }
          else{
            stashed_loc.kind = SYMS_DwSimpleLocKind_Fail;
            stashed_loc.fail_kind = SYMS_DwLocFailKind_MissingFrameBase;
            goto finish;
          }
        }break;
        
        case SYMS_DwOp_BREG0:  case SYMS_DwOp_BREG1:  case SYMS_DwOp_BREG2:
        case SYMS_DwOp_BREG3:  case SYMS_DwOp_BREG4:  case SYMS_DwOp_BREG5:
        case SYMS_DwOp_BREG6:  case SYMS_DwOp_BREG7:  case SYMS_DwOp_BREG8:
        case SYMS_DwOp_BREG9:  case SYMS_DwOp_BREG10: case SYMS_DwOp_BREG11:
        case SYMS_DwOp_BREG12: case SYMS_DwOp_BREG13: case SYMS_DwOp_BREG14:
        case SYMS_DwOp_BREG15: case SYMS_DwOp_BREG16: case SYMS_DwOp_BREG17:
        case SYMS_DwOp_BREG18: case SYMS_DwOp_BREG19: case SYMS_DwOp_BREG20:
        case SYMS_DwOp_BREG21: case SYMS_DwOp_BREG22: case SYMS_DwOp_BREG23:
        case SYMS_DwOp_BREG24: case SYMS_DwOp_BREG25: case SYMS_DwOp_BREG26:
        case SYMS_DwOp_BREG27: case SYMS_DwOp_BREG28: case SYMS_DwOp_BREG29:
        case SYMS_DwOp_BREG30: case SYMS_DwOp_BREG31:
        {
          SYMS_S64 offset = 0;
          step_cursor += syms_based_range_read_sleb128(base, range, step_cursor, &offset);
          SYMS_U64 reg_idx = op - SYMS_DwOp_BREG0;
          SYMS_DwRegsX64 *regs = config->regs;
          if (regs != 0){
            if (reg_idx < SYMS_ARRAY_SIZE(regs->r)){
              SYMS_U64 x = regs->r[reg_idx] + offset;
              syms_dw_expr__stack_push(scratch.arena, &stack, x);
            }
            else{
              stashed_loc.kind = SYMS_DwSimpleLocKind_Fail;
              stashed_loc.fail_kind = SYMS_DwLocFailKind_BadData;
              stashed_loc.fail_data = op_offset;
              goto finish;
            }
          }
          else{
            stashed_loc.kind = SYMS_DwSimpleLocKind_Fail;
            stashed_loc.fail_kind = SYMS_DwLocFailKind_MissingRegisters;
            goto finish;
          }
        }break;
        
        case SYMS_DwOp_BREGX:
        {
          SYMS_U64 reg_idx = 0;
          step_cursor += syms_based_range_read_uleb128(base, range, step_cursor, &reg_idx);
          SYMS_S64 offset = 0;
          step_cursor += syms_based_range_read_sleb128(base, range, step_cursor, &offset);
          SYMS_DwRegsX64 *regs = config->regs;
          if (regs != 0){
            if (reg_idx < SYMS_ARRAY_SIZE(regs->r)){
              SYMS_U64 x = regs->r[reg_idx] + offset;
              syms_dw_expr__stack_push(scratch.arena, &stack, x);
            }
            else{
              stashed_loc.kind = SYMS_DwSimpleLocKind_Fail;
              stashed_loc.fail_kind = SYMS_DwLocFailKind_BadData;
              stashed_loc.fail_data = op_offset;
              goto finish;
            }
          }
          else{
            stashed_loc.kind = SYMS_DwSimpleLocKind_Fail;
            stashed_loc.fail_kind = SYMS_DwLocFailKind_MissingRegisters;
            goto finish;
          }
        }break;
        
        
        //// stack operations ////
        
        case SYMS_DwOp_DUP:
        {
          SYMS_U64 x = syms_dw_expr__stack_pick(&stack, 0);
          syms_dw_expr__stack_push(scratch.arena, &stack, x);
        }break;
        
        case SYMS_DwOp_DROP:
        {
          syms_dw_expr__stack_pop(&stack);
        }break;
        
        case SYMS_DwOp_PICK:
        {
          SYMS_U64 idx = 0;
          step_cursor += syms_based_range_read(base, range, step_cursor, 1, &idx);
          SYMS_U64 x = syms_dw_expr__stack_pick(&stack, idx);
          syms_dw_expr__stack_push(scratch.arena, &stack, x);
        }break;
        
        case SYMS_DwOp_OVER:
        {
          SYMS_U64 x = syms_dw_expr__stack_pick(&stack, 1);
          syms_dw_expr__stack_push(scratch.arena, &stack, x);
        }break;
        
        case SYMS_DwOp_SWAP:
        {
          SYMS_U64 a = syms_dw_expr__stack_pop(&stack);
          SYMS_U64 b = syms_dw_expr__stack_pop(&stack);
          syms_dw_expr__stack_push(scratch.arena, &stack, b);
          syms_dw_expr__stack_push(scratch.arena, &stack, a);
        }break;
        
        case SYMS_DwOp_ROT:
        {
          SYMS_U64 a = syms_dw_expr__stack_pop(&stack);
          SYMS_U64 b = syms_dw_expr__stack_pop(&stack);
          SYMS_U64 c = syms_dw_expr__stack_pop(&stack);
          syms_dw_expr__stack_push(scratch.arena, &stack, a);
          syms_dw_expr__stack_push(scratch.arena, &stack, c);
          syms_dw_expr__stack_push(scratch.arena, &stack, b);
        }break;
        
        case SYMS_DwOp_DEREF:
        {
          SYMS_U64 addr = syms_dw_expr__stack_pop(&stack);
          SYMS_B32 read_success = syms_false;
          SYMS_MemoryView *memview = config->memview;
          if (memview != 0){
            SYMS_U64 x = 0;
            if (syms_memory_view_read_struct(memview, addr, &x)){
              syms_dw_expr__stack_push(scratch.arena, &stack, x);
              read_success = syms_true;
            }
          }
          if (!read_success){
            stashed_loc.kind = SYMS_DwSimpleLocKind_Fail;
            stashed_loc.fail_kind = SYMS_DwLocFailKind_MissingMemory;
            stashed_loc.fail_data = addr;
            goto finish;
          }
        }break;
        
        case SYMS_DwOp_DEREF_SIZE:
        {
          SYMS_U64 raw_size = 0;
          step_cursor += syms_based_range_read(base, range, step_cursor, 1, &raw_size);
          SYMS_U64 size = SYMS_ClampTop(raw_size, 8);
          SYMS_U64 addr = syms_dw_expr__stack_pop(&stack);
          SYMS_B32 read_success = syms_false;
          SYMS_MemoryView *memview = config->memview;
          if (memview != 0){
            SYMS_U64 x = 0;
            if (syms_memory_view_read(memview, addr, size, &x)){
              syms_dw_expr__stack_push(scratch.arena, &stack, x);
              read_success = syms_true;
            }
          }
          if (!read_success){
            stashed_loc.kind = SYMS_DwSimpleLocKind_Fail;
            stashed_loc.fail_kind = SYMS_DwLocFailKind_MissingMemory;
            stashed_loc.fail_data = addr;
            goto finish;
          }
        }break;
        
        case SYMS_DwOp_XDEREF:
        case SYMS_DwOp_XDEREF_SIZE:
        {
          stashed_loc.kind = SYMS_DwSimpleLocKind_Fail;
          stashed_loc.fail_kind = SYMS_DwLocFailKind_NotSupported;
          goto finish;
        }break;
        
        case SYMS_DwOp_PUSH_OBJECT_ADDRESS:
        {
          if (config->object_address != 0){
            SYMS_U64 x = *config->object_address;
            syms_dw_expr__stack_push(scratch.arena, &stack, x);
          }
          else{
            stashed_loc.kind = SYMS_DwSimpleLocKind_Fail;
            stashed_loc.fail_kind = SYMS_DwLocFailKind_MissingObjectAddress;
            goto finish;
          }
        }break;
        
        // NOTE: pop offset from stack, convert it to TLS address, then push it back.
        case SYMS_DwOp_GNU_PUSH_TLS_ADDRESS:
        case SYMS_DwOp_FORM_TLS_ADDRESS:
        {
          SYMS_S64 s = (SYMS_S64)syms_dw_expr__stack_pop(&stack);

          if (config->tls_address != 0){
            SYMS_U64 x = *config->tls_address + s;
            syms_dw_expr__stack_push(scratch.arena, &stack, x);
          }
          else{
            stashed_loc.kind = SYMS_DwSimpleLocKind_Fail;
            stashed_loc.fail_kind = SYMS_DwLocFailKind_MissingTLSAddress;
            goto finish;
          }
        }break;
        
        case SYMS_DwOp_CALL_FRAME_CFA:
        {
          if (config->cfa != 0){
            SYMS_U64 x = *config->cfa;
            syms_dw_expr__stack_push(scratch.arena, &stack, x);
          }
          else{
            stashed_loc.kind = SYMS_DwSimpleLocKind_Fail;
            stashed_loc.fail_kind = SYMS_DwLocFailKind_MissingCFA;
            goto finish;
          }
        }break;
        
        
        //// arithmetic and logical operations ////
        
        case SYMS_DwOp_ABS:
        {
          SYMS_S64 s = (SYMS_S64)syms_dw_expr__stack_pop(&stack);
          SYMS_S64 x = (s < 0)?-s:s;
          syms_dw_expr__stack_push(scratch.arena, &stack, x);
        }break;
        
        case SYMS_DwOp_AND:
        {
          SYMS_U64 x = syms_dw_expr__stack_pop(&stack);
          SYMS_U64 y = syms_dw_expr__stack_pop(&stack);
          syms_dw_expr__stack_push(scratch.arena, &stack, x&y);
        }break;
        
        case SYMS_DwOp_DIV:
        {
          SYMS_S64 d = (SYMS_S64)syms_dw_expr__stack_pop(&stack);
          SYMS_S64 n = (SYMS_S64)syms_dw_expr__stack_pop(&stack);
          SYMS_S64 x = (d == 0)?0:n/d;
          syms_dw_expr__stack_push(scratch.arena, &stack, x);
        }break;
        
        case SYMS_DwOp_MINUS:
        {
          SYMS_U64 b = syms_dw_expr__stack_pop(&stack);
          SYMS_U64 a = syms_dw_expr__stack_pop(&stack);
          SYMS_U64 x = a - b;
          syms_dw_expr__stack_push(scratch.arena, &stack, x);
        }break;
        
        case SYMS_DwOp_MOD:
        {
          SYMS_S64 d = (SYMS_S64)syms_dw_expr__stack_pop(&stack);
          SYMS_S64 n = (SYMS_S64)syms_dw_expr__stack_pop(&stack);
          SYMS_S64 x = (d == 0)?0:n%d;
          syms_dw_expr__stack_push(scratch.arena, &stack, x);
        }break;
        
        case SYMS_DwOp_MUL:
        {
          SYMS_U64 b = syms_dw_expr__stack_pop(&stack);
          SYMS_U64 a = syms_dw_expr__stack_pop(&stack);
          SYMS_U64 x = a*b;
          syms_dw_expr__stack_push(scratch.arena, &stack, x);
        }break;
        
        case SYMS_DwOp_NEG:
        {
          SYMS_S64 s = (SYMS_S64)syms_dw_expr__stack_pop(&stack);
          SYMS_S64 x = -s;
          syms_dw_expr__stack_push(scratch.arena, &stack, x);
        }break;
        
        case SYMS_DwOp_NOT:
        {
          SYMS_U64 y = syms_dw_expr__stack_pop(&stack);
          SYMS_U64 x = ~y;
          syms_dw_expr__stack_push(scratch.arena, &stack, x);
        }break;
        
        case SYMS_DwOp_OR:
        {
          SYMS_U64 y = syms_dw_expr__stack_pop(&stack);
          SYMS_U64 z = syms_dw_expr__stack_pop(&stack);
          SYMS_U64 x = y | z;
          syms_dw_expr__stack_push(scratch.arena, &stack, x);
        }break;
        
        case SYMS_DwOp_PLUS:
        {
          SYMS_U64 y = syms_dw_expr__stack_pop(&stack);
          SYMS_U64 z = syms_dw_expr__stack_pop(&stack);
          SYMS_U64 x = y + z;
          syms_dw_expr__stack_push(scratch.arena, &stack, x);
        }break;
        
        case SYMS_DwOp_PLUS_UCONST:
        {
          SYMS_U64 y = 0;
          step_cursor += syms_based_range_read_uleb128(base, range, step_cursor, &y);
          SYMS_U64 z = syms_dw_expr__stack_pop(&stack);
          SYMS_U64 x = y + z;
          syms_dw_expr__stack_push(scratch.arena, &stack, x);
        }break;
        
        case SYMS_DwOp_SHL:
        {
          SYMS_U64 y = syms_dw_expr__stack_pop(&stack);
          SYMS_U64 z = syms_dw_expr__stack_pop(&stack);
          SYMS_U64 x = 0;
          if (y < 64){
            x = z << y;
          }
          syms_dw_expr__stack_push(scratch.arena, &stack, x);
        }break;
        
        case SYMS_DwOp_SHR:
        {
          SYMS_U64 y = syms_dw_expr__stack_pop(&stack);
          SYMS_U64 z = syms_dw_expr__stack_pop(&stack);
          SYMS_U64 x = 0;
          if (y < 64){
            x = z >> y;
          }
          syms_dw_expr__stack_push(scratch.arena, &stack, x);
        }break;
        
        case SYMS_DwOp_SHRA:
        {
          SYMS_U64 y = syms_dw_expr__stack_pop(&stack);
          SYMS_U64 z = syms_dw_expr__stack_pop(&stack);
          SYMS_U64 x = 0;
          if (y < 64){
            x = z >> y;
            // sign extensions
            if (y > 0 && (z & (1ull << 63))){
              x |= ~((1 << (64 - y)) - 1);
            }
          }
          syms_dw_expr__stack_push(scratch.arena, &stack, x);
        }break;
        
        case SYMS_DwOp_XOR:
        {
          SYMS_U64 y = syms_dw_expr__stack_pop(&stack);
          SYMS_U64 z = syms_dw_expr__stack_pop(&stack);
          SYMS_U64 x = y ^ z;
          syms_dw_expr__stack_push(scratch.arena, &stack, x);
        }break;
        
        
        //// control flow operations ////
        
        case SYMS_DwOp_LE:
        {
          SYMS_S64 b = (SYMS_S64)syms_dw_expr__stack_pop(&stack);
          SYMS_S64 a = (SYMS_S64)syms_dw_expr__stack_pop(&stack);
          SYMS_U64 x = (a <= b);
          syms_dw_expr__stack_push(scratch.arena, &stack, x);
        }break;
        
        case SYMS_DwOp_GE:
        {
          SYMS_S64 b = (SYMS_S64)syms_dw_expr__stack_pop(&stack);
          SYMS_S64 a = (SYMS_S64)syms_dw_expr__stack_pop(&stack);
          SYMS_U64 x = (a >= b);
          syms_dw_expr__stack_push(scratch.arena, &stack, x);
        }break;
        
        case SYMS_DwOp_EQ:
        {
          SYMS_S64 b = (SYMS_S64)syms_dw_expr__stack_pop(&stack);
          SYMS_S64 a = (SYMS_S64)syms_dw_expr__stack_pop(&stack);
          SYMS_U64 x = (a == b);
          syms_dw_expr__stack_push(scratch.arena, &stack, x);
        }break;
        
        case SYMS_DwOp_LT:
        {
          SYMS_S64 b = (SYMS_S64)syms_dw_expr__stack_pop(&stack);
          SYMS_S64 a = (SYMS_S64)syms_dw_expr__stack_pop(&stack);
          SYMS_U64 x = (a < b);
          syms_dw_expr__stack_push(scratch.arena, &stack, x);
        }break;
        
        case SYMS_DwOp_GT:
        {
          SYMS_S64 b = (SYMS_S64)syms_dw_expr__stack_pop(&stack);
          SYMS_S64 a = (SYMS_S64)syms_dw_expr__stack_pop(&stack);
          SYMS_U64 x = (a > b);
          syms_dw_expr__stack_push(scratch.arena, &stack, x);
        }break;
        
        case SYMS_DwOp_NE:
        {
          SYMS_S64 b = (SYMS_S64)syms_dw_expr__stack_pop(&stack);
          SYMS_S64 a = (SYMS_S64)syms_dw_expr__stack_pop(&stack);
          SYMS_U64 x = (a != b);
          syms_dw_expr__stack_push(scratch.arena, &stack, x);
        }break;
        
        case SYMS_DwOp_SKIP:
        {
          SYMS_S16 d = 0;
          step_cursor += syms_based_range_read(base, range, step_cursor, 2, &d);
          step_cursor = step_cursor + d;
        }break;
        
        case SYMS_DwOp_BRA:
        {
          SYMS_S16 d = 0;
          step_cursor += syms_based_range_read(base, range, step_cursor, 2, &d);
          SYMS_U64 b = syms_dw_expr__stack_pop(&stack);
          if (b != 0){
            step_cursor = step_cursor + d;
          }
        }break;
        
        case SYMS_DwOp_CALL2:
        {
          SYMS_U16 p = 0;
          step_cursor += syms_based_range_read(base, range, step_cursor, 2, &p);
          if (config->call.func != 0){
            SYMS_String8 sub_data = config->call.func(config->call.user_ptr, p);
            syms_dw_expr__call_push(scratch.arena, &call_stack, sub_data.str, sub_data.size);
          }
          else{
            stashed_loc.kind = SYMS_DwSimpleLocKind_Fail;
            stashed_loc.fail_kind = SYMS_DwLocFailKind_MissingCallResolution;
            goto finish;
          }
        }break;
        
        case SYMS_DwOp_CALL4:
        {
          SYMS_U32 p = 0;
          step_cursor += syms_based_range_read(base, range, step_cursor, 4, &p);
          if (config->call.func != 0){
            SYMS_String8 sub_data = config->call.func(config->call.user_ptr, p);
            syms_dw_expr__call_push(scratch.arena, &call_stack, sub_data.str, sub_data.size);
          }
          else{
            stashed_loc.kind = SYMS_DwSimpleLocKind_Fail;
            stashed_loc.fail_kind = SYMS_DwLocFailKind_MissingCallResolution;
            goto finish;
          }
        }break;
        
        case SYMS_DwOp_CALL_REF:
        {
          stashed_loc.kind = SYMS_DwSimpleLocKind_Fail;
          stashed_loc.fail_kind = SYMS_DwLocFailKind_NotSupported;
          goto finish;
        }break;
        
        
        //// special operations ////
        
        case SYMS_DwOp_NOP:break;
        
        
        //// register location descriptions ////
        
        case SYMS_DwOp_REG0:  case SYMS_DwOp_REG1:  case SYMS_DwOp_REG2:
        case SYMS_DwOp_REG3:  case SYMS_DwOp_REG4:  case SYMS_DwOp_REG5:
        case SYMS_DwOp_REG6:  case SYMS_DwOp_REG7:  case SYMS_DwOp_REG8:
        case SYMS_DwOp_REG9:  case SYMS_DwOp_REG10: case SYMS_DwOp_REG11:
        case SYMS_DwOp_REG12: case SYMS_DwOp_REG13: case SYMS_DwOp_REG14:
        case SYMS_DwOp_REG15: case SYMS_DwOp_REG16: case SYMS_DwOp_REG17:
        case SYMS_DwOp_REG18: case SYMS_DwOp_REG19: case SYMS_DwOp_REG20:
        case SYMS_DwOp_REG21: case SYMS_DwOp_REG22: case SYMS_DwOp_REG23:
        case SYMS_DwOp_REG24: case SYMS_DwOp_REG25: case SYMS_DwOp_REG26:
        case SYMS_DwOp_REG27: case SYMS_DwOp_REG28: case SYMS_DwOp_REG29:
        case SYMS_DwOp_REG30: case SYMS_DwOp_REG31:
        {
          SYMS_U64 reg_idx = op - SYMS_DwOp_REG0;
          stashed_loc.kind = SYMS_DwSimpleLocKind_Register;
          stashed_loc.reg_idx = reg_idx;
        }break;
        
        case SYMS_DwOp_REGX:
        {
          SYMS_U64 reg_idx = 0;
          step_cursor += syms_based_range_read(base, range, step_cursor, size_param, &reg_idx);
          stashed_loc.kind = SYMS_DwSimpleLocKind_Register;
          stashed_loc.reg_idx = reg_idx;
        }break;
        
        
        //// implicit location descriptions ////
        
        case SYMS_DwOp_IMPLICIT_VALUE:
        {
          SYMS_U64 size = 0;
          step_cursor += syms_based_range_read(base, range, step_cursor, size_param, &size);
          if (step_cursor + size <= range.max){
            void *data = (SYMS_U8*)base + range.min + step_cursor;
            stashed_loc.kind = SYMS_DwSimpleLocKind_ValueLong;
            stashed_loc.val_long.str  = (SYMS_U8*)data;
            stashed_loc.val_long.size = size;
          }
          else{
            stashed_loc.kind = SYMS_DwSimpleLocKind_Fail;
            stashed_loc.fail_kind = SYMS_DwLocFailKind_BadData;
            goto finish;
          }
          step_cursor += size;
        }break;
        
        case SYMS_DwOp_STACK_VALUE:
        {
          SYMS_U64 x = syms_dw_expr__stack_pop(&stack);
          stashed_loc.kind = SYMS_DwSimpleLocKind_Value;
          stashed_loc.val = x;
        }break;
        
        
        //// composite location descriptions ////
        
        case SYMS_DwOp_PIECE:
        case SYMS_DwOp_BIT_PIECE:
        {
          if (arena_optional == 0){
            stashed_loc.kind = SYMS_DwSimpleLocKind_Fail;
            stashed_loc.fail_kind = SYMS_DwLocFailKind_MissingArenaForComposite;
            goto finish;
          }
          else{
            
            // determine this piece's size & offset
            SYMS_U64 bit_size = 0;
            SYMS_U64 bit_off = 0;
            SYMS_B32 is_bit_loc = syms_false;
            switch (op){
              case SYMS_DwOp_PIECE:
              {
                SYMS_U64 size = 0;
                step_cursor += syms_based_range_read_uleb128(base, range, step_cursor, &size);
                bit_size = size*8;
              }break;
              case SYMS_DwOp_BIT_PIECE:
              {
                step_cursor += syms_based_range_read_uleb128(base, range, step_cursor, &bit_size);
                step_cursor += syms_based_range_read_uleb128(base, range, step_cursor, &bit_off);
                is_bit_loc = syms_true;
              }break;
            }
            
            // determine this piece's location information
            SYMS_DwSimpleLoc piece_loc = stashed_loc;
            if (piece_loc.kind == SYMS_DwSimpleLocKind_Address){
              if (syms_dw_expr__stack_is_empty(&stack)){
                piece_loc.kind = SYMS_DwSimpleLocKind_Empty;
              }
              else{
                SYMS_U64 x = syms_dw_expr__stack_pop(&stack);
                piece_loc.addr = x;
              }
            }
            
            // push the piece
            SYMS_DwPiece *piece = syms_push_array_zero(arena_optional, SYMS_DwPiece, 1);
            SYMS_QueuePush(result.first_piece, result.last_piece, piece);
            piece->loc = piece_loc;
            piece->bit_size = bit_size;
            piece->bit_off  = bit_off;
            piece->is_bit_loc = is_bit_loc;
            
            // zero the stached loc
            syms_memzero_struct(&stashed_loc);
          }
        }break;
        
        
        //// final fallback ////
        
        default:
        {
          stashed_loc.kind = SYMS_DwSimpleLocKind_Fail;
          stashed_loc.fail_kind = SYMS_DwLocFailKind_NotSupported;
          goto finish;
        }break;
      }
      
      // increment cursor
      cursor = step_cursor;
    }
    
    // advance cursor or finish call
    if (cursor < call->size){
      call->cursor = cursor;
    }
    else{
      syms_dw_expr__call_pop(&call_stack);
    }
    
    // advance step counter
    step_counter += 1;
    if (step_counter == max_step_count){
      stashed_loc.kind = SYMS_DwSimpleLocKind_Fail;
      stashed_loc.fail_kind = SYMS_DwLocFailKind_TimeOut;
      goto finish;
    }
  }
  
  finish:;
  
  // non-piece location
  {
    SYMS_DwSimpleLoc loc = stashed_loc;
    if (result.first_piece == 0){
      
      // normal location resolution
      loc = stashed_loc;
      if (loc.kind == SYMS_DwSimpleLocKind_Address){
        if (syms_dw_expr__stack_is_empty(&stack)){
          loc.kind = SYMS_DwSimpleLocKind_Empty;
        }
        else{
          SYMS_U64 x = syms_dw_expr__stack_pop(&stack);
          loc.addr = x;
        }
      }
    }
    
    // non-piece location resolution after composite
    else{
      
      // change the default kind to empty
      if (loc.kind == SYMS_DwSimpleLocKind_Address){
        loc.kind = SYMS_DwSimpleLocKind_Empty;
      }
      
      // the non-piece should either be empty or fail
      if (loc.kind != SYMS_DwSimpleLocKind_Empty &&
          loc.kind != SYMS_DwSimpleLocKind_Fail){
        loc.kind = SYMS_DwSimpleLocKind_Fail;
        loc.fail_kind = SYMS_DwLocFailKind_BadData;
      }
    }
    
    result.non_piece_loc = loc;
  }
  
  // clear stack
  syms_release_scratch(scratch);
  
  return(result);
}


//- dw expr val stack

SYMS_API SYMS_DwExprStack
syms_dw_expr__stack_make(SYMS_Arena *arena){
  SYMS_DwExprStack result = {0};
  return(result);
}

SYMS_API void
syms_dw_expr__stack_push(SYMS_Arena *arena, SYMS_DwExprStack *stack, SYMS_U64 x){
  SYMS_DwExprStackNode *node = stack->free_nodes;
  if (node == 0){
    SYMS_StackPop(stack->free_nodes);
  }
  else{
    node = syms_push_array(arena, SYMS_DwExprStackNode, 1);
  }
  SYMS_StackPush(stack->stack, node);
  node->val = x;
  stack->count += 1;
}

SYMS_API SYMS_U64
syms_dw_expr__stack_pop(SYMS_DwExprStack *stack){
  SYMS_U64 result = 0;
  SYMS_DwExprStackNode *node = stack->stack;
  if (node != 0){
    SYMS_StackPop(stack->stack);
    stack->count -= 1;
    result = node->val;
  }
  return(result);
}

SYMS_API SYMS_U64
syms_dw_expr__stack_pick(SYMS_DwExprStack *stack, SYMS_U64 idx){
  SYMS_U64 result = 0;
  if (idx < stack->count){
    SYMS_U64 counter = idx;
    SYMS_DwExprStackNode *node = stack->stack;
    for (;node != 0 && counter > 0;
         node = node->next, counter -= 1);
    if (counter == 0 && node != 0){
      result = node->val;
    }
  }
  return(result);
}

SYMS_API SYMS_B32
syms_dw_expr__stack_is_empty(SYMS_DwExprStack *stack){
  SYMS_B32 result = (stack->count == 0);
  return(result);
}

//- dw expr call stack

SYMS_API SYMS_DwExprCall*
syms_dw_expr__call_top(SYMS_DwExprCallStack *stack){
  SYMS_DwExprCall *call = stack->stack;
  return(call);
}

SYMS_API void
syms_dw_expr__call_push(SYMS_Arena *arena, SYMS_DwExprCallStack *stack, void *ptr, SYMS_U64 size){
  SYMS_DwExprCall *call = 0;
  if (call != 0){
    SYMS_StackPop(stack->free_calls);
  }
  else{
    call = syms_push_array(arena, SYMS_DwExprCall, 1);
  }
  syms_memzero_struct(call);
  SYMS_StackPush(stack->stack, call);
  stack->depth += 1;
}

SYMS_API void
syms_dw_expr__call_pop(SYMS_DwExprCallStack *stack){
  SYMS_DwExprCall *top = stack->stack;
  if (top != 0){
    SYMS_StackPop(stack->stack);
    SYMS_StackPush(stack->free_calls, top);
  }
}

//- analysis tasks

SYMS_API SYMS_DwExprAnalysisTask*
syms_dw_expr__analysis_task_from_p(SYMS_DwExprAnalysisTask *first, SYMS_U64 p){
  SYMS_DwExprAnalysisTask *result = 0;
  for (SYMS_DwExprAnalysisTask *task = first;
       task != 0;
       task = task->next){
    if (task->p == p){
      result = task;
      break;
    }
  }
  return(result);
}

#endif //SYMS_DWARF_EXPR_C

