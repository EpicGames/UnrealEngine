// Copyright Epic Games, Inc. All Rights Reserved.
/* date = October 4th 2021 2:20 pm */

#ifndef SYMS_DWARF_EXPR_H
#define SYMS_DWARF_EXPR_H

////////////////////////////////
//~ NOTE(allen): Include Generated Types

#include "syms/core/generated/syms_meta_dwarf_expr.h"

////////////////////////////////
//~ NOTE(allen): Dwarf Register Layout

typedef struct SYMS_DwRegsX64{
  union{
    struct{
      SYMS_U64 rax;
      SYMS_U64 rdx;
      SYMS_U64 rcx;
      SYMS_U64 rbx;
      SYMS_U64 rsi;
      SYMS_U64 rdi;
      SYMS_U64 rbp;
      SYMS_U64 rsp;
      SYMS_U64 r8;
      SYMS_U64 r9;
      SYMS_U64 r10;
      SYMS_U64 r11;
      SYMS_U64 r12;
      SYMS_U64 r13;
      SYMS_U64 r14;
      SYMS_U64 r15;
      SYMS_U64 rip;
    };
    SYMS_U64 r[17];
  };
} SYMS_DwRegsX64;

SYMS_GLOBAL SYMS_READ_ONLY SYMS_RegID syms_dw_reg_table_x64[] = {
  SYMS_RegX64Code_rax,
  SYMS_RegX64Code_rdx,
  SYMS_RegX64Code_rcx,
  SYMS_RegX64Code_rbx,
  SYMS_RegX64Code_rsi,
  SYMS_RegX64Code_rdi,
  SYMS_RegX64Code_rbp,
  SYMS_RegX64Code_rsp,
  SYMS_RegX64Code_r8,
  SYMS_RegX64Code_r9,
  SYMS_RegX64Code_r10,
  SYMS_RegX64Code_r11,
  SYMS_RegX64Code_r12,
  SYMS_RegX64Code_r13,
  SYMS_RegX64Code_r14,
  SYMS_RegX64Code_r15,
  SYMS_RegX64Code_rip,
};

SYMS_GLOBAL SYMS_READ_ONLY SYMS_RegID syms_dw_reg_table_x86[] = {
  SYMS_RegX86Code_eax,
  SYMS_RegX86Code_edx,
  SYMS_RegX86Code_ecx,
  SYMS_RegX86Code_ebx,
  SYMS_RegX86Code_esi,
  SYMS_RegX86Code_edi,
  SYMS_RegX86Code_ebp,
  SYMS_RegX86Code_esp,
  SYMS_RegX86Code_eip,
};

////////////////////////////////
//~ NOTE(allen): Dwarf Expression Eval Types

//- machine configuration types
typedef SYMS_String8 SYMS_DwExprResolveCallFunc(void *call_user_ptr, SYMS_U64 p);

typedef struct SYMS_DwExprMachineCallConfig{
  void *user_ptr;
  SYMS_DwExprResolveCallFunc *func;
} SYMS_DwExprMachineCallConfig;

typedef struct SYMS_DwExprMachineConfig{
  // (read only in the eval functions)
  SYMS_U64 max_step_count;
  SYMS_MemoryView *memview;
  SYMS_DwRegsX64 *regs;
  SYMS_U64 *text_section_base;
  SYMS_U64 *frame_base;
  SYMS_U64 *object_address;
  SYMS_U64 *tls_address;
  SYMS_U64 *cfa;
  SYMS_DwExprMachineCallConfig call;
} SYMS_DwExprMachineConfig;


//- detail analysis types
typedef SYMS_U32 SYMS_DwExprFlags;
enum{
  SYMS_DwExprFlag_UsesTextBase       = (1 << 0),
  SYMS_DwExprFlag_UsesMemory         = (1 << 1),
  SYMS_DwExprFlag_UsesRegisters      = (1 << 2),
  SYMS_DwExprFlag_UsesFrameBase      = (1 << 3),
  SYMS_DwExprFlag_UsesObjectAddress  = (1 << 4),
  SYMS_DwExprFlag_UsesTLSAddress     = (1 << 5),
  SYMS_DwExprFlag_UsesCFA            = (1 << 6),
  SYMS_DwExprFlag_UsesCallResolution = (1 << 7),
  SYMS_DwExprFlag_UsesComposite      = (1 << 8),
  
  SYMS_DwExprFlag_NotSupported  = (1 << 16),
  SYMS_DwExprFlag_BadData       = (1 << 17),
  SYMS_DwExprFlag_NonLinearFlow = (1 << 18),
};

typedef struct SYMS_DwExprAnalysis{
  SYMS_DwExprFlags flags;
} SYMS_DwExprAnalysis;

typedef struct SYMS_DwExprAnalysisTask{
  struct SYMS_DwExprAnalysisTask *next;
  SYMS_U64 p;
  SYMS_String8 data;
} SYMS_DwExprAnalysisTask;


//- location types
typedef enum SYMS_DwSimpleLocKind{
  SYMS_DwSimpleLocKind_Address,
  SYMS_DwSimpleLocKind_Register,
  SYMS_DwSimpleLocKind_Value,
  SYMS_DwSimpleLocKind_ValueLong,
  SYMS_DwSimpleLocKind_Empty,
  SYMS_DwSimpleLocKind_Fail,
} SYMS_DwSimpleLocKind;

typedef enum SYMS_DwLocFailKind{
  // NOTE(allen): Interpreting Fail Kinds
  // BadData:        the evaluator detected that the dwarf expression operation is incorrectly formed
  // NotSupported:   the evaluator does not support a dwarf feature that was found in the dwarf expression
  // TimeOut:        the evaluator hit the maximum step count
  // TooComplicated: used by analyzer when it the expression uses features outside of the analyzer's scope
  // Missing*:       the dwarf machine config was missing necessary information to finish the evaluation
  
  SYMS_DwLocFailKind_BadData,
  SYMS_DwLocFailKind_NotSupported,
  SYMS_DwLocFailKind_TimeOut,
  SYMS_DwLocFailKind_TooComplicated,
  SYMS_DwLocFailKind_MissingTextBase,
  SYMS_DwLocFailKind_MissingMemory,
  SYMS_DwLocFailKind_MissingRegisters,
  SYMS_DwLocFailKind_MissingFrameBase,
  SYMS_DwLocFailKind_MissingObjectAddress,
  SYMS_DwLocFailKind_MissingTLSAddress,
  SYMS_DwLocFailKind_MissingCFA,
  SYMS_DwLocFailKind_MissingCallResolution,
  SYMS_DwLocFailKind_MissingArenaForComposite,
} SYMS_DwLocFailKind;

typedef struct SYMS_DwSimpleLoc{
  SYMS_DwSimpleLocKind kind;
  union{
    SYMS_U64 addr;
    SYMS_U64 reg_idx;
    SYMS_U64 val;
    SYMS_String8 val_long;
    struct{
      SYMS_DwLocFailKind fail_kind;
      SYMS_U64 fail_data;
    };
  };
} SYMS_DwSimpleLoc;

typedef struct SYMS_DwPiece{
  // NOTE(allen): Hint for Interpreting Pieces
  // src = decode(loc, is_bit_loc, bit_size);
  //  dst |= (src >> bit_off) << bit_cursor;
  // bit_cursor += bit_size;
  
  struct SYMS_DwPiece *next;
  SYMS_DwSimpleLoc loc;
  SYMS_U64 bit_size;
  SYMS_U64 bit_off;
  SYMS_B32 is_bit_loc;
} SYMS_DwPiece;

typedef struct SYMS_DwLocation{
  // NOTE(allen): Interpreting a Dwarf Location
  //
  // CASE (any number of pieces, fail in the non-piece):
  //   this is how errors are reported, error information is in the non-piece
  //   the 'fail' location kind should never show up in a piece
  //   if there are any pieces they can be treated as correct information that
  //   was successfully decoded before the error was encountered
  //
  // CASE (no pieces, empty non-piece):
  //   the data is completely optimized out and unrecoverable
  //
  // CASE (no pieces, non-empty non-piece):
  //   the size of the data is not known by the location, but something in the
  //   surrounding context of the location (eg type info) should know the size
  //
  // CASE (one-or-more pieces, empty non-piece):
  //   the data is described by the pieces
  //
  // CASE (one-or-more pieces, non-empty non-fail non-piece):
  //   this is supposed to be impossible; the non-piece either carries an error
  //   or *all* of the location information about the data, there should never
  //   be a mix of piece-based location and non-piece-based location data.
  
  SYMS_DwPiece *first_piece;
  SYMS_DwPiece *last_piece;
  SYMS_U64 count;
  
  SYMS_DwSimpleLoc non_piece_loc;
} SYMS_DwLocation;


//- full evaluator state types
typedef struct SYMS_DwExprStackNode{
  struct SYMS_DwExprStackNode *next;
  SYMS_U64 val;
} SYMS_DwExprStackNode;

typedef struct SYMS_DwExprStack{
  SYMS_DwExprStackNode *stack;
  SYMS_DwExprStackNode *free_nodes;
  SYMS_U64 count;
} SYMS_DwExprStack;

typedef struct SYMS_DwExprCall{
  struct SYMS_DwExprCall *next;
  void *ptr;
  SYMS_U64 size;
  SYMS_U64 cursor;
} SYMS_DwExprCall;

typedef struct SYMS_DwExprCallStack{
  SYMS_DwExprCall *stack;
  SYMS_DwExprCall *free_calls;
  SYMS_U64 depth;
} SYMS_DwExprCallStack;


////////////////////////////////
//~ NOTE(allen): Dwarf Expression Analysis & Eval Functions

//- analyzers

// NOTE(allen): This analyzer provides the most simplified dwarf expression
// decoding. If the expression consists of a single op that can be interpreted
// as a valid dwarf expression, then it represents that expression as a simple
// location.
//
// If there is a single 'piece' op that is represeted here as an empty simple
// location, losing whatever additional size information from the piece.
//
// If there is an op that requires the machine configuration data the analyzer
// fails with "too complicated" - unless the required configuration data is the
// text section base which this analyzer treats as a non-optional parameter and
// always decodes successfully.
//
// If the expression contains more than one op than the analyzer fails with
// "too complicated".

SYMS_API SYMS_DwSimpleLoc syms_dw_expr__analyze_fast(void *base, SYMS_U64Range range, SYMS_U64 text_section_base);



// NOTE(allen): This analyzer does a one-pass scan through the expression to
// help a caller determine what to expect before doing a full evaluation which
// has to maintain value stacks, perform more checks, and execute any loops
// that may appear in the expression, etc.
//
// For each piece of data that can be equipped to a machine config there is a
// 'Uses' flag in the analysis. A user can use these flags to determine what to
// prepare and equip before a full eval. This can be a lot more efficient than
// always preparing everything, or iteratively equipping and retrying after
// each failure.
//
// The analysis can also catch some cases of bad data and unsupported features.
// These flags are useful for short circuit style optimizations, but they are
// not definitive, some bad data can only be caught by the full evaluator.
// Sometimes the full evaluator might miss bad data that this analyzer will see
// if control flow in the evaluator completely skips the bad data. A forgiving
// interpretation of dwarf expression data would only rely on the results of
// the full evaluator. A more strict interpretation would consider it an error
// if either this analyzer or the evaluator finds bad data.
//
// The analyzer also determines if there is any possibility for non-linear
// flow. Jumps, branches, and call ops all create non-linear flow. An
// expression that doesn't have non-linear flow is trivially gauranteed to
// terminate and therefore a good candidate for conversion to a human readable
// expression.
//
// The call config is optional (may be null). If is provided the analysis
// includes features seen in all of the expressions that might be reached by
// call ops from the initial expression.

SYMS_API SYMS_DwExprAnalysis syms_dw_expr__analyze_details(void *base, SYMS_U64Range range,
                                                           SYMS_DwExprMachineCallConfig *call_config);




//- full eval
SYMS_API SYMS_DwLocation syms_dw_expr__eval(SYMS_Arena *arena_optional, void *base, SYMS_U64Range range,
                                            SYMS_DwExprMachineConfig *config);


//- dw expr val stack
SYMS_API SYMS_DwExprStack syms_dw_expr__stack_make(SYMS_Arena *arena);
SYMS_API void             syms_dw_expr__stack_push(SYMS_Arena *arena, SYMS_DwExprStack *stack, SYMS_U64 x);
SYMS_API SYMS_U64         syms_dw_expr__stack_pop(SYMS_DwExprStack *stack);
SYMS_API SYMS_U64         syms_dw_expr__stack_pick(SYMS_DwExprStack *stack, SYMS_U64 idx);
SYMS_API SYMS_B32         syms_dw_expr__stack_is_empty(SYMS_DwExprStack *stack);

//- dw expr call stack
SYMS_API SYMS_DwExprCall* syms_dw_expr__call_top(SYMS_DwExprCallStack *stack);
SYMS_API void             syms_dw_expr__call_push(SYMS_Arena *arena, SYMS_DwExprCallStack *stack,
                                                  void *ptr, SYMS_U64 size);
SYMS_API void             syms_dw_expr__call_pop(SYMS_DwExprCallStack *stack);


//- analysis tasks
SYMS_API SYMS_DwExprAnalysisTask* syms_dw_expr__analysis_task_from_p(SYMS_DwExprAnalysisTask *first, SYMS_U64 p);

#endif //SYMS_DWARF_EXPR_H
