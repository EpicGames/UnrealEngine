// Copyright Epic Games, Inc. All Rights Reserved.

#ifndef SYMS_DWARF_PARSER_C
#define SYMS_DWARF_PARSER_C

// TODO(rjf):
//
// [ ] Any time we encode a subrange of a section inside of a
//     SYMS_DwAttribValue, we need to do that consistently, regardless of
//     whether or not it is a string, memory block, etc. We should just use
//     the SYMS_DwSectionKind and then the min/max pair.
//
// [ ] Things we are not reporting, or haven't figured out:
//   @dwarf_expr @dwarf_v5 @dw_cross_unit
//   [ ] currently, we're filtering out template arguments in the member accelerator.
//       this is because they don't correspond one-to-one with anything in PDB, but
//       they do contain useful information that we might want to expose another way
//       somehow.
//   [ ] DWARF V5 features that nobody seems to use right now
//     [ ] ref_addr_desc + next_info_ctx
//         apparently these are necessary when dereferencing some DWARF V5 ways of
//         forming references. They don't seem to come up at all for any real data
//         but might be a case somewhere.
//     [ ] case when only .debug_line and .debug_line_str is available, without
//         compilation unit debug info? do we care about this at all?
//     [ ] SYMS_DwFormKind_REF_SIG8, which requires using .debug_names
//         to do a lookup for a reference
//     [ ] DWARF V5, but also V1 & V2 for syms_dw_range_list_from_range_offset
//   [ ] SYMS_DwAttribClass_RNGLIST and SYMS_DwFormKind_RNGLISTX
//   [ ] SYMS_DwOpCode_XDEREF_SIZE + SYMS_DwOpCode_XDEREF
//   [ ] SYMS_DwOpCode_PIECE + SYMS_DwOpCode_BIT_PIECE
//   [ ] SYMS_DwExtOpcode_DEFINE_FILE, for line info
//   [ ] DWARF procedures in DWARF expr evaluation
//   [ ] SYMS_DwAttribKind_DATA_MEMBER_LOCATION is not being *fully* handled right
//       now; full handling requires evaluating a DWARF expression to find out the
//       offset of a member. Right now we handle the common case, which is when it
//       is encoded as a constant value.
//   [ ] inline information
//   [ ] full info we are not handling:
//     [ ] friend classes
//     [ ] DWARF macro info
//     [ ] whether or not a function is the entry point
//   [ ] attributes we are not handling that may be important:
//     [ ] SYMS_DwAttribKind_ABSTRACT_ORIGIN
//       - ???
//     [ ] SYMS_DwAttribKind_VARIABLE_PARAMETER
//       - determines whether or not a parameter to a function is mutable, I think?
//     [ ] SYMS_DwAttribKind_MUTABLE
//       - I think this is for specific keywords, may not be relevant to C/++
//     [ ] SYMS_DwAttribKind_CALL_COLUMN
//       - column position of an inlined subroutine
//     [ ] SYMS_DwAttribKind_CALL_FILE
//       - file of inlined subroutine
//     [ ] SYMS_DwAttribKind_CALL_LINE
//       - line number of inlined subroutine
//     [ ] SYMS_DwAttribKind_CONST_EXPR
//       - ??? maybe C++ constexpr?
//     [ ] SYMS_DwAttribKind_ENUM_CLASS
//       - c++ thing that's an enum with a backing type
//     [ ] SYMS_DwAttribKind_LINKAGE_NAME
//       - name used to do linking

////////////////////////////////
//~ rjf: Globals

SYMS_READ_ONLY SYMS_GLOBAL SYMS_DwCompRoot syms_dw_nil_comp_root = {SYMS_DwCompUnitKind_RESERVED};

////////////////////////////////
//~ rjf: Basic Helpers

SYMS_API SYMS_U64
syms_dw_hash_from_string(SYMS_String8 string)
{
  SYMS_U64 result = 5381;
  for(SYMS_U64 i = 0; i < string.size; i += 1)
  {
    result = ((result << 5) + result) + string.str[i];
  }
  return result;
}

#if 1
SYMS_API SYMS_U64
syms_dw_hash_from_sid(SYMS_SymbolID id)
{
  SYMS_U64 low_bit = id & 1;
  SYMS_U64 chaotic_bits = id & 0x0000000000000007;
  SYMS_U64 hash = (id >> 3) | (chaotic_bits << 61);
  for(SYMS_U64 rot = 0; rot < chaotic_bits; rot += 1)
  {
    SYMS_U64 lower_portion = hash & 0x1FFFFFFFFFFFFFFF;
    SYMS_U64 low_bit = lower_portion & 1;
    SYMS_U64 lower_portion_rotated = (low_bit << 60) | (lower_portion >> 1);
    hash = (hash & 0xE000000000000000) | lower_portion_rotated;
  }
  hash = ((hash & 0xE000000000000000) >> 61) | ((hash & 0x1FFFFFFFFFFFFFFF) << 3);
  hash |= low_bit;
  return hash;
}
#elif 0
SYMS_API SYMS_U64
syms_dw_hash_from_sid(SYMS_SymbolID id)
{
  SYMS_U64 low_bit = id & 1;
  SYMS_U64 h = (id >> 4) << 1;
  h &= 0xfffffffffffffffe;
  h += low_bit;
  return h;
}
#elif 0
SYMS_API SYMS_U64
syms_dw_hash_from_sid(SYMS_SymbolID id)
{
  SYMS_U64 x = id;
  SYMS_U64 z = (x += 0x9e3779b97f4a7c15);
	z = (z ^ (z >> 30)) * 0xbf58476d1ce4e5b9;
	z = (z ^ (z >> 27)) * 0x94d049bb133111eb;
	return z ^ (z >> 31);
}
#else
SYMS_API SYMS_U64
syms_dw_hash_from_sid(SYMS_SymbolID id)
{
  SYMS_U64 key = id;
  key = ~key + (key << 21);
  key = (key << 21) - key - 1;
  key = key ^ (key >> 24);
  key = (key + (key << 3)) + (key << 8);
  key = key ^ (key >> 14);
  key = (key + (key << 2)) + (key << 4);
  key = key ^ (key >> 28);
  key = key + (key << 31);
  return key;
}
#endif

SYMS_API SYMS_SymbolID
syms_dw_sid_from_info_offset(SYMS_U64 info_offset)
{
  return info_offset;
}

SYMS_API SYMS_DwAttribClass
syms_dw_pick_attrib_value_class(SYMS_DwLanguage lang, SYMS_DwVersion ver, SYMS_DwAttribKind attrib, SYMS_DwFormKind form_kind)
{
  // NOTE(rjf): DWARF's spec specifies two mappings:
  // (SYMS_DwAttribKind) => List(SYMS_DwAttribClass)
  // (SYMS_DwFormKind)   => List(SYMS_DwAttribClass)
  //
  // This function's purpose is to find the overlapping class between an
  // SYMS_DwAttribKind and SYMS_DwFormKind.
  SYMS_DwAttribClass result = 0;
  SYMS_DwAttribClass attrib_info;
  SYMS_DwAttribClass form_info;

  switch(ver)
  {
    case SYMS_DwVersion_V1: break;
    case SYMS_DwVersion_V2:
    {
      attrib_info = syms_dw_attrib_class_from_attrib_kind_v2(attrib);
      form_info = syms_dw_attrib_class_from_form_kind_v2(form_kind);

      // rust compiler writes version 5 attributes
      if(lang == SYMS_DwLanguage_Rust && (attrib_info == 0 || form_info == 0))
      {
        attrib_info = syms_dw_attrib_class_from_attrib_kind(attrib);
        form_info = syms_dw_attrib_class_from_form_kind(form_kind);
      }
    } break;
    case SYMS_DwVersion_V3:
    case SYMS_DwVersion_V4:
    case SYMS_DwVersion_V5:
    {
      attrib_info = syms_dw_attrib_class_from_attrib_kind(attrib);
      form_info = syms_dw_attrib_class_from_form_kind(form_kind);
    } break;
  }

  if(attrib_info != 0 && form_info != 0)
  {
    result = SYMS_DwAttribClass_UNDEFINED;
    for(SYMS_U32 i = 0; i < 32; ++i)
    {
      SYMS_U32 n = 1u << i;
      if((attrib_info & n) != 0 && (form_info & n) != 0)
      {
        result = ((SYMS_DwAttribClass) n);
        break;
      }
    }
  }

  return result;
}

SYMS_API SYMS_SymbolKind
syms_dw_symbol_kind_from_tag_stub(SYMS_String8 data, SYMS_DwDbgAccel *dbg, SYMS_DwAttribValueResolveParams resolve_params, SYMS_DwTagStub *stub)
{
  SYMS_SymbolKind symbol_kind = SYMS_SymbolKind_Null;
  switch(stub->kind)
  {
    //- rjf: procs
    case SYMS_DwTagKind_INLINED_SUBROUTINE: {symbol_kind = SYMS_SymbolKind_Inline;}break;
    case SYMS_DwTagKind_SUBPROGRAM:
    {
      symbol_kind = SYMS_SymbolKind_Procedure;
    }break;
    
    //- rjf: constants
    case SYMS_DwTagKind_ENUMERATOR:
    {
      symbol_kind = SYMS_SymbolKind_Const;
    }break;
    
    //- rjf: types
    case SYMS_DwTagKind_STRUCTURE_TYPE:
    case SYMS_DwTagKind_UNION_TYPE:
    case SYMS_DwTagKind_CLASS_TYPE:
    case SYMS_DwTagKind_VOLATILE_TYPE:
    case SYMS_DwTagKind_ENUMERATION_TYPE:
    case SYMS_DwTagKind_TYPEDEF:
    case SYMS_DwTagKind_POINTER_TYPE:
    case SYMS_DwTagKind_ARRAY_TYPE:
    case SYMS_DwTagKind_REFERENCE_TYPE:
    case SYMS_DwTagKind_STRING_TYPE:
    case SYMS_DwTagKind_SUBROUTINE_TYPE:
    case SYMS_DwTagKind_PTR_TO_MEMBER_TYPE:
    case SYMS_DwTagKind_SUBRANGE_TYPE:
    case SYMS_DwTagKind_BASE_TYPE:
    case SYMS_DwTagKind_CONST_TYPE:
    case SYMS_DwTagKind_RVALUE_REFERENCE_TYPE:
    symbol_kind = SYMS_SymbolKind_Type; break;
    
    //- rjf: variables
    case SYMS_DwTagKind_FORMAL_PARAMETER:
    symbol_kind = SYMS_SymbolKind_LocalVariable; break;
    
    //- rjf: variables that could possibly be static
    case SYMS_DwTagKind_MEMBER:
    case SYMS_DwTagKind_VARIABLE:
    {
      SYMS_ArenaTemp scratch = syms_get_scratch(0, 0);
      SYMS_B32 has_external = syms_false;
      SYMS_B32 has_location = syms_false;
      SYMS_B32 has_const_val = syms_false;
      SYMS_DwAttribValue location_value = {SYMS_DwSectionKind_Null};
      SYMS_DwAttribList attribs = syms_dw_attrib_list_from_stub(scratch.arena, data, dbg, resolve_params.language, resolve_params.version, resolve_params.addr_size, stub);
      for(SYMS_DwAttribNode *attrib_n = attribs.first;
          attrib_n != 0;
          attrib_n = attrib_n->next)
      {
        SYMS_DwAttrib *attrib = &attrib_n->attrib;
        if(attrib->attrib_kind == SYMS_DwAttribKind_EXTERNAL)
        {
          has_external = syms_true;
        }
        else if(attrib->attrib_kind == SYMS_DwAttribKind_LOCATION)
        {
          has_location = syms_true;
          location_value = syms_dw_attrib_value_from_form_value(data, dbg, resolve_params,
                                                                attrib->form_kind,
                                                                attrib->value_class,
                                                                attrib->form_value);
        }
        else if(attrib->attrib_kind == SYMS_DwAttribKind_CONST_VALUE)
        {
          has_const_val = syms_true;
        }
      }
      
      SYMS_B32 location_is_fixed_address = syms_false;
      if(has_location)
      {
        void *info_base = syms_dw_sec_base_from_dbg(data, dbg, SYMS_DwSectionKind_Info);
        SYMS_U64Range loc_rng = syms_make_u64_range(location_value.v[0],
                                                    location_value.v[0] + location_value.v[1]);
        
        // TODO(allen): fill this in correctly
        SYMS_U64 text_base = 0;
        SYMS_DwSimpleLoc loc = syms_dw_expr__analyze_fast(info_base, loc_rng, text_base);
        location_is_fixed_address = (loc.kind == SYMS_DwSimpleLocKind_Address);
      }
      
      if(has_const_val)
      {
        symbol_kind = SYMS_SymbolKind_Const;
      }
      else if(location_is_fixed_address)
      {
        symbol_kind = SYMS_SymbolKind_ImageRelativeVariable;
      }
      else if(!has_external && !location_is_fixed_address)
      {
        symbol_kind = SYMS_SymbolKind_LocalVariable;
      }
      syms_release_scratch(scratch);
    }break;
    
    //- rjf: scopes
    case SYMS_DwTagKind_LEXICAL_BLOCK:
    symbol_kind = SYMS_SymbolKind_Scope; break;
    
    default: break;
  }
  return symbol_kind;
}

SYMS_API SYMS_SecInfoArray
syms_dw_copy_sec_info_array(SYMS_Arena *arena, SYMS_SecInfoArray array)
{
  SYMS_SecInfoArray result = {0};
  result.count = array.count;
  result.sec_info = syms_push_array_zero(arena, SYMS_SecInfo, result.count);
  for(SYMS_U64 idx = 0; idx < array.count; idx += 1)
  {
    result.sec_info[idx].vrange = array.sec_info[idx].vrange;
    result.sec_info[idx].frange = array.sec_info[idx].frange;
    result.sec_info[idx].name = syms_push_string_copy(arena, array.sec_info[idx].name);
  }
  return result;
}

SYMS_API SYMS_String8
syms_dw_path_from_dir_and_filename(SYMS_Arena *arena, SYMS_String8 dir, SYMS_String8 filename)
{
  SYMS_ArenaTemp scratch = syms_get_scratch(&arena, 1);
  SYMS_String8List pieces = {0};
  SYMS_U8 slash_char = '/';
  
  //- rjf: get pieces of full path from dir
  SYMS_B32 starts_with_slash = 0;
  {
    SYMS_U64 start = 0;
    for(SYMS_U64 idx = 0; idx <= dir.size; idx += 1)
    {
      if(idx == dir.size && idx > start)
      {
        SYMS_String8 str = syms_str8(dir.str+start, idx-start);
        syms_string_list_push(scratch.arena, &pieces, str);
      }
      else if(idx < dir.size && (dir.str[idx] == '/' || dir.str[idx] == '\\'))
      {
        if(idx == 0)
        {
          starts_with_slash = 1;
        }
        if(dir.str[idx] == '\\')
        {
          slash_char = '\\';
        }
        if(idx - start != 0)
        {
          SYMS_String8 str = syms_str8(dir.str+start, idx-start);
          syms_string_list_push(scratch.arena, &pieces, str);
        }
        start = idx+1;
      }
    }
  }
  
  //- rjf: get pieces of full path from filename
  {
    SYMS_U64 start = 0;
    for(SYMS_U64 idx = 0; idx <= filename.size; idx += 1)
    {
      if(idx == filename.size && idx > start)
      {
        SYMS_String8 str = syms_str8(filename.str+start, idx-start);
        syms_string_list_push(scratch.arena, &pieces, str);
      }
      else if(idx < filename.size && (filename.str[idx] == '/' || filename.str[idx] == '\\'))
      {
        if(idx - start != 0)
        {
          SYMS_String8 str = syms_str8(filename.str+start, idx-start);
          syms_string_list_push(scratch.arena, &pieces, str);
        }
        start = idx+1;
      }
    }
  }
  
  SYMS_String8 result = {0};
  SYMS_StringJoin join = {0};
  join.pre = starts_with_slash ? syms_str8_lit("/") : syms_str8_lit("");
  join.sep = syms_str8(&slash_char, 1);
  result = syms_string_list_join(arena, &pieces, &join);
  syms_release_scratch(scratch);
  return result;
}

SYMS_API void
syms_dw_symbol_id_chunk_list_push(SYMS_Arena *arena, SYMS_DwSymbolIDChunkList *list, SYMS_SymbolID sid)
{
  SYMS_DwSymbolIDChunk *chunk = list->last;
  if(chunk == 0 || chunk->count >= SYMS_ARRAY_SIZE(chunk->ids))
  {
    chunk = syms_push_array_zero(arena, SYMS_DwSymbolIDChunk, 1);
    SYMS_QueuePush(list->first, list->last, chunk);
    list->chunk_count += 1;
  }
  chunk->ids[chunk->count] = sid;
  chunk->count += 1;
  list->total_id_count += 1;
}

SYMS_API SYMS_SymbolIDArray
syms_dw_sid_array_from_chunk_list(SYMS_Arena *arena, SYMS_DwSymbolIDChunkList list)
{
  SYMS_SymbolIDArray array = {0};
  array.count = list.total_id_count;
  array.ids = syms_push_array(arena, SYMS_SymbolID, array.count);
  SYMS_U64 idx = 0;
  for(SYMS_DwSymbolIDChunk *chunk = list.first; chunk != 0; chunk = chunk->next)
  {
    syms_memmove(array.ids + idx, chunk->ids, chunk->count*sizeof(SYMS_SymbolID));
    idx += chunk->count;
  }
  return array;
}

////////////////////////////////
//~ rjf: DWARF-Specific Based Range Reads

SYMS_API SYMS_U64
syms_dw_based_range_read_length(void *base, SYMS_U64Range range, SYMS_U64 offset, SYMS_U64 *out_value)
{
  SYMS_U64 bytes_read = 0;
  SYMS_U64 value = 0;
  SYMS_U32 first32 = 0;
  if(syms_based_range_read_struct(base, range, offset, &first32))
  {
    // NOTE(rjf): DWARF 32-bit => use the first 32 bits as the size.
    if(first32 != 0xffffffff)
    {
      value = (SYMS_U64)first32;
      bytes_read = sizeof(SYMS_U32);
    }
    // NOTE(rjf): DWARF 64-bit => first 32 are just a marker, use the next 64 bits as the size.
    else if(syms_based_range_read_struct(base, range, offset + sizeof(SYMS_U32), &value))
    {
      value = 0;
      bytes_read = sizeof(SYMS_U32) + sizeof(SYMS_U64);
    }
  }
  if(out_value != 0)
  {
    *out_value = value;
  }
  return bytes_read;
}

SYMS_API SYMS_U64
syms_dw_based_range_read_abbrev_tag(void *base, SYMS_U64Range range, SYMS_U64 offset, SYMS_DwAbbrev *out_abbrev)
{
  SYMS_U64 total_bytes_read = 0;
  
  //- rjf: parse ID
  SYMS_U64 id_off = offset;
  SYMS_U64 sub_kind_off = id_off;
  SYMS_U64 id = 0;
  {
    SYMS_U64 bytes_read = syms_based_range_read_uleb128(base, range, id_off, &id);
    sub_kind_off += bytes_read;
    total_bytes_read += bytes_read;
  }
  
  //- rjf: parse sub-kind
  SYMS_U64 sub_kind = 0;
  SYMS_U64 next_off = sub_kind_off;
  if(id != 0)
  {
    SYMS_U64 bytes_read = syms_based_range_read_uleb128(base, range, sub_kind_off, &sub_kind);
    next_off += bytes_read;
    total_bytes_read += bytes_read;
  }
  
  //- rjf: parse whether this tag has children
  SYMS_U8 has_children = 0;
  if(id != 0)
  {
    total_bytes_read += syms_based_range_read_struct(base, range, next_off, &has_children);
  }
  
  //- rjf: fill abbrev
  if(out_abbrev != 0)
  {
    SYMS_DwAbbrev abbrev;
    syms_memzero_struct(&abbrev);
    abbrev.kind = SYMS_DwAbbrevKind_Tag;
    abbrev.abbrev_range = syms_make_u64_range(range.min+offset, range.min+offset+total_bytes_read);
    abbrev.sub_kind = sub_kind;
    abbrev.id = id;
    if(has_children)
    {
      abbrev.flags |= SYMS_DwAbbrevFlag_HasChildren;
    }
    *out_abbrev = abbrev;
  }
  
  return total_bytes_read;
}

SYMS_API SYMS_U64
syms_dw_based_range_read_abbrev_attrib_info(void *base, SYMS_U64Range range, SYMS_U64 offset, SYMS_DwAbbrev *out_abbrev)
{
  SYMS_U64 total_bytes_read = 0;
  
  //- rjf: parse ID
  SYMS_U64 id_off = offset;
  SYMS_U64 sub_kind_off = id_off;
  SYMS_U64 id = 0;
  {
    SYMS_U64 bytes_read = syms_based_range_read_uleb128(base, range, id_off, &id);
    sub_kind_off += bytes_read;
    total_bytes_read += bytes_read;
  }
  
  //- rjf: parse sub-kind (form-kind)
  SYMS_U64 sub_kind = 0;
  SYMS_U64 next_off = sub_kind_off;
  {
    SYMS_U64 bytes_read = syms_based_range_read_uleb128(base, range, sub_kind_off, &sub_kind);
    next_off += bytes_read;
    total_bytes_read += bytes_read;
  }
  
  //- rjf: parse implicit const
  SYMS_U64 implicit_const = 0;
  if(sub_kind == SYMS_DwFormKind_IMPLICIT_CONST)
  {
    SYMS_U64 bytes_read = syms_based_range_read_uleb128(base, range, next_off, &implicit_const);
    total_bytes_read += bytes_read;
  }
  
  //- rjf: fill abbrev
  if(out_abbrev != 0)
  {
    SYMS_DwAbbrev abbrev;
    syms_memzero_struct(&abbrev);
    abbrev.kind = SYMS_DwAbbrevKind_Attrib;
    abbrev.abbrev_range = syms_make_u64_range(offset, offset+total_bytes_read);
    abbrev.sub_kind = sub_kind;
    abbrev.id = id;
    if(sub_kind == SYMS_DwFormKind_IMPLICIT_CONST)
    {
      abbrev.flags |= SYMS_DwAbbrevFlag_HasImplicitConst;
      abbrev.const_value = implicit_const;
    }
    *out_abbrev = abbrev;
  }
  
  return total_bytes_read;
}

SYMS_API SYMS_U64
syms_dw_based_range_read_attrib_form_value(void *base, SYMS_U64Range range, SYMS_U64 offset, SYMS_DwMode mode, SYMS_U64 address_size, SYMS_DwFormKind form_kind, SYMS_U64 implicit_const, SYMS_DwAttribValue *form_value_out)
{
  SYMS_U64 bytes_read = 0;
  SYMS_U64 bytes_to_read = 0;
  SYMS_DwAttribValue form_value;
  syms_memzero_struct(&form_value);
  
  switch(form_kind)
  {
    
    //- rjf: 1-byte uint reads
    case SYMS_DwFormKind_REF1:  case SYMS_DwFormKind_DATA1: case SYMS_DwFormKind_FLAG:
    case SYMS_DwFormKind_STRX1: case SYMS_DwFormKind_ADDRX1:
    bytes_to_read = 1; goto read_fixed_uint;
    
    //- rjf: 2-byte uint reads
    case SYMS_DwFormKind_REF2: case SYMS_DwFormKind_DATA2: case SYMS_DwFormKind_STRX2:
    case SYMS_DwFormKind_ADDRX2:
    bytes_to_read = 2; goto read_fixed_uint;
    
    //- rjf: 3-byte uint reads
    case SYMS_DwFormKind_STRX3: case SYMS_DwFormKind_ADDRX3:
    bytes_to_read = 3; goto read_fixed_uint;
    
    //- rjf: 4-byte uint reads
    case SYMS_DwFormKind_DATA4: case SYMS_DwFormKind_REF4: case SYMS_DwFormKind_REF_SUP4: case SYMS_DwFormKind_STRX4: case SYMS_DwFormKind_ADDRX4:
    bytes_to_read = 4; goto read_fixed_uint;
    
    //- rjf: 8-byte uint reads
    case SYMS_DwFormKind_DATA8: case SYMS_DwFormKind_REF8: case SYMS_DwFormKind_REF_SIG8: case SYMS_DwFormKind_REF_SUP8:
    bytes_to_read = 8; goto read_fixed_uint;
    
    //- rjf: address-size reads
    case SYMS_DwFormKind_ADDR:       bytes_to_read = address_size; goto read_fixed_uint;
    
    //- rjf: offset-size reads
    case SYMS_DwFormKind_REF_ADDR: case SYMS_DwFormKind_SEC_OFFSET: case SYMS_DwFormKind_LINE_STRP:
    case SYMS_DwFormKind_STRP: case SYMS_DwFormKind_STRP_SUP:
    bytes_to_read = syms_dw_offset_size_from_mode(mode); goto read_fixed_uint;
    
    //- rjf: fixed-size uint reads
    {
      read_fixed_uint:;
      SYMS_U64 value = 0;
      bytes_read = syms_based_range_read(base, range, offset, bytes_to_read, &value);
      form_value.v[0] = value;
    }break;
    
    //- rjf: uleb128 reads
    case SYMS_DwFormKind_UDATA: case SYMS_DwFormKind_REF_UDATA: case SYMS_DwFormKind_STRX:
    case SYMS_DwFormKind_ADDRX: case SYMS_DwFormKind_LOCLISTX:  case SYMS_DwFormKind_RNGLISTX:
    {
      SYMS_U64 value = 0;
      bytes_read = syms_based_range_read_uleb128(base, range, offset, &value);
      form_value.v[0] = value;
    }break;
    
    //- rjf: sleb128 reads
    case SYMS_DwFormKind_SDATA:
    {
      SYMS_S64 value = 0;
      bytes_read = syms_based_range_read_sleb128(base, range, offset, &value);
      form_value.v[0] = value;
    }break;
    
    //- rjf: fixed-size uint read + skip
    case SYMS_DwFormKind_BLOCK1: bytes_to_read = 1; goto read_fixed_uint_skip;
    case SYMS_DwFormKind_BLOCK2: bytes_to_read = 2; goto read_fixed_uint_skip;
    case SYMS_DwFormKind_BLOCK4: bytes_to_read = 4; goto read_fixed_uint_skip;
    {
      read_fixed_uint_skip:;
      SYMS_U64 size = 0;
      bytes_read = syms_based_range_read(base, range, offset, bytes_to_read, &size);
      form_value.v[0] = size;
      form_value.v[1] = offset;
      bytes_read += size;
    }break;
    
    //- rjf: uleb 128 read + skip
    case SYMS_DwFormKind_BLOCK:
    {
      SYMS_U64 size = 0;
      bytes_read = syms_based_range_read_uleb128(base, range, offset, &size);
      form_value.v[0] = size;
      form_value.v[1] = offset;
      bytes_read += size;
    }break;
    
    //- rjf: u64 ranges
    case SYMS_DwFormKind_DATA16:
    {
      SYMS_U64 value1 = 0;
      SYMS_U64 value2 = 0;
      bytes_read += syms_based_range_read_struct(base, range, offset,                    &value1);
      bytes_read += syms_based_range_read_struct(base, range, offset + sizeof(SYMS_U64), &value2);
      form_value.v[0] = value1;
      form_value.v[1] = value2;
    }break;
    
    //- rjf: strings
    case SYMS_DwFormKind_STRING:
    {
      SYMS_String8 string = syms_based_range_read_string(base, range, offset);
      bytes_read = string.size + 1;
      SYMS_U64 string_offset = offset;
      SYMS_U64 string_size = (offset + bytes_read) - string_offset;
      form_value.v[0] = string_offset;
      form_value.v[1] = string_offset+string_size-1;
    }break;
    
    //- rjf: implicit const
    case SYMS_DwFormKind_IMPLICIT_CONST:
    {
      // NOTE(nick): This is special case.
      // Unlike other forms that have their values stored in the .debug_info section,
      // This one defines it's value in the .debug_abbrev section.
      form_value.v[0] = implicit_const;
    }break;
    
    //- rjf: expr loc
    case SYMS_DwFormKind_EXPRLOC:
    {
      SYMS_U64 size = 0;
      bytes_read = syms_based_range_read_uleb128(base, range, offset, &size);
      form_value.v[0] = offset + bytes_read;
      form_value.v[1] = size;
      bytes_read += size;
    }break;
    
    //- rjf: flag present
    case SYMS_DwFormKind_FLAG_PRESENT:
    {
      form_value.v[0] = 1;
    }break;
    
    case SYMS_DwFormKind_INDIRECT:
    {
      SYMS_INVALID_CODE_PATH;
    }break;
    
    case SYMS_DwFormKind_INVALID:
    {
    }break;
  }
  
  if(form_value_out != 0)
  {
    *form_value_out = form_value;
  }
  
  return bytes_read;
}

////////////////////////////////
//~ rjf: Debug Info Accelerator (DbgAccel) code

//- rjf: debug info and interaction with ELF file accel

SYMS_API SYMS_B32
syms_dw_elf_bin_accel_is_dbg(SYMS_ElfBinAccel *bin_accel)
{
  SYMS_B32 result = syms_false;
  SYMS_String8 required_sections[] =
  {
    syms_str8_comp(".debug_info"),
  };
  for(SYMS_U64 required_section_idx = 0;
      required_section_idx < SYMS_ARRAY_SIZE(required_sections);
      required_section_idx += 1)
  {
    SYMS_B32 found = syms_false;
    for(SYMS_U64 section_idx = 0;
        section_idx < bin_accel->sections.count;
        section_idx += 1)
    {
      if(syms_string_match(required_sections[required_section_idx], bin_accel->sections.v[section_idx].name, 0))
      {
        found = syms_true;
        break;
      }
    }
    if(found)
    {
      result = syms_true;
    }
    else
    {
      result = syms_false;
      break;
    }
  }
  return result;
}

SYMS_API SYMS_B32
syms_dw_mach_bin_accel_is_dbg(SYMS_MachBinAccel *bin_accel)
{
  SYMS_ArenaTemp scratch = syms_get_scratch(0,0);
  SYMS_B32 is_dbg = syms_false;
  SYMS_SecInfoArray sec_array = syms_mach_sec_info_array_from_bin(scratch.arena, syms_str8(0,0), bin_accel);
  for(SYMS_U64 i = 0; i < sec_array.count; i += 1)
  {
    if(syms_string_match(sec_array.sec_info[i].name, syms_str8_lit("__debug_info"), 0) ||
       syms_string_match(sec_array.sec_info[i].name, syms_str8_lit("__debug_line"), 0))
    {
      is_dbg = syms_true;
      break;
    }
  }
  syms_release_scratch(scratch);
  return is_dbg;
}

SYMS_API SYMS_DwDbgAccel *
syms_dw_dbg_accel_from_sec_info_array(SYMS_Arena *arena, SYMS_String8 data, SYMS_U64 vbase, SYMS_Arch arch, SYMS_SecInfoArray sections)
{
  //- rjf: scan file accel for info and abbrev sections, which are all that is necessary for
  // some useful debug info to come from DWARF.
  SYMS_B32 has_necessary_sections = syms_false;
  {
    SYMS_DwSectionKind necessary_kinds[] = 
    {
      SYMS_DwSectionKind_Info,
      SYMS_DwSectionKind_Abbrev
    };
    SYMS_U32 match_count = 0;
    for (SYMS_U64 kind_idx = 0; kind_idx < SYMS_ARRAY_SIZE(necessary_kinds); kind_idx += 1) 
    {
      SYMS_String8 dw_name = syms_dw_name_string_from_section_kind(necessary_kinds[kind_idx]);
      SYMS_String8 dwo_name = syms_dw_dwo_name_string_from_section_kind(necessary_kinds[kind_idx]);
      SYMS_String8 dw_mach_name = syms_dw_mach_name_string_from_section_kind(necessary_kinds[kind_idx]);
      for(SYMS_U64 section_idx = 0; section_idx < sections.count; section_idx += 1) 
      {
        if(syms_string_match(sections.sec_info[section_idx].name, dw_name, 0) ||
           syms_string_match(sections.sec_info[section_idx].name, dwo_name, 0) ||
           syms_string_match(sections.sec_info[section_idx].name, dw_mach_name, 0)) 
        {
          match_count += 1;
          break;
        }
      }
    }
    has_necessary_sections = (match_count == SYMS_ARRAY_SIZE(necessary_kinds));
  }
  
  //- rjf: scan sections for .text section, so we can store its index (useful for
  // later stuff in the expression transpiler)
  SYMS_U64 text_section_idx = 0;
  for(SYMS_U64 idx = 0; idx < sections.count; idx += 1)
  {
    if(syms_string_match(sections.sec_info[idx].name, syms_str8_lit(".text"), 0))
    {
      text_section_idx = idx;
      break;
    }
  }
  
  //- rjf: build section map
  SYMS_DwSection *section_map = 0;
  SYMS_B32 is_dwo = syms_false;
  if(has_necessary_sections)
  {
    section_map = syms_push_array_zero(arena, SYMS_DwSection, SYMS_DwSectionKind_COUNT);
    for(SYMS_DwSectionKind section_kind = (SYMS_DwSectionKind)(SYMS_DwSectionKind_Null+1);
        section_kind < SYMS_DwSectionKind_COUNT;
        section_kind = (SYMS_DwSectionKind)(section_kind + 1))
    {
      SYMS_String8 section_kind_name = syms_dw_name_string_from_section_kind(section_kind);
      SYMS_String8 dwo_section_kind_name = syms_dw_dwo_name_string_from_section_kind(section_kind);
      SYMS_String8 mach_kind_name = syms_dw_mach_name_string_from_section_kind(section_kind);
      for(SYMS_U64 section_idx = 0; section_idx < sections.count; section_idx += 1)
      {
        SYMS_B32 match_main = syms_string_match(sections.sec_info[section_idx].name, section_kind_name, 0);
        SYMS_B32 match_dwo = syms_string_match(sections.sec_info[section_idx].name, dwo_section_kind_name, 0);
        SYMS_B32 match_mach = syms_string_match(sections.sec_info[section_idx].name, mach_kind_name, 0);
        if(match_main || match_dwo || match_mach)
        {
          section_map[section_kind].range = sections.sec_info[section_idx].frange;
          if(match_dwo && section_kind == SYMS_DwSectionKind_Info)
          {
            is_dwo = syms_true;
          }
          break;
        }
      }
    }
  }
  
  //- rjf: find acceptable vrange for this file, to detect busted line info sections later
  SYMS_U64Range acceptable_vrange = {0};
  if(has_necessary_sections)
  {
    for(SYMS_U64 idx = 0; idx < sections.count; idx += 1)
    {
      acceptable_vrange.min = SYMS_MIN(acceptable_vrange.min, sections.sec_info[idx].vrange.min);
      acceptable_vrange.max = SYMS_MAX(acceptable_vrange.max, sections.sec_info[idx].vrange.max);
    }
  }
  
  //- rjf: find .debug_info and .debug_aranges for initial quick parse
  void *info_base = 0;
  SYMS_U64Range info_rng = {0};
  SYMS_DwMode aranges_mode = SYMS_DwMode_Null;
  void *aranges_base = 0;
  SYMS_U64Range aranges_rng = {0};
  if(has_necessary_sections)
  {
    info_base    = data.str + section_map[SYMS_DwSectionKind_Info].range.min;
    info_rng     = syms_make_u64_range(0, syms_u64_range_size(section_map[SYMS_DwSectionKind_Info].range));
    aranges_mode = syms_dw_mode_from_sec(section_map, SYMS_DwSectionKind_ARanges);
    aranges_base = data.str + section_map[SYMS_DwSectionKind_ARanges].range.min;
    aranges_rng  = syms_make_u64_range(0, syms_u64_range_size(section_map[SYMS_DwSectionKind_ARanges].range));
  }
  
  //- rjf: parse comp unit range info from .debug_info and .debug_aranges
  SYMS_U64 unit_count = 0;
  SYMS_DwUnitRangeInfo *unit_range_info = 0;
  if(has_necessary_sections)
  {
    SYMS_ArenaTemp scratch = syms_get_scratch(&arena, 1);
    
    //- rjf: get all comp unit file ranges
    SYMS_U64RangeList unit_franges = {0};
    {
      for(SYMS_U64 offset = 0;;)
      {
        SYMS_U64 info_size = 0;
        SYMS_U64 bytes_read = syms_dw_based_range_read_length(info_base, info_rng, offset, &info_size);
        if(info_size == 0)
        {
          break;
        }
        SYMS_U64 total_info_size = info_size + bytes_read;
        SYMS_U64 info_start = info_rng.min + offset;
        SYMS_U64Range range = syms_make_u64_range(info_start, info_start + total_info_size);
        syms_u64_range_list_push(scratch.arena, &unit_franges, range);
        offset += total_info_size;
      }
    }
    unit_count = unit_franges.node_count;
    
    //- rjf: get all comp unit addr ranges via .debug_aranges
    SYMS_U64RangeArray *unit_aranges = syms_push_array_zero(scratch.arena, SYMS_U64RangeArray, unit_count);
    {
      for(SYMS_U64 offset = 0; offset < aranges_rng.max;)
      {
        // rjf: parse hdr
        SYMS_U64 unit_length = 0;
        SYMS_U16 version = 0;
        SYMS_U64 debug_info_offset = 0;
        SYMS_U8 address_size = 0;
        SYMS_U8 segment_size = 0;
        offset += syms_dw_based_range_read_length(aranges_base, aranges_rng, offset, &unit_length);
        offset += syms_based_range_read_struct(aranges_base, aranges_rng, offset, &version);
        offset += syms_based_range_read(aranges_base, aranges_rng, offset, syms_dw_offset_size_from_mode(aranges_mode), &debug_info_offset);
        offset += syms_based_range_read_struct(aranges_base, aranges_rng, offset, &address_size);
        offset += syms_based_range_read_struct(aranges_base, aranges_rng, offset, &segment_size);
        
        // rjf: figure out which UID we're looking at
        SYMS_UnitID uid = 0;
        SYMS_U64 unit_idx = 0;
        {
          for(SYMS_U64RangeNode *n = unit_franges.first; n != 0; n = n->next, unit_idx += 1)
          {
            if(n->range.min <= debug_info_offset && debug_info_offset < n->range.max)
            {
              uid = unit_idx+1;
              break;
            }
          }
        }
        
        // rjf: figure out tuple size
        SYMS_U64 tuple_size = address_size*2 + segment_size;
        
        // rjf: adjust parsing offset to be divisible by the tuple size (see the February 13, 2017 spec)
        SYMS_U64 bytes_too_far_past_boundary = offset%tuple_size;
        if(bytes_too_far_past_boundary)
        {
          offset += (tuple_size - bytes_too_far_past_boundary);
        }
        
        // rjf: parse tuples
        SYMS_U64RangeList rngs = {0};
        {
          for(;;)
          {
            SYMS_U64 segment = 0;
            SYMS_U64 address = 0;
            SYMS_U64 length = 0;
            offset += syms_based_range_read(aranges_base, aranges_rng, offset, segment_size, &segment);
            offset += syms_based_range_read(aranges_base, aranges_rng, offset, address_size, &address);
            offset += syms_based_range_read(aranges_base, aranges_rng, offset, address_size, &length);
            if(segment == 0 && address == 0 && length == 0)
            {
              break;
            }
            if(length != 0 && acceptable_vrange.min <= address && address < acceptable_vrange.max)
            {
              SYMS_U64Range rng = syms_make_u64_range(address, address+length);
              syms_u64_range_list_push(scratch.arena, &rngs, rng);
            }
          }
        }
        
        // rjf: fill unit_aranges slot
        unit_aranges[unit_idx] = syms_u64_range_array_from_list(arena, &rngs);
        
        // rjf: log
#if SYMS_ENABLE_DEV_LOG
        SYMS_LogOpen(SYMS_LogFeature_DwarfUnitRanges, uid, rnglog);
        {
          SYMS_Log(".debug_aranges [unit #%i]:\n", uid);
          for(SYMS_U64 idx = 0; idx < unit_aranges[unit_idx].count; idx += 1)
          {
            SYMS_Log(" [0x%" SYMS_PRIx64 ", 0x%" SYMS_PRIx64 ")\n", unit_aranges[unit_idx].ranges[idx].min, unit_aranges[unit_idx].ranges[idx].max);
          }
        }
        SYMS_LogClose(rnglog);
#endif
      }
    }
    
    //- rjf: build range info array
    unit_range_info = syms_push_array_zero(arena, SYMS_DwUnitRangeInfo, unit_franges.node_count);
    {
      SYMS_U64RangeNode *frange_node = unit_franges.first;
      for(SYMS_U64 unit_idx = 0;
          unit_idx < unit_franges.node_count && frange_node != 0;
          unit_idx += 1, frange_node = frange_node->next)
      {
        unit_range_info[unit_idx].uid = unit_idx+1;
        unit_range_info[unit_idx].frange = frange_node->range;
        unit_range_info[unit_idx].addr_ranges = unit_aranges[unit_idx];
      }
    }
    
    syms_release_scratch(scratch);
  }
  
  //- rjf: fill+return
  SYMS_DwDbgAccel *dbg = (SYMS_DwDbgAccel*)&syms_format_nil;
  if(has_necessary_sections)
  {
    dbg = syms_push_array_zero(arena, SYMS_DwDbgAccel, 1);
    dbg->format = SYMS_FileFormat_DWARF;
    dbg->arch = arch;
    dbg->vbase = vbase;
    dbg->sections = syms_dw_copy_sec_info_array(arena, sections);
    dbg->text_section_idx = text_section_idx;
    dbg->acceptable_vrange = acceptable_vrange;
    dbg->section_map = section_map;
    dbg->unit_count = unit_count;
    dbg->unit_range_info = unit_range_info;
    dbg->is_dwo = is_dwo;
  }
  return dbg;
}

SYMS_API SYMS_DwDbgAccel *
syms_dw_dbg_accel_from_elf_bin(SYMS_Arena *arena, SYMS_String8 data, SYMS_ElfBinAccel *bin)
{
  SYMS_ArenaTemp scratch = syms_get_scratch(&arena, 1);
  SYMS_SecInfoArray sec_info_array = syms_elf_sec_info_array_from_bin(scratch.arena, data, bin);
  SYMS_DwDbgAccel *dbg = syms_dw_dbg_accel_from_sec_info_array(arena, data, syms_elf_default_vbase_from_bin(bin), bin->header.arch, sec_info_array);
  syms_release_scratch(scratch);
  return dbg;
}

SYMS_API SYMS_DwDbgAccel *
syms_dw_dbg_accel_from_mach_bin(SYMS_Arena *arena, SYMS_String8 data, SYMS_MachBinAccel *bin)
{
  SYMS_ArenaTemp scratch = syms_get_scratch(&arena, 1);
  SYMS_SecInfoArray sec_info_array = syms_mach_sec_info_array_from_bin(scratch.arena, data, bin);
  SYMS_DwDbgAccel *dbg = syms_dw_dbg_accel_from_sec_info_array(arena, data, syms_mach_default_vbase_from_bin(bin), bin->arch, sec_info_array);
  syms_release_scratch(scratch);
  return dbg;
}

SYMS_API SYMS_ExtFileList
syms_dw_ext_file_list_from_dbg(SYMS_Arena *arena, SYMS_String8 data, SYMS_DwDbgAccel *dbg)
{
  SYMS_ExtFileList list;
  syms_memzero_struct(&list);
  SYMS_ArenaTemp scratch = syms_get_scratch(&arena, 1);
  SYMS_DwUnitSetAccel *unit_set = syms_dw_unit_set_accel_from_dbg(scratch.arena, data, dbg);
  SYMS_U64 count = syms_dw_unit_count_from_set(unit_set);
  for(SYMS_U64 unit_num = 1; unit_num <= count; unit_num += 1)
  {
    SYMS_UnitID uid = syms_dw_uid_from_number(unit_set, unit_num);
    SYMS_DwCompRoot *root = syms_dw_comp_root_from_uid(unit_set, uid);
    SYMS_DwExtDebugRef ext_ref = syms_dw_ext_debug_ref_from_comp_root(root);
    if(ext_ref.dwo_path.size != 0)
    {
      SYMS_ExtFileNode *node = syms_push_array(arena, SYMS_ExtFileNode, 1);
      node->ext_file.file_name = syms_push_string_copy(arena, ext_ref.dwo_path);
      syms_memzero_struct(&node->ext_file.match_key);
      syms_memmove(node->ext_file.match_key.v, &ext_ref.dwo_id, sizeof(SYMS_U64));
      SYMS_QueuePush(list.first, list.last, node);
      list.node_count += 1;
    }
  }
  syms_release_scratch(scratch);
  return list;
}

SYMS_API SYMS_SecInfoArray
syms_dw_sec_info_array_from_dbg(SYMS_Arena *arena, SYMS_String8 data, SYMS_DwDbgAccel *dbg)
{
  SYMS_SecInfoArray result = syms_dw_copy_sec_info_array(arena, dbg->sections);
  return result;
}

SYMS_API SYMS_ExtMatchKey
syms_dw_ext_match_key_from_dbg(SYMS_String8 data, SYMS_DwDbgAccel *dbg)
{
  SYMS_ExtMatchKey key;
  syms_memzero_struct(&key);
  if(dbg->is_dwo)
  {
    SYMS_ArenaTemp scratch = syms_get_scratch(0, 0);
    SYMS_ExtFileList ext_files = syms_dw_ext_file_list_from_dbg(scratch.arena, data, dbg);
    if(ext_files.node_count != 0)
    {
      SYMS_ExtFileNode *n = ext_files.first;
      key = n->ext_file.match_key;;
    }
    syms_arena_temp_end(scratch);
  }
  else
  {
    SYMS_U32 checksum = syms_elf_gnu_debuglink_crc32(0, data);
    syms_memmove(key.v, &checksum, sizeof(checksum));
  }
  return key;
}

SYMS_API SYMS_U64
syms_dw_default_vbase_from_dbg(SYMS_DwDbgAccel *dbg)
{
  return dbg->vbase;
}

//- rjf: top-level unit info

SYMS_API SYMS_UnitID
syms_dw_uid_from_foff(SYMS_DwDbgAccel *dbg, SYMS_U64 foff)
{
  SYMS_UnitID uid = 0;
  for(SYMS_U64 unit_idx = 0; unit_idx < dbg->unit_count; unit_idx += 1)
  {
    if(dbg->unit_range_info[unit_idx].frange.min <= foff && foff < dbg->unit_range_info[unit_idx].frange.max)
    {
      uid = unit_idx+1;
      break;
    }
  }
  return uid;
}

//- rjf: important DWARF section base/range accessors

SYMS_API SYMS_DwMode
syms_dw_mode_from_sec(SYMS_DwSection *section_map, SYMS_DwSectionKind kind)
{
  SYMS_U64Range range = section_map[kind].range;
  SYMS_U64 size = range.max - range.min;
  if(size > 0xffffffff)
  {
    return SYMS_DwMode_64Bit;
  }
  else
  {
    return SYMS_DwMode_32Bit;
  }
}

SYMS_API SYMS_B32
syms_dw_sec_is_present(SYMS_DwDbgAccel *dbg, SYMS_DwSectionKind kind)
{
  SYMS_U64Range *range = &dbg->section_map[kind].range;
  SYMS_B32 result = (range->max > range->min);
  return(result);
}

SYMS_API void *
syms_dw_sec_base_from_dbg(SYMS_String8 data, SYMS_DwDbgAccel *dbg, SYMS_DwSectionKind kind)
{
  return data.str + dbg->section_map[kind].range.min;
}

SYMS_API SYMS_U64Range
syms_dw_sec_range_from_dbg(SYMS_DwDbgAccel *dbg, SYMS_DwSectionKind kind)
{
  return syms_make_u64_range(0, syms_u64_range_size(dbg->section_map[kind].range));
}

////////////////////////////////
//~ rjf: Abbrev Table

SYMS_API SYMS_DwAbbrevTable
syms_dw_make_abbrev_table(SYMS_Arena *arena, SYMS_String8 data, SYMS_DwDbgAccel *dbg, SYMS_U64 abbrev_offset)
{
  void *file_base = syms_dw_sec_base_from_dbg(data, dbg, SYMS_DwSectionKind_Abbrev);
  SYMS_U64Range abbrev_range = syms_dw_sec_range_from_dbg(dbg, SYMS_DwSectionKind_Abbrev);
  
  SYMS_DwAbbrevTable table;
  syms_memzero_struct(&table);
  
  //- rjf: count the tags we have
  SYMS_U64 tag_count = 0;
  for(SYMS_U64 abbrev_read_off = abbrev_offset - abbrev_range.min;;)
  {
    SYMS_DwAbbrev tag;
    {
      SYMS_U64 bytes_read = syms_dw_based_range_read_abbrev_tag(file_base, abbrev_range, abbrev_read_off, &tag);
      abbrev_read_off += bytes_read;
      if(bytes_read == 0 || tag.id == 0)
      {
        break;
      }
    }
    for(;;)
    {
      SYMS_DwAbbrev attrib;
      syms_memzero_struct(&attrib);
      SYMS_U64 bytes_read = syms_dw_based_range_read_abbrev_attrib_info(file_base, abbrev_range, abbrev_read_off, &attrib);
      abbrev_read_off += bytes_read;
      if(bytes_read == 0 || attrib.id == 0)
      {
        break;
      }
    }
    tag_count += 1;
  }
  
  //- rjf: build table
  table.count = tag_count;
  table.entries = syms_push_array(arena, SYMS_DwAbbrevTableEntry, table.count);
  syms_memset(table.entries, 0, sizeof(SYMS_DwAbbrevTableEntry)*table.count);
  SYMS_U64 tag_idx = 0;
  for(SYMS_U64 abbrev_read_off = abbrev_offset - abbrev_range.min;;)
  {
    SYMS_DwAbbrev tag;
    {
      SYMS_U64 bytes_read = syms_dw_based_range_read_abbrev_tag(file_base, abbrev_range, abbrev_read_off, &tag);
      abbrev_read_off += bytes_read;
      if(bytes_read == 0 || tag.id == 0)
      {
        break;
      }
    }
    
    // rjf: insert this tag into the table
    {
      table.entries[tag_idx].id = tag.id;
      table.entries[tag_idx].off = tag.abbrev_range.min;
      tag_idx += 1;
    }
    
    for(;;)
    {
      SYMS_DwAbbrev attrib;
      syms_memzero_struct(&attrib);
      SYMS_U64 bytes_read = syms_dw_based_range_read_abbrev_attrib_info(file_base, abbrev_range, abbrev_read_off, &attrib);
      abbrev_read_off += bytes_read;
      if(bytes_read == 0 || attrib.id == 0)
      {
        break;
      }
    }
    tag_count += 1;
  }
  
  return table;
}

SYMS_API SYMS_U64
syms_dw_abbrev_offset_from_abbrev_id(SYMS_DwAbbrevTable table, SYMS_U64 abbrev_id)
{
  SYMS_U64 abbrev_offset = SYMS_U64_MAX;
  if(table.count > 0)
  {
    SYMS_S64 min = 0;
    SYMS_S64 max = (SYMS_S64)table.count - 1;
    while(min <= max)
    {
      SYMS_S64 mid = (min + max) / 2;
      if (abbrev_id > table.entries[mid].id)
      {
        min = mid + 1;
      }
      else if (abbrev_id < table.entries[mid].id)
      {
        max = mid - 1;
      }
      else
      {
        abbrev_offset = table.entries[mid].off;
        break;
      }
    }
  }
  return abbrev_offset;
}

////////////////////////////////
//~ rjf: Miscellaneous DWARF Section Parsing

//- rjf: .debug_ranges (DWARF V4)

SYMS_API SYMS_U64RangeList
syms_dw_v4_range_list_from_range_offset(SYMS_Arena *arena, SYMS_String8 data, SYMS_DwDbgAccel *dbg, SYMS_U64 addr_size, SYMS_U64 comp_unit_base_addr, SYMS_U64 range_off)
{
  void *base = syms_dw_sec_base_from_dbg(data, dbg, SYMS_DwSectionKind_Ranges);
  SYMS_U64Range rng = syms_dw_sec_range_from_dbg(dbg, SYMS_DwSectionKind_Ranges);
  
  SYMS_U64RangeList list;
  syms_memzero_struct(&list);
  
  SYMS_U64 read_off = range_off;
  SYMS_U64 base_addr = comp_unit_base_addr;
  
  for(;read_off < rng.max;)
  {
    SYMS_U64 v0 = 0;
    SYMS_U64 v1 = 0;
    read_off += syms_based_range_read(base, rng, read_off, addr_size, &v0);
    read_off += syms_based_range_read(base, rng, read_off, addr_size, &v1);
    
    //- rjf: base address entry
    if((addr_size == 4 && v0 == 0xffffffff) ||
       (addr_size == 8 && v0 == 0xffffffffffffffff))
    {
      base_addr = v1;
    }
    //- rjf: end-of-list entry
    else if(v0 == 0 && v1 == 0)
    {
      break;
    }
    //- rjf: range list entry
    else
    {
      SYMS_U64 min_addr = v0 + base_addr;
      SYMS_U64 max_addr = v1 + base_addr;
      syms_u64_range_list_push(arena, &list, syms_make_u64_range(min_addr, max_addr));
    }
  }
  
  return list;
}

//- rjf: .debug_loc (DWARF V4)

SYMS_API SYMS_LocRangeList
syms_dw_v4_location_ranges_from_loc_offset(SYMS_Arena *arena, SYMS_String8 data, SYMS_DwDbgAccel *dbg, SYMS_U64 addr_size, SYMS_U64 comp_unit_base_addr, SYMS_U64 offset)
{
  void *base = syms_dw_sec_base_from_dbg(data, dbg, SYMS_DwSectionKind_Loc);
  SYMS_U64Range rng = syms_dw_sec_range_from_dbg(dbg, SYMS_DwSectionKind_Loc);
  
  SYMS_LocRangeList list = {0};
  
  SYMS_U64 read_off = offset;
  SYMS_U64 base_addr = comp_unit_base_addr;
  
  for(;read_off < rng.max;)
  {
    SYMS_U64 v0 = 0;
    SYMS_U64 v1 = 0;
    read_off += syms_based_range_read(base, rng, read_off, addr_size, &v0);
    read_off += syms_based_range_read(base, rng, read_off, addr_size, &v1);
    
    //- rjf: base address entry
    if((addr_size == 4 && v0 == 0xffffffff) ||
       (addr_size == 8 && v0 == 0xffffffffffffffff))
    {
      base_addr = v1;
    }
    //- rjf: end-of-list entry
    else if(v0 == 0 && v1 == 0)
    {
      break;
    }
    //- rjf: location list entry
    else
    {
      SYMS_U64 start_addr = v0 + base_addr;
      SYMS_U64 end_addr = v1 + base_addr;
      
      // rjf: push to list
      SYMS_LocRangeNode *n = syms_push_array_zero(arena, SYMS_LocRangeNode, 1);
      n->loc_range.vrange = syms_make_u64_range(start_addr, end_addr);
      n->loc_range.loc_id = (SYMS_LocID)read_off;
      SYMS_QueuePush(list.first, list.last, n);
      list.count += 1;
      
      // rjf: skip past location description
      SYMS_U16 locdesc_length = 0;
      read_off += syms_based_range_read_struct(base, rng, read_off, &locdesc_length);
      read_off += locdesc_length;
    }
  }
  
  return list;
}

SYMS_API SYMS_Location
syms_dw_v4_location_from_loc_id(SYMS_Arena *arena, SYMS_String8 data, SYMS_DwDbgAccel *dbg, SYMS_LocID id)
{
  SYMS_Location result = {0};
  
  void *base = syms_dw_sec_base_from_dbg(data, dbg, SYMS_DwSectionKind_Loc);
  SYMS_U64Range sec_range = syms_dw_sec_range_from_dbg(dbg, SYMS_DwSectionKind_Loc);
  SYMS_U64 read_off = (SYMS_U64)id;
  SYMS_U16 length = 0;
  read_off += syms_based_range_read_struct(base, sec_range, read_off, &length);
  void *expr_base = syms_based_range_ptr(base, sec_range, read_off);
  SYMS_U64Range expr_range = syms_make_u64_range(read_off, read_off+length);
  SYMS_String8 location = syms_dw_expr__transpile_to_eval(arena, dbg, base, expr_range);
  
  // TODO(rjf): wire up `location` to `result`, once SYMS_Location changes to
  // returning a SYMS_String8 instead of an op list.
  
  return result;
}

//- rjf: .debug_pubtypes + .debug_pubnames (DWARF V4)

SYMS_API SYMS_DwPubStringsTable
syms_dw_v4_pub_strings_table_from_section_kind(SYMS_Arena *arena, SYMS_String8 data, SYMS_DwDbgAccel *dbg, SYMS_DwSectionKind section_kind)
{
  SYMS_DwPubStringsTable names_table;
  syms_memzero_struct(&names_table);
  
  SYMS_ArenaTemp scratch = syms_get_scratch(&arena, 1);
  
  // TODO(rjf): Arbitrary choice.
  names_table.size = 16384;
  names_table.buckets = syms_push_array_zero(arena, SYMS_DwPubStringsBucket*, names_table.size);
  
  SYMS_DwMode mode = syms_dw_mode_from_sec(dbg->section_map, section_kind);
  void *base = syms_dw_sec_base_from_dbg(data, dbg, section_kind);
  SYMS_U64Range rng = syms_dw_sec_range_from_dbg(dbg, section_kind);
  SYMS_U64 read_off = 0;
  
  SYMS_U64 off_size = syms_dw_offset_size_from_mode(mode);
  
  SYMS_U64 table_length = 0;
  SYMS_U16 unit_version = 0;
  SYMS_U64 cu_info_off = 0;
  SYMS_U64 cu_info_len = 0;
  SYMS_U64 cu_num = 0;
  read_off += syms_dw_based_range_read_length(base, rng, read_off, &table_length);
  read_off += syms_based_range_read_struct(base, rng, read_off, &unit_version);
  read_off += syms_based_range_read(base, rng, read_off, off_size, &cu_info_off);
  read_off += syms_dw_based_range_read_length(base, rng, read_off, &cu_info_len);
  cu_num = syms_dw_uid_from_foff(dbg, cu_info_off);
  
  for(;;)
  {
    SYMS_U64 info_off = 0;
    {
      SYMS_U64 bytes_read = syms_based_range_read(base, rng, read_off, off_size, &info_off);
      read_off += bytes_read;
      if(bytes_read == 0)
      {
        break;
      }
    }
    
    //- rjf: if we got a nonzero .debug_info offset, we've found a valid entry.
    if(info_off != 0)
    {
      SYMS_String8 string = syms_based_range_read_string(base, rng, read_off);
      read_off += string.size + 1;
      SYMS_U64 hash = syms_dw_hash_from_string(string);
      SYMS_U64 bucket_idx = hash % names_table.size;
      SYMS_DwPubStringsBucket *bucket = syms_push_array(arena, SYMS_DwPubStringsBucket, 1);
      bucket->next = names_table.buckets[bucket_idx];
      bucket->string = string;
      bucket->sid = syms_dw_sid_from_info_offset(info_off);
      bucket->uid = cu_num;
      names_table.buckets[bucket_idx] = bucket;
    }
    
    //- rjf: if we did not read a proper entry in the table, we need to try to
    // read the header of the next table.
    else
    {
      SYMS_U64 next_table_length = 0;
      {
        SYMS_U64 bytes_read = syms_dw_based_range_read_length(base, rng, read_off, &next_table_length);
        if(bytes_read == 0 || next_table_length == 0)
        {
          break;
        }
        read_off += bytes_read;
      }
      read_off += syms_based_range_read_struct(base, rng, read_off, &unit_version);
      read_off += syms_based_range_read(base, rng, read_off, off_size, &cu_info_off);
      read_off += syms_dw_based_range_read_length(base, rng, read_off, &cu_info_len);
      cu_num = syms_dw_uid_from_foff(dbg, cu_info_off);
    }
  }
  
  syms_release_scratch(scratch);
  
  return names_table;
}

SYMS_API SYMS_USIDList
syms_dw_v4_usid_list_from_pub_table_string(SYMS_Arena *arena, SYMS_DwPubStringsTable tbl, SYMS_String8 string)
{
  SYMS_USIDList list = {0};
  SYMS_U64 hash = syms_dw_hash_from_string(string);
  SYMS_U64 idx = hash % tbl.size;
  for(SYMS_DwPubStringsBucket *bucket = tbl.buckets[idx]; bucket; bucket = bucket->next)
  {
    if(syms_string_match(bucket->string, string, 0))
    {
      SYMS_USIDNode *n = syms_push_array(arena, SYMS_USIDNode, 1);
      n->usid.uid = bucket->uid;
      n->usid.sid = bucket->sid;
      SYMS_QueuePush(list.first, list.last, n);
      list.count += 1;
    }
  }
  return list;
}

//- rjf: .debug_str_offsets (DWARF V5)

SYMS_API SYMS_U64
syms_dw_v5_offset_from_offs_section_base_index(SYMS_String8 data, SYMS_DwDbgAccel *dbg, SYMS_DwSectionKind section, SYMS_U64 base, SYMS_U64 index)
{
  SYMS_U64 result = 0;
  
  SYMS_DwMode mode = syms_dw_mode_from_sec(dbg->section_map, section);
  void *sec_base = syms_dw_sec_base_from_dbg(data, dbg, section);
  SYMS_U64Range rng = syms_dw_sec_range_from_dbg(dbg, section);
  SYMS_U64 read_off = base;
  
  //- rjf: get the length of each entry
  SYMS_U64 entry_len = mode == SYMS_DwMode_64Bit ? 8 : 4;
  
  //- rjf: parse the unit's length (not including the length itself)
  SYMS_U64 unit_length = 0;
  read_off += syms_dw_based_range_read_length(sec_base, rng, read_off, &unit_length);
  
  //- rjf: parse version
  SYMS_U16 version = 0;
  read_off += syms_based_range_read_struct(sec_base, rng, read_off, &version);
  SYMS_ASSERT_PARANOID(version == 5); // must be 5 as of V5.
  
  //- rjf: parse padding
  SYMS_U16 padding = 0;
  read_off += syms_based_range_read_struct(sec_base, rng, read_off, &padding);
  SYMS_ASSERT_PARANOID(padding == 0); // must be 0 as of V5.
  
  //- rjf: read
  if (unit_length >= sizeof(SYMS_U16)*2) 
  {
    void *entries = (SYMS_U8 *)sec_base + read_off;
    SYMS_U64 count = (unit_length - sizeof(SYMS_U16)*2) / entry_len;
    if(0 <= index && index < count)
    {
      switch(entry_len)
      {
        default: break;
        case 4: {result = ((SYMS_U32 *)entries)[index];}break;
        case 8: {result = ((SYMS_U64 *)entries)[index];}break;
      }
    }
  }
  
  return result;
}

//- rjf: .debug_addr parsing

SYMS_API SYMS_U64
syms_dw_v5_addr_from_addrs_section_base_index(SYMS_String8 data, SYMS_DwDbgAccel *dbg, SYMS_DwSectionKind section, SYMS_U64 base, SYMS_U64 index)
{
  SYMS_U64 result = 0;
  
  void *sec_base = syms_dw_sec_base_from_dbg(data, dbg, section);
  SYMS_U64Range rng = syms_dw_sec_range_from_dbg(dbg, section);
  SYMS_U64 read_off = base;
  
  //- rjf: parse the unit's length (not including the length itself)
  SYMS_U64 unit_length = 0;
  read_off += syms_dw_based_range_read_length(sec_base, rng, read_off, &unit_length);
  
  //- rjf: parse version
  SYMS_U16 version = 0;
  read_off += syms_based_range_read_struct(sec_base, rng, read_off, &version);
  SYMS_ASSERT_PARANOID(version == 5); // must be 5 as of V5.
  
  //- rjf: parse address size
  SYMS_U8 address_size = 0;
  read_off += syms_based_range_read_struct(sec_base, rng, read_off, &address_size);
  
  //- rjf: parse segment selector size
  SYMS_U8 segment_selector_size = 0;
  read_off += syms_based_range_read_struct(sec_base, rng, read_off, &segment_selector_size);
  
  //- rjf: read
  SYMS_U64 entry_size = address_size + segment_selector_size;
  SYMS_U64 count = (unit_length - sizeof(SYMS_U16)*2) / entry_size;
  if(0 <= index && index < count)
  {
    void *entry = (SYMS_U8 *)syms_based_range_ptr(sec_base, rng, read_off) + entry_size*index;
    SYMS_U64Range entry_rng = syms_make_u64_range(0, entry_size);
    SYMS_U64 segment = 0;
    SYMS_U64 addr = 0;
    syms_based_range_read(entry, entry_rng, 0,                     sizeof(segment), &segment);
    syms_based_range_read(entry, entry_rng, segment_selector_size, sizeof(addr),    &addr);
    result = addr;
  }
  
  return result;
}

//- rjf: .debug_rnglists + .debug_loclists parsing

SYMS_API SYMS_U64
syms_dw_v5_sec_offset_from_rnglist_or_loclist_section_base_index(SYMS_String8 data, SYMS_DwDbgAccel *dbg, SYMS_DwSectionKind section_kind, SYMS_U64 base, SYMS_U64 index)
{
  //
  // NOTE(rjf): This is only appropriate to call when SYMS_DwFormKind_RNGLISTX is
  // used to access a range list, *OR* when SYMS_DwFormKind_LOCLISTX is used to
  // access a location list. Otherwise, SYMS_DwFormKind_SEC_OFFSET is required.
  //
  // See the DWARF V5 spec (February 13, 2017), page 242. (rnglists)
  // See the DWARF V5 spec (February 13, 2017), page 215. (loclists)
  //
  
  SYMS_U64 result = 0;
  SYMS_DwMode mode = syms_dw_mode_from_sec(dbg->section_map, section_kind);
  void *sec_base = syms_dw_sec_base_from_dbg(data, dbg, section_kind);
  SYMS_U64Range rng = syms_dw_sec_range_from_dbg(dbg, section_kind);
  SYMS_U64 read_off = base;
  
  //- rjf: get the length of each entry
  SYMS_U64 entry_len = mode == SYMS_DwMode_64Bit ? 8 : 4;
  
  //- rjf: parse the unit's length (not including the length itself)
  SYMS_U64 unit_length = 0;
  read_off += syms_dw_based_range_read_length(sec_base, rng, read_off, &unit_length);
  
  //- rjf: parse version
  SYMS_U16 version = 0;
  read_off += syms_based_range_read_struct(sec_base, rng, read_off, &version);
  SYMS_ASSERT_PARANOID(version == 5); // must be 5 as of V5.
  
  //- rjf: parse address size
  SYMS_U8 address_size = 0;
  read_off += syms_based_range_read_struct(sec_base, rng, read_off, &address_size);
  
  //- rjf: parse segment selector size
  SYMS_U8 segment_selector_size = 0;
  read_off += syms_based_range_read_struct(sec_base, rng, read_off, &segment_selector_size);
  
  //- rjf: parse offset entry count
  SYMS_U32 offset_entry_count = 0;
  read_off += syms_based_range_read_struct(sec_base, rng, read_off, &offset_entry_count);
  
  //- rjf: read from offsets array
  SYMS_U64 table_off = read_off;
  void *offsets_arr = syms_based_range_ptr(sec_base, rng, read_off);
  if(0 <= index && index < (SYMS_U64)offset_entry_count)
  {
    SYMS_U64 rnglist_offset = 0;
    switch(entry_len)
    {
      default: break;
      case 4: rnglist_offset = ((SYMS_U32 *)offsets_arr)[index]; break;
      case 8: rnglist_offset = ((SYMS_U64 *)offsets_arr)[index]; break;
    }
    result = rnglist_offset+table_off;
  }
  
  return result;
}

SYMS_API SYMS_U64RangeList
syms_dw_v5_range_list_from_rnglist_offset(SYMS_Arena *arena, SYMS_String8 data, SYMS_DwDbgAccel *dbg, SYMS_DwSectionKind section, SYMS_U64 addr_size, SYMS_U64 addr_section_base, SYMS_U64 offset)
{
  SYMS_U64RangeList list = {0};
  
  SYMS_U64 read_off = offset;
  void *base = syms_dw_sec_base_from_dbg(data, dbg, section);
  SYMS_U64Range rng = syms_dw_sec_range_from_dbg(dbg, section);
  
  SYMS_U64 base_addr = 0;
  
  for(SYMS_B32 done = syms_false; !done;)
  {
    SYMS_U8 kind8 = 0;
    read_off += syms_based_range_read_struct(base, rng, read_off, &kind8);
    SYMS_DwRngListEntryKind kind = (SYMS_DwRngListEntryKind)kind8;
    
    switch(kind)
    {
      //- rjf: can be used in split and non-split units:
      default:
      case SYMS_DwRngListEntryKind_EndOfList:
      {
        done = syms_true;
      }break;
      
      case SYMS_DwRngListEntryKind_BaseAddressX:
      {
        SYMS_U64 base_addr_idx = 0;
        read_off += syms_based_range_read_uleb128(base, rng, read_off, &base_addr_idx);
        base_addr = syms_dw_v5_addr_from_addrs_section_base_index(data, dbg, SYMS_DwSectionKind_Addr, addr_section_base, base_addr_idx);
      }break;
      
      case SYMS_DwRngListEntryKind_StartxEndx:
      {
        SYMS_U64 start_addr_idx = 0;
        SYMS_U64 end_addr_idx = 0;
        read_off += syms_based_range_read_uleb128(base, rng, read_off, &start_addr_idx);
        read_off += syms_based_range_read_uleb128(base, rng, read_off, &end_addr_idx);
        SYMS_U64 start_addr = syms_dw_v5_addr_from_addrs_section_base_index(data, dbg, SYMS_DwSectionKind_Addr, addr_section_base, start_addr_idx);
        SYMS_U64 end_addr = syms_dw_v5_addr_from_addrs_section_base_index(data, dbg, SYMS_DwSectionKind_Addr, addr_section_base, end_addr_idx);
        syms_u64_range_list_push(arena, &list, syms_make_u64_range(start_addr, end_addr));
      }break;
      
      case SYMS_DwRngListEntryKind_StartxLength:
      {
        SYMS_U64 start_addr_idx = 0;
        SYMS_U64 length = 0;
        read_off += syms_based_range_read_uleb128(base, rng, read_off, &start_addr_idx);
        read_off += syms_based_range_read_uleb128(base, rng, read_off, &length);
        SYMS_U64 start_addr = syms_dw_v5_addr_from_addrs_section_base_index(data, dbg, SYMS_DwSectionKind_Addr, addr_section_base, start_addr_idx);
        SYMS_U64 end_addr = start_addr + length;
        syms_u64_range_list_push(arena, &list, syms_make_u64_range(start_addr, end_addr));
      }break;
      
      case SYMS_DwRngListEntryKind_OffsetPair:
      {
        SYMS_U64 start_offset = 0;
        SYMS_U64 end_offset = 0;
        read_off += syms_based_range_read_uleb128(base, rng, read_off, &start_offset);
        read_off += syms_based_range_read_uleb128(base, rng, read_off, &end_offset);
        syms_u64_range_list_push(arena, &list, syms_make_u64_range(start_offset + base_addr, end_offset + base_addr));
      }break;
      
      //- rjf: non-split units only:
      
      case SYMS_DwRngListEntryKind_BaseAddress:
      {
        SYMS_U64 new_base_addr = 0;
        read_off += syms_based_range_read(base, rng, read_off, addr_size, &new_base_addr);
        base_addr = new_base_addr;
      }break;
      
      case SYMS_DwRngListEntryKind_StartEnd:
      {
        SYMS_U64 start = 0;
        SYMS_U64 end = 0;
        read_off += syms_based_range_read(base, rng, read_off, addr_size, &start);
        read_off += syms_based_range_read(base, rng, read_off, addr_size, &end);
        syms_u64_range_list_push(arena, &list, syms_make_u64_range(start, end));
      }break;
      
      case SYMS_DwRngListEntryKind_StartLength:
      {
        SYMS_U64 start = 0;
        SYMS_U64 length = 0;
        read_off += syms_based_range_read(base, rng, read_off, addr_size, &start);
        read_off += syms_based_range_read_uleb128(base, rng, read_off, &length);
        syms_u64_range_list_push(arena, &list, syms_make_u64_range(start, start+length));
      }break;
    }
  }
  
  return list;
}

//- rjf: .debug_loclists parsing

SYMS_API SYMS_LocRangeList
syms_dw_v5_location_ranges_from_loclist_offset(SYMS_Arena *arena, SYMS_String8 data, SYMS_DwDbgAccel *dbg, SYMS_DwSectionKind section, SYMS_U64 addr_size, SYMS_U64 addr_section_base, SYMS_U64 offset)
{
  SYMS_LocRangeList list = {0};
  
  SYMS_U64 read_off = offset;
  void *base = syms_dw_sec_base_from_dbg(data, dbg, section);
  SYMS_U64Range rng = syms_dw_sec_range_from_dbg(dbg, section);
  
  SYMS_U64 base_addr = 0;
  
  for(SYMS_B32 done = syms_false; !done;)
  {
    SYMS_U8 kind8 = 0;
    read_off += syms_based_range_read_struct(base, rng, read_off, &kind8);
    SYMS_DwLocListEntryKind kind = (SYMS_DwLocListEntryKind)kind8;
    
    SYMS_B32 do_counted_location_description = 0;
    SYMS_U64 start_addr = 0;
    SYMS_U64 end_addr = 0;
    
    switch(kind)
    {
      //- rjf: can be used in split and non-split units:
      default:
      case SYMS_DwLocListEntryKind_EndOfList:
      {
        done = syms_true;
      }break;
      
      case SYMS_DwLocListEntryKind_BaseAddressX:
      {
        SYMS_U64 base_addr_idx = 0;
        read_off += syms_based_range_read_uleb128(base, rng, read_off, &base_addr_idx);
        base_addr = syms_dw_v5_addr_from_addrs_section_base_index(data, dbg, SYMS_DwSectionKind_Addr, addr_section_base, base_addr_idx);
      }break;
      
      case SYMS_DwLocListEntryKind_StartXEndX:
      {
        SYMS_U64 start_addr_idx = 0;
        SYMS_U64 end_addr_idx = 0;
        read_off += syms_based_range_read_uleb128(base, rng, read_off, &start_addr_idx);
        read_off += syms_based_range_read_uleb128(base, rng, read_off, &end_addr_idx);
        start_addr = syms_dw_v5_addr_from_addrs_section_base_index(data, dbg, SYMS_DwSectionKind_Addr, addr_section_base, start_addr_idx);
        end_addr = syms_dw_v5_addr_from_addrs_section_base_index(data, dbg, SYMS_DwSectionKind_Addr, addr_section_base, end_addr_idx);
        do_counted_location_description = 1;
      }break;
      
      case SYMS_DwLocListEntryKind_StartXLength:
      {
        SYMS_U64 start_addr_idx = 0;
        SYMS_U64 length = 0;
        read_off += syms_based_range_read_uleb128(base, rng, read_off, &start_addr_idx);
        read_off += syms_based_range_read_uleb128(base, rng, read_off, &length);
        start_addr = syms_dw_v5_addr_from_addrs_section_base_index(data, dbg, SYMS_DwSectionKind_Addr, addr_section_base, start_addr_idx);
        end_addr = start_addr + length;
        do_counted_location_description = 1;
      }break;
      
      case SYMS_DwLocListEntryKind_OffsetPair:
      {
        SYMS_U64 start_offset = 0;
        SYMS_U64 end_offset = 0;
        read_off += syms_based_range_read_uleb128(base, rng, read_off, &start_offset);
        read_off += syms_based_range_read_uleb128(base, rng, read_off, &end_offset);
        start_addr = start_offset + base_addr;
        end_addr = end_offset + base_addr;
        do_counted_location_description = 1;
      }break;
      
      case SYMS_DwLocListEntryKind_DefaultLocation:
      {
        do_counted_location_description = 1;
      }break;
      
      //- rjf: non-split units only:
      
      case SYMS_DwLocListEntryKind_BaseAddress:
      {
        SYMS_U64 new_base_addr = 0;
        read_off += syms_based_range_read(base, rng, read_off, addr_size, &new_base_addr);
        base_addr = new_base_addr;
      }break;
      
      case SYMS_DwLocListEntryKind_StartEnd:
      {
        SYMS_U64 start = 0;
        SYMS_U64 end = 0;
        read_off += syms_based_range_read(base, rng, read_off, addr_size, &start);
        read_off += syms_based_range_read(base, rng, read_off, addr_size, &end);
        start_addr = start;
        end_addr = end;
        do_counted_location_description = 1;
      }break;
      
      case SYMS_DwLocListEntryKind_StartLength:
      {
        SYMS_U64 start = 0;
        SYMS_U64 length = 0;
        read_off += syms_based_range_read(base, rng, read_off, addr_size, &start);
        read_off += syms_based_range_read_uleb128(base, rng, read_off, &length);
        start_addr = start;
        end_addr = start+length;
        do_counted_location_description = 1;
      }break;
    }
    
    //- rjf: parse a counted location description + push to result, if valid
    if(do_counted_location_description)
    {
      SYMS_LocRangeNode *n = syms_push_array_zero(arena, SYMS_LocRangeNode, 1);
      n->loc_range.vrange.min = start_addr;
      n->loc_range.vrange.max = end_addr;
      n->loc_range.loc_id = read_off;
      SYMS_QueuePush(list.first, list.last, n);
      list.count += 1;
      
      // rjf: skip past the location description
      SYMS_U64 locdesc_length = 0;
      read_off += syms_based_range_read_uleb128(base, rng, read_off, &locdesc_length);
      read_off += locdesc_length;
    }
  }
  
  return list;
}

SYMS_API SYMS_Location
syms_dw_v5_location_from_loclist_id(SYMS_Arena *arena, SYMS_String8 data, SYMS_DwDbgAccel *dbg, SYMS_DwSectionKind section, SYMS_LocID id)
{
  SYMS_Location result = {0};
  
  void *base = syms_dw_sec_base_from_dbg(data, dbg, section);
  SYMS_U64Range sec_range = syms_dw_sec_range_from_dbg(dbg, section);
  SYMS_U64 read_off = (SYMS_U64)id;
  SYMS_U64 length = 0;
  read_off += syms_based_range_read_uleb128(base, sec_range, read_off, &length);
  void *expr_base = syms_based_range_ptr(base, sec_range, read_off);
  SYMS_U64Range expr_range = syms_make_u64_range(read_off, read_off+length);
  SYMS_String8 location = syms_dw_expr__transpile_to_eval(arena, dbg, base, expr_range);
  
  // TODO(rjf): wire up `location` to `result`, once SYMS_Location changes to
  // returning a SYMS_String8 instead of an op list.
  
  return result;
}

////////////////////////////////
//~ rjf: Attrib Value Parsing

SYMS_API SYMS_DwAttribValueResolveParams
syms_dw_attrib_value_resolve_params_from_comp_root(SYMS_DwCompRoot *root)
{
  SYMS_DwAttribValueResolveParams params;
  syms_memzero_struct(&params);
  params.version                  = root->version;
  params.language                 = root->language;
  params.addr_size                = root->address_size;
  params.containing_unit_info_off = root->info_off;
  params.debug_addrs_base         = root->addrs_base;
  params.debug_rnglists_base      = root->rnglist_base;
  params.debug_str_offs_base      = root->stroffs_base;
  params.debug_loclists_base      = root->loclist_base;
  return params;
}

SYMS_API SYMS_DwAttribValue
syms_dw_attrib_value_from_form_value(SYMS_String8 data, SYMS_DwDbgAccel *dbg, SYMS_DwAttribValueResolveParams resolve_params,
                                     SYMS_DwFormKind form_kind, SYMS_DwAttribClass value_class,
                                     SYMS_DwAttribValue form_value)
{
  SYMS_DwAttribValue value;
  syms_memzero_struct(&value);
  
  //~ rjf: DWARF V5 value parsing
  
  //- rjf: (DWARF V5 ONLY) the form value is storing an address index (ADDRess indeX), which we
  // must resolve to an actual address using the containing comp unit's contribution to the
  // .debug_addr section.
  if(resolve_params.version >= SYMS_DwVersion_V5 &&
     value_class == SYMS_DwAttribClass_ADDRESS &&
     (form_kind == SYMS_DwFormKind_ADDRX  || form_kind == SYMS_DwFormKind_ADDRX1 ||
      form_kind == SYMS_DwFormKind_ADDRX2 || form_kind == SYMS_DwFormKind_ADDRX3 ||
      form_kind == SYMS_DwFormKind_ADDRX4))
  {
    SYMS_U64 addr_index = form_value.v[0];
    SYMS_U64 addr = syms_dw_v5_addr_from_addrs_section_base_index(data, dbg, SYMS_DwSectionKind_Addr, resolve_params.debug_addrs_base, addr_index);
    value.v[0] = addr;
  }
  //- rjf: (DWARF V5 ONLY) lookup into the .debug_loclists section via an index
  else if(resolve_params.version >= SYMS_DwVersion_V5 &&
          value_class == SYMS_DwAttribClass_LOCLIST &&
          form_kind == SYMS_DwFormKind_LOCLISTX)
  {
    SYMS_U64 loclist_index = form_value.v[0];
    SYMS_U64 loclist_offset = syms_dw_v5_sec_offset_from_rnglist_or_loclist_section_base_index(data, dbg, SYMS_DwSectionKind_LocLists, resolve_params.debug_loclists_base, loclist_index);
    value.section = SYMS_DwSectionKind_LocLists;
    value.v[0] = loclist_offset;
  }
  //- rjf: (DWARF V5 ONLY) lookup into the .debug_loclists section via an offset
  else if(resolve_params.version >= SYMS_DwVersion_V5 &&
          (value_class == SYMS_DwAttribClass_LOCLIST ||
           value_class == SYMS_DwAttribClass_LOCLISTPTR) &&
          form_kind == SYMS_DwFormKind_SEC_OFFSET)
  {
    SYMS_U64 loclist_offset = form_value.v[0];
    value.section = SYMS_DwSectionKind_LocLists;
    value.v[0] = loclist_offset;
  }
  //- rjf: (DWARF V5 ONLY) lookup into the .debug_rnglists section via an index
  else if(resolve_params.version >= SYMS_DwVersion_V5 &&
          (value_class == SYMS_DwAttribClass_RNGLISTPTR ||
           value_class == SYMS_DwAttribClass_RNGLIST) &&
          form_kind == SYMS_DwFormKind_RNGLISTX)
  {
    SYMS_U64 rnglist_index = form_value.v[0];
    SYMS_U64 rnglist_offset = syms_dw_v5_sec_offset_from_rnglist_or_loclist_section_base_index(data, dbg, SYMS_DwSectionKind_RngLists, resolve_params.debug_rnglists_base, rnglist_index);
    value.section = SYMS_DwSectionKind_RngLists;
    value.v[0] = rnglist_offset;
  }
  //- rjf: (DWARF V5 ONLY) lookup into the .debug_rnglists section via an offset
  else if(resolve_params.version >= SYMS_DwVersion_V5 &&
          (value_class == SYMS_DwAttribClass_RNGLISTPTR ||
           value_class == SYMS_DwAttribClass_RNGLIST) &&
          form_kind != SYMS_DwFormKind_RNGLISTX)
  {
    SYMS_U64 rnglist_offset = form_value.v[0];
    value.section = SYMS_DwSectionKind_RngLists;
    value.v[0] = rnglist_offset;
  }
  //- rjf: (DWARF V5 ONLY) .debug_str_offsets table index, that we need to resolve
  // using the containing compilation unit's contribution to the section
  else if(resolve_params.version >= SYMS_DwVersion_V5 &&
          value_class == SYMS_DwAttribClass_STRING && 
          (form_kind == SYMS_DwFormKind_STRX ||
           form_kind == SYMS_DwFormKind_STRX1 ||
           form_kind == SYMS_DwFormKind_STRX2 ||
           form_kind == SYMS_DwFormKind_STRX3 ||
           form_kind == SYMS_DwFormKind_STRX4))
  {
    SYMS_U64 str_index = form_value.v[0];
    SYMS_U64 str_offset = syms_dw_v5_offset_from_offs_section_base_index(data, dbg, SYMS_DwSectionKind_StrOffsets, resolve_params.debug_str_offs_base, str_index);
    value.section = SYMS_DwSectionKind_Str;
    void *base = syms_dw_sec_base_from_dbg(data, dbg, value.section);
    SYMS_U64Range range = syms_dw_sec_range_from_dbg(dbg, value.section);
    SYMS_String8 string = syms_based_range_read_string(base, range, str_offset);
    value.v[0] = str_offset;
    value.v[1] = value.v[0] + string.size;
  }
  //- rjf: (DWARF V5 ONLY) reference that we should resolve through ref_addr_desc
  else if(resolve_params.version >= SYMS_DwVersion_V5 &&
          value_class == SYMS_DwAttribClass_REFERENCE &&
          form_kind == SYMS_DwFormKind_REF_ADDR)
  {
    // TODO(nick): DWARF 5 @dwarf_v5
  }
  //- TODO(rjf): (DWARF V5 ONLY) reference resolution using the .debug_names section
  else if(resolve_params.version >= SYMS_DwVersion_V5 &&
          form_kind == SYMS_DwFormKind_REF_SIG8)
  {
    // TODO(nick): DWARF 5: We need to handle .debug_names section in order to resolve this value. @dwarf_v5
    value.v[0] = SYMS_U64_MAX;
  }
  
  //~ rjf: All other value parsing (DWARF V4 and below)
  
  //- rjf: reference to an offset relative to the compilation unit's info base
  else if (value_class == SYMS_DwAttribClass_REFERENCE &&
           (form_kind == SYMS_DwFormKind_REF1 ||
            form_kind == SYMS_DwFormKind_REF2 ||
            form_kind == SYMS_DwFormKind_REF4 ||
            form_kind == SYMS_DwFormKind_REF8 ||
            form_kind == SYMS_DwFormKind_REF_UDATA))
  {
    value.v[0] = resolve_params.containing_unit_info_off + form_value.v[0];
  }
  
  //- rjf: info-section string -- this is a string that is just pasted straight
  // into the .debug_info section
  else if(value_class == SYMS_DwAttribClass_STRING && form_kind == SYMS_DwFormKind_STRING)
  {
    value = form_value;
    value.section = SYMS_DwSectionKind_Info;
  }
  
  //- rjf: string-section string -- this is a string that's inside the .debug_str
  // section, and we've been provided an offset to it
  else if(value_class == SYMS_DwAttribClass_STRING && 
          (form_kind == SYMS_DwFormKind_STRP ||
           form_kind == SYMS_DwFormKind_STRP_SUP))
  {
    value.section = SYMS_DwSectionKind_Str;
    void *base = syms_dw_sec_base_from_dbg(data, dbg, value.section);
    SYMS_U64Range range = syms_dw_sec_range_from_dbg(dbg, value.section);
    SYMS_String8 string = syms_based_range_read_string(base, range, form_value.v[0]);
    value.v[0] = form_value.v[0];
    value.v[1] = value.v[0] + string.size;
  }
  //- rjf: line-string
  else if(value_class == SYMS_DwAttribClass_STRING && form_kind == SYMS_DwFormKind_LINE_STRP)
  {
    value.section = SYMS_DwSectionKind_LineStr;
    void *base = syms_dw_sec_base_from_dbg(data, dbg, value.section);
    SYMS_U64Range range = syms_dw_sec_range_from_dbg(dbg, value.section);
    SYMS_String8 string = syms_based_range_read_string(base, range, form_value.v[0]);
    value.v[0] = form_value.v[0];
    value.v[1] = value.v[0] + string.size;
  }
  //- rjf: .debug_ranges
  else if(resolve_params.version < SYMS_DwVersion_V5 &&
          (value_class == SYMS_DwAttribClass_RNGLISTPTR ||
           value_class == SYMS_DwAttribClass_RNGLIST) &&
          (form_kind == SYMS_DwFormKind_SEC_OFFSET))
  {
    SYMS_U64 ranges_offset = form_value.v[0];
    value.section = SYMS_DwSectionKind_Ranges;
    value.v[0] = ranges_offset;
  }
  //- rjf: .debug_loc
  else if(resolve_params.version < SYMS_DwVersion_V5 &&
          (value_class == SYMS_DwAttribClass_LOCLISTPTR ||
           value_class == SYMS_DwAttribClass_LOCLIST) &&
          (form_kind == SYMS_DwFormKind_SEC_OFFSET))
  {
    SYMS_U64 offset = form_value.v[0];
    value.section = SYMS_DwSectionKind_Loc;
    value.v[0] = offset;
  }
  //- rjf: invalid attribute class
  else if(value_class == 0)
  {
    SYMS_ASSERT_PARANOID(!"attribute class was not resolved");
  }
  //- rjf: in all other cases, we can accept the form_value as the correct
  // representation for the parsed value, so we can just copy it over.
  else
  {
    value = form_value;
  }
  
  return value;
}

SYMS_API SYMS_String8
syms_dw_string_from_attrib_value(SYMS_String8 data, SYMS_DwDbgAccel *dbg, SYMS_DwAttribValue value)
{
  SYMS_String8 string;
  syms_memzero_struct(&string);
  SYMS_DwSectionKind section_kind = value.section;
  void *base = syms_dw_sec_base_from_dbg(data, dbg, section_kind);
  SYMS_U64Range range = syms_dw_sec_range_from_dbg(dbg, section_kind);
  string.str = (SYMS_U8 *)syms_based_range_ptr(base, range, value.v[0]);
  string.size = value.v[1] - value.v[0];
  return string;
}

SYMS_API SYMS_U64RangeList
syms_dw_range_list_from_high_low_pc_and_ranges_attrib_value(SYMS_Arena *arena, SYMS_String8 data, SYMS_DwDbgAccel *dbg, SYMS_U64 address_size, SYMS_U64 comp_unit_base_addr, SYMS_U64 addr_section_base, SYMS_U64 low_pc, SYMS_U64 high_pc, SYMS_DwAttribValue ranges_value)
{
  SYMS_U64RangeList list = {0};
  switch(ranges_value.section)
  {
    //- rjf: (DWARF V5 ONLY) .debug_rnglists offset
    case SYMS_DwSectionKind_RngLists:
    {
      list = syms_dw_v5_range_list_from_rnglist_offset(arena, data, dbg, ranges_value.section, address_size, addr_section_base, ranges_value.v[0]);
    }break;
    
    //- rjf: (DWARF V4 and earlier) .debug_ranges parsing
    case SYMS_DwSectionKind_Ranges:
    {
      list = syms_dw_v4_range_list_from_range_offset(arena, data, dbg, address_size, comp_unit_base_addr, ranges_value.v[0]);
    }break;
    
    //- rjf: fall back to trying to use low/high PCs
    default:
    {
      syms_u64_range_list_push(arena, &list, syms_make_u64_range(low_pc, high_pc));
    }break;
  }
  return list;
}

////////////////////////////////
//~ rjf: Tag Parsing

SYMS_API SYMS_DwAttribListParseResult
syms_dw_parse_attrib_list_from_info_abbrev_offsets(SYMS_Arena *arena, SYMS_String8 data, SYMS_DwDbgAccel *dbg, SYMS_DwLanguage lang, SYMS_DwVersion ver, SYMS_U64 address_size, SYMS_U64 info_off, SYMS_U64 abbrev_off)
{
  //- rjf: set up prereqs
  SYMS_DwMode info_mode = syms_dw_mode_from_sec(dbg->section_map, SYMS_DwSectionKind_Info);
  void *info_base = syms_dw_sec_base_from_dbg(data, dbg, SYMS_DwSectionKind_Info);
  SYMS_U64Range info_range = syms_dw_sec_range_from_dbg(dbg, SYMS_DwSectionKind_Info);
  SYMS_DwMode abbrev_mode = syms_dw_mode_from_sec(dbg->section_map, SYMS_DwSectionKind_Abbrev);
  void *abbrev_base = syms_dw_sec_base_from_dbg(data, dbg, SYMS_DwSectionKind_Abbrev);
  SYMS_U64Range abbrev_range = syms_dw_sec_range_from_dbg(dbg, SYMS_DwSectionKind_Abbrev);
  
  //- rjf: set up read offsets
  SYMS_U64 info_read_off   = info_off;
  SYMS_U64 abbrev_read_off = abbrev_off;
  
  //- rjf: parse all attributes 
  SYMS_DwAttribListParseResult result = {0};
  for(SYMS_B32 good_abbrev = syms_true; good_abbrev;)
  {
    SYMS_U64 attrib_info_offset = info_read_off;
    
    //- rjf: parse abbrev attrib info
    SYMS_DwAbbrev abbrev;
    syms_memzero_struct(&abbrev);
    {
      SYMS_U64 bytes_read = syms_dw_based_range_read_abbrev_attrib_info(abbrev_base, abbrev_range, abbrev_read_off, &abbrev);
      abbrev_read_off += bytes_read;
      good_abbrev = abbrev.id != 0;
    }
    
    //- rjf: extract attrib info from abbrev
    SYMS_DwAttribKind attrib_kind = (SYMS_DwAttribKind)abbrev.id;
    SYMS_DwFormKind form_kind = (SYMS_DwFormKind)abbrev.sub_kind;
    SYMS_DwAttribClass attrib_class = syms_dw_pick_attrib_value_class(lang, ver, attrib_kind, form_kind);
    
    //- rjf: parse the form value from the file
    SYMS_DwAttribValue form_value;
    syms_memzero_struct(&form_value);
    if(good_abbrev)
    {
      // NOTE(nick): This is a special case form. Basically it let's user to
      // define attribute form in the .debug_info.
      if(form_kind == SYMS_DwFormKind_INDIRECT)
      {
        SYMS_U64 override_form_kind = 0;
        info_read_off += syms_based_range_read_uleb128(info_base, info_range, info_read_off, &override_form_kind);
        form_kind = (SYMS_DwFormKind)override_form_kind;
      }
      SYMS_U64 bytes_read = syms_dw_based_range_read_attrib_form_value(info_base, info_range, info_read_off, info_mode, address_size,
                                                                       form_kind, abbrev.const_value, &form_value);
      info_read_off += bytes_read;
    }
    
    //- rjf: push this parsed attrib to the list
    if(good_abbrev)
    {
      SYMS_DwAttribNode *node = syms_push_array_zero(arena, SYMS_DwAttribNode, 1);
      node->attrib.info_off     = attrib_info_offset;
      node->attrib.abbrev_id    = abbrev.id;
      node->attrib.attrib_kind  = attrib_kind;
      node->attrib.form_kind    = form_kind;
      node->attrib.value_class  = attrib_class;
      node->attrib.form_value   = form_value;
      result.attribs.count += 1;
      SYMS_QueuePush(result.attribs.first, result.attribs.last, node);
    }
  }
  
  result.max_info_off = info_read_off;
  result.max_abbrev_off = abbrev_read_off;
  return result;
}

SYMS_API SYMS_DwTag *
syms_dw_tag_from_info_offset(SYMS_Arena *arena,
                             SYMS_String8 data,
                             SYMS_DwDbgAccel *dbg,
                             SYMS_DwAbbrevTable abbrev_table,
                             SYMS_DwLanguage lang,
                             SYMS_DwVersion ver,
                             SYMS_U64 address_size,
                             SYMS_U64 info_offset)
{
  void *info_base = syms_dw_sec_base_from_dbg(data, dbg, SYMS_DwSectionKind_Info);
  SYMS_U64Range info_range = syms_dw_sec_range_from_dbg(dbg, SYMS_DwSectionKind_Info);
  void *abbrev_base = syms_dw_sec_base_from_dbg(data, dbg, SYMS_DwSectionKind_Abbrev);
  SYMS_U64Range abbrev_range = syms_dw_sec_range_from_dbg(dbg, SYMS_DwSectionKind_Abbrev);
  
  SYMS_DwTag *tag = syms_push_array_zero(arena, SYMS_DwTag, 1);
  
  //- rjf: calculate .debug_info read cursor, relative to info range minimum
  SYMS_U64 info_read_off = info_offset - info_range.min;
  
  //- rjf: read abbrev ID
  SYMS_U64 abbrev_id = 0;
  info_read_off += syms_based_range_read_uleb128(info_base, info_range, info_read_off, &abbrev_id);
  SYMS_B32 good_abbrev_id = abbrev_id != 0;
  
  //- rjf: figure out abbrev offset for this ID
  SYMS_U64 abbrev_offset = 0;
  if(good_abbrev_id)
  {
    abbrev_offset = syms_dw_abbrev_offset_from_abbrev_id(abbrev_table, abbrev_id);
  }
  
  //- rjf: calculate .debug_abbrev read cursor, relative to abbrev range minimum
  SYMS_U64 abbrev_read_off = abbrev_offset - abbrev_range.min;
  
  //- rjf: parse abbrev tag info
  SYMS_DwAbbrev abbrev_tag_info;
  syms_memzero_struct(&abbrev_tag_info);
  SYMS_B32 good_tag_abbrev = syms_false;
  if(good_abbrev_id)
  {
    abbrev_read_off += syms_dw_based_range_read_abbrev_tag(abbrev_base, abbrev_range, abbrev_read_off, &abbrev_tag_info);
    good_tag_abbrev = syms_true;//abbrev_tag_info.id != 0;
  }
  
  //- rjf: parse all attributes for this tag
  SYMS_U64 attribs_info_off = 0;
  SYMS_U64 attribs_abbrev_off = 0;
  SYMS_DwAttribList attribs = {0};
  if(good_tag_abbrev)
  {
    SYMS_DwAttribListParseResult attribs_parse = syms_dw_parse_attrib_list_from_info_abbrev_offsets(arena, data, dbg, lang, ver, address_size, info_read_off, abbrev_read_off);
    attribs_info_off = info_read_off;
    attribs_abbrev_off = abbrev_read_off;
    info_read_off = attribs_parse.max_info_off;
    abbrev_read_off = attribs_parse.max_abbrev_off;
    attribs = attribs_parse.attribs;
  }
  
  //- rjf: fill tag
  {
    tag->abbrev_id = abbrev_id;
    tag->info_range = syms_make_u64_range(info_offset, info_range.min + info_read_off);
    tag->abbrev_range = syms_make_u64_range(abbrev_offset, abbrev_range.min + abbrev_read_off);
    tag->has_children = !!(abbrev_tag_info.flags & SYMS_DwAbbrevFlag_HasChildren);
    tag->kind = (SYMS_DwTagKind)abbrev_tag_info.sub_kind;
    tag->attribs_info_off   = attribs_info_off;
    tag->attribs_abbrev_off = attribs_abbrev_off;
    tag->attribs = attribs;
  }
  
  return tag;
}

SYMS_API SYMS_DwTagStub
syms_dw_stub_from_tag(SYMS_String8 data, SYMS_DwDbgAccel *dbg, SYMS_DwAttribValueResolveParams resolve_params,
                      SYMS_DwTag *tag)
{
  SYMS_DwTagStub stub = {0};
  stub.sid = tag->info_range.min;
  stub.kind = tag->kind;
  stub.children_info_off = tag->has_children ? tag->info_range.max : 0;
  stub.attribs_info_off = tag->attribs_info_off;
  stub.attribs_abbrev_off = tag->attribs_abbrev_off;
  for(SYMS_DwAttribNode *n = tag->attribs.first; n != 0; n = n->next)
  {
    SYMS_DwAttrib *attrib = &n->attrib;
    switch(attrib->attrib_kind)
    {
      default: break;
      case SYMS_DwAttribKind_SPECIFICATION:
      {
        SYMS_DwAttribValue value = syms_dw_attrib_value_from_form_value(data, dbg, resolve_params, attrib->form_kind, attrib->value_class, attrib->form_value);
        stub.ref = value.v[0];
        stub.flags |= SYMS_DwTagStubFlag_HasSpecification;
      }break;
      case SYMS_DwAttribKind_ABSTRACT_ORIGIN:
      {
        SYMS_DwAttribValue value = syms_dw_attrib_value_from_form_value(data, dbg, resolve_params, attrib->form_kind, attrib->value_class, attrib->form_value);
        stub.abstract_origin = value.v[0];
      }break;
      case SYMS_DwAttribKind_OBJECT_POINTER:       stub.flags |= SYMS_DwTagStubFlag_HasObjectPointerArg; break;
      case SYMS_DwAttribKind_LOCATION:             stub.flags |= SYMS_DwTagStubFlag_HasLocation; break;
      case SYMS_DwAttribKind_EXTERNAL:             stub.flags |= SYMS_DwTagStubFlag_HasExternal; break;
    }
  }
  return stub;
}

////////////////////////////////
//~ rjf: Unit Set Accel;

SYMS_API SYMS_U64
syms_dw_v5_header_offset_from_table_offset(SYMS_String8 data, SYMS_DwDbgAccel *dbg, SYMS_DwSectionKind section, SYMS_U64 table_off)
{
  // NOTE(rjf): From the DWARF V5 spec (February 13, 2017), page 401:
  //
  // "
  // Each skeleton compilation unit also has a DW_AT_addr_base attribute,
  // which provides the relocated offset to that compilation unit’s
  // contribution in the executable’s .debug_addr section. Unlike the
  // DW_AT_stmt_list attribute, the offset refers to the first address table
  // slot, not to the section header. In this example, we see that the first
  // address (slot 0) from demo1.o begins at offset 48. Because the
  // .debug_addr section contains an 8-byte header, the object file’s
  // contribution to the section actually begins at offset 40 (for a 64-bit
  // DWARF object, the header would be 16 bytes long, and the value for the
  // DW_AT_addr_base attribute would then be 56). All attributes in demo1.dwo
  // that use DW_FORM_addrx, DW_FORM_addrx1, DW_FORM_addrx2, DW_FORM_addrx3
  // or DW_FORM_addrx4 would then refer to address table slots relative to
  // that offset. Likewise, the .debug_addr contribution from demo2.dwo begins
  // at offset 72, and its first address slot is at offset 80. Because these
  // contributions have been processed by the linker, they contain relocated
  // values for the addresses in the program that are referred to by the
  // debug information.
  // "
  //
  // This seems to at least partially explain why the addr_base is showing up
  // 8 bytes later than we are expecting it to. We can't actually just store
  // the base that we read from the SYMS_DwAttribKind_ADDR_BASE attrib, because
  // it's showing up *after* the header, so we need to bump it back.
  
  // NOTE(rjf): From the DWARF V5 spec (February 13, 2017), page 66:
  //
  // "
  // A DW_AT_rnglists_base attribute, whose value is of class rnglistsptr. This
  // attribute points to the beginning of the offsets table (immediately
  // following the header) of the compilation unit's contribution to the
  // .debug_rnglists section. References to range lists (using DW_FORM_rnglistx)
  // within the compilation unit are interpreted relative to this base.
  // "
  //
  // Similarly, we need to figure out where to go to parse the header.
  
  SYMS_U64 max_header_size = 0;
  SYMS_U64 min_header_size = 0;
  switch(section)
  {
    default:
    case SYMS_DwSectionKind_Addr:
    {
      max_header_size = 16;
      min_header_size = 8;
    }break;
    case SYMS_DwSectionKind_StrOffsets:
    {
      max_header_size = 16;
      min_header_size = 8;
    }break;
    case SYMS_DwSectionKind_RngLists:
    {
      max_header_size = 20;
      min_header_size = 12;
    }break;
    case SYMS_DwSectionKind_LocLists:
    {
      // TODO(rjf)
    }break;
  }
  
  SYMS_U64 past_header = table_off;
  void *addr_base = syms_dw_sec_base_from_dbg(data, dbg, section);
  SYMS_U64Range addr_rng = syms_dw_sec_range_from_dbg(dbg, section);
  
  //- rjf: figure out which sized header we have
  SYMS_U64 header_size = 0;
  {
    // rjf: try max header, and if it works, the header is the max size, otherwise we will
    // need to rely on the min header size
    SYMS_U32 first32 = 0;
    syms_based_range_read_struct(addr_base, addr_rng, past_header-max_header_size, &first32);
    if(first32 == 0xffffffff)
    {
      header_size = max_header_size;
    }
    else
    {
      header_size = min_header_size;
    }
  }
  
  return table_off - header_size;
}

SYMS_API SYMS_DwCompRoot
syms_dw_comp_root_from_range(SYMS_Arena *arena, SYMS_String8 data, SYMS_DwDbgAccel *dbg, SYMS_U64 index, SYMS_U64Range range)
{
  void *info_base = syms_dw_sec_base_from_dbg(data, dbg, SYMS_DwSectionKind_Info);
  SYMS_ArenaTemp scratch = syms_get_scratch(&arena, 1);
  
  //- rjf: up-front known parsing offsets (yep, that's right, it's only 1!)
  SYMS_U64 size_off = 0;
  
  //- rjf: parse size of this compilation unit's data
  SYMS_U64 size = 0;
  SYMS_U64 version_off = size_off;
  {
    SYMS_U64 bytes_read = syms_dw_based_range_read_length(info_base, range, size_off, &size);
    version_off += bytes_read;
  }
  
  //- rjf: parse version
  SYMS_B32 got_version = syms_false;
  SYMS_U16 version = 0;
  SYMS_U64 unit_off = version_off;
  if(syms_based_range_read_struct(info_base, range, version_off, &version))
  {
    unit_off += sizeof(SYMS_U16);
    got_version = syms_true;
  }
  
  //- rjf: parse unit kind, abbrev_base, address size
  SYMS_B32 got_unit_kind = syms_false;
  SYMS_U64 next_off = unit_off;
  SYMS_U64 unit_kind = SYMS_DwCompUnitKind_RESERVED;
  SYMS_U64 abbrev_base = SYMS_U64_MAX;
  SYMS_U64 address_size = 0;
  SYMS_U64 spec_dwo_id = 0;
  if(got_version)
  {
    switch(version)
    {
      default: break;
      case SYMS_DwVersion_V2: {
        abbrev_base = 0;
        next_off += syms_based_range_read(info_base, range, next_off, 4, &abbrev_base);
        next_off += syms_based_range_read(info_base, range, next_off, 1, &address_size);
        got_unit_kind = syms_true;
      } break;
      case SYMS_DwVersion_V3:
      case SYMS_DwVersion_V4:
      {
        next_off += syms_dw_based_range_read_length(info_base, range, next_off, &abbrev_base);
        next_off += syms_based_range_read(info_base, range, next_off, 1, &address_size);
        got_unit_kind = syms_true;
      }break;
      case SYMS_DwVersion_V5:
      {
        next_off += syms_based_range_read(info_base, range, next_off, 1, &unit_kind);
        next_off += syms_based_range_read(info_base, range, next_off, 1, &address_size);
        next_off += syms_dw_based_range_read_length(info_base, range, next_off, &abbrev_base);
        got_unit_kind = syms_true;
        
        //- rjf: parse DWO ID if appropriate
        if(unit_kind == SYMS_DwCompUnitKind_SKELETON || dbg->is_dwo)
        {
          next_off += syms_based_range_read(info_base, range, next_off, 8, &spec_dwo_id);
        }
      }break;
    }
  }
  
  //- rjf: build abbrev table
  SYMS_DwAbbrevTable abbrev_table = {0};
  if(got_unit_kind)
  {
    abbrev_table = syms_dw_make_abbrev_table(arena, data, dbg, abbrev_base);
  }
  
  //- rjf: parse compilation unit's tag
  SYMS_B32 got_comp_unit_tag = syms_false;
  SYMS_DwTag *comp_unit_tag = 0;
  if(got_unit_kind)
  {
    SYMS_U64 comp_root_tag_off = range.min + next_off;
    comp_unit_tag = syms_dw_tag_from_info_offset(scratch.arena, data, dbg, abbrev_table, SYMS_DwLanguage_NULL, (SYMS_DwVersion)version, address_size, comp_root_tag_off);
    got_comp_unit_tag = syms_true;
  }
  
  //- rjf: get all of the attribute values we need to start resolving attribute values
  SYMS_DwAttribValueResolveParams resolve_params;
  syms_memzero_struct(&resolve_params);
  resolve_params.version = (SYMS_DwVersion)version;
  if(got_comp_unit_tag)
  {
    for(SYMS_DwAttribNode *attrib_n = comp_unit_tag->attribs.first; attrib_n; attrib_n = attrib_n->next)
    {
      SYMS_DwAttrib *attrib = &attrib_n->attrib;
      
      // NOTE(rjf): We'll have to rely on just the form value at this point,
      // since we can't use the unit yet (since we're currently in the process
      // of building it). This should always be enough, otherwise there would
      // be a cyclic dependency in the requirements of each part of the
      // compilation unit's parse. DWARF is pretty crazy, but not *that* crazy,
      // so this should be good.
      switch(attrib->attrib_kind)
      {
        default: break;
        case SYMS_DwAttribKind_ADDR_BASE:         {resolve_params.debug_addrs_base     = attrib->form_value.v[0];}break;
        case SYMS_DwAttribKind_STR_OFFSETS_BASE:  {resolve_params.debug_str_offs_base  = attrib->form_value.v[0];}break;
        case SYMS_DwAttribKind_RNGLISTS_BASE:     {resolve_params.debug_rnglists_base  = attrib->form_value.v[0];}break;
        case SYMS_DwAttribKind_LOCLISTS_BASE:     {resolve_params.debug_loclists_base  = attrib->form_value.v[0];}break;
      }
    }
  }
  
  //- rjf: correct table offsets to header offsets (since DWARF V5 insists on being as useless as possible)
  if(got_comp_unit_tag && version >= SYMS_DwVersion_V5)
  {
    resolve_params.debug_addrs_base = syms_dw_v5_header_offset_from_table_offset(data, dbg, SYMS_DwSectionKind_Addr,
                                                                                 resolve_params.debug_addrs_base);
    resolve_params.debug_str_offs_base = syms_dw_v5_header_offset_from_table_offset(data, dbg, SYMS_DwSectionKind_StrOffsets,
                                                                                    resolve_params.debug_str_offs_base);
    resolve_params.debug_loclists_base = syms_dw_v5_header_offset_from_table_offset(data, dbg, SYMS_DwSectionKind_LocLists,
                                                                                    resolve_params.debug_loclists_base);
    resolve_params.debug_rnglists_base = syms_dw_v5_header_offset_from_table_offset(data, dbg, SYMS_DwSectionKind_RngLists,
                                                                                    resolve_params.debug_rnglists_base);
  }
  
  //- rjf: parse the rest of the compilation unit tag's attributes that we'd
  // like to cache
  SYMS_String8    name              = {0};
  SYMS_String8    producer          = {0};
  SYMS_String8    compile_dir       = {0};
  SYMS_String8    external_dwo_name = {0};
  SYMS_String8    external_gnu_dwo_name = {0};
  SYMS_U64        gnu_dwo_id        = 0;
  SYMS_U64        language          = 0;
  SYMS_U64        name_case         = 0;
  SYMS_B32        use_utf8          = 0;
  SYMS_U64        low_pc            = 0;
  SYMS_U64        high_pc           = 0;
  SYMS_B32        high_pc_is_relative = 0;
  SYMS_DwAttribValue ranges_attrib_value = {SYMS_DwSectionKind_Null};
  SYMS_U64        line_base         = 0;
  if(got_comp_unit_tag)
  {
    for(SYMS_DwAttribNode *attrib_n = comp_unit_tag->attribs.first; attrib_n; attrib_n = attrib_n->next)
    {
      SYMS_DwAttrib *attrib = &attrib_n->attrib;
      
      //- rjf: form value => value
      SYMS_DwAttribValue value;
      SYMS_B32 good_value = syms_false;
      {
        syms_memzero_struct(&value);
        if(syms_dw_are_attrib_class_and_form_kind_compatible(attrib->value_class, attrib->form_kind))
        {
          value = syms_dw_attrib_value_from_form_value(data, dbg, resolve_params,
                                                       attrib->form_kind,
                                                       attrib->value_class,
                                                       attrib->form_value);
          good_value = syms_true;
        }
      }
      
      //- rjf: map value to extracted info
      if(good_value)
      {
        switch(attrib->attrib_kind)
        {
          case SYMS_DwAttribKind_NAME:              {name                   = syms_dw_string_from_attrib_value(data, dbg, value); }break;
          case SYMS_DwAttribKind_PRODUCER:          {producer               = syms_dw_string_from_attrib_value(data, dbg, value); }break;
          case SYMS_DwAttribKind_COMP_DIR:          {compile_dir            = syms_dw_string_from_attrib_value(data, dbg, value); }break;
          case SYMS_DwAttribKind_DWO_NAME:          {external_dwo_name      = syms_dw_string_from_attrib_value(data, dbg, value); }break;
          case SYMS_DwAttribKind_GNU_DWO_NAME:      {external_gnu_dwo_name  = syms_dw_string_from_attrib_value(data, dbg, value); }break;
          case SYMS_DwAttribKind_GNU_DWO_ID:        {gnu_dwo_id             = value.v[0]; }break;
          case SYMS_DwAttribKind_LANGUAGE:          {language               = value.v[0]; }break;
          case SYMS_DwAttribKind_IDENTIFIER_CASE:   {name_case              = value.v[0]; }break;
          case SYMS_DwAttribKind_USE_UTF8:          {use_utf8               = (SYMS_B32)value.v[0]; }break;
          case SYMS_DwAttribKind_LOW_PC:            {low_pc                 = value.v[0]; }break;
          case SYMS_DwAttribKind_HIGH_PC:           {high_pc                = value.v[0]; high_pc_is_relative = attrib->value_class != SYMS_DwAttribClass_ADDRESS;}break;
          case SYMS_DwAttribKind_RANGES:            {ranges_attrib_value    = value; }break;
          case SYMS_DwAttribKind_STMT_LIST:         {line_base              = value.v[0]; }break;
          default: break;
        }
      }
    }
  }
  
  //- rjf: build+fill unit
  SYMS_DwCompRoot unit;
  syms_memzero_struct(&unit);
  
  //- rjf: fill header data
  unit.size            = size;
  unit.kind            = (SYMS_DwCompUnitKind)unit_kind;
  unit.version         = (SYMS_DwVersion)version;
  unit.address_size    = address_size;
  unit.abbrev_off      = abbrev_base;
  unit.info_off        = range.min;
  unit.index           = index;
  unit.tags_info_range = syms_make_u64_range(range.min+next_off, range.max);
  unit.abbrev_table    = abbrev_table;
  
  //- rjf: fill out offsets we need for attrib value resolution
  unit.rnglist_base = resolve_params.debug_rnglists_base;
  unit.loclist_base = resolve_params.debug_loclists_base;
  unit.addrs_base   = resolve_params.debug_addrs_base;
  unit.stroffs_base = resolve_params.debug_str_offs_base;
  
  //- rjf: fill out general info
  unit.name          = name;
  unit.producer      = producer;
  unit.compile_dir   = compile_dir;
  unit.external_dwo_name = external_dwo_name.size != 0 ? external_dwo_name : external_gnu_dwo_name;
  if(external_dwo_name.size)
  {
    unit.dwo_id = spec_dwo_id;
  }
  else if(external_gnu_dwo_name.size)
  {
    unit.dwo_id = gnu_dwo_id;
  }
  unit.language      = (SYMS_DwLanguage)language;
  unit.name_case     = name_case;
  unit.use_utf8      = use_utf8;
  unit.line_off      = line_base;
  unit.low_pc        = low_pc;
  unit.high_pc       = high_pc;
  unit.ranges_attrib_value = ranges_attrib_value;
  
  //- rjf: fill fixup of low/high PC situation
  if(high_pc_is_relative)
  {
    unit.high_pc += unit.low_pc;
  }
  
  //- rjf: fill base address
  {
    unit.base_addr = unit.low_pc;
  }
  
  //- rjf: build+fill directory and file tables
  {
    SYMS_DwMode line_mode = syms_dw_mode_from_sec(dbg->section_map, SYMS_DwSectionKind_Line);
    void *line_base = syms_dw_sec_base_from_dbg(data, dbg, SYMS_DwSectionKind_Line);
    SYMS_U64Range line_rng = syms_dw_sec_range_from_dbg(dbg, SYMS_DwSectionKind_Line);
    SYMS_DwLineVMHeader vm_header;
    SYMS_U64 read_size = syms_dw_read_line_vm_header(arena, line_base, line_rng, unit.line_off, data, line_mode, dbg, &unit, &vm_header);
    if (read_size > 0) {
      unit.dir_table = vm_header.dir_table;
      unit.file_table = vm_header.file_table;
    }
  }
  
  syms_release_scratch(scratch);
  return unit;
}

SYMS_API SYMS_DwExtDebugRef
syms_dw_ext_debug_ref_from_comp_root(SYMS_DwCompRoot *root)
{
  SYMS_DwExtDebugRef ref;
  syms_memzero_struct(&ref);
  ref.dwo_path = root->external_dwo_name;
  ref.dwo_id = root->dwo_id;
  return ref;
}

SYMS_API SYMS_DwUnitSetAccel *
syms_dw_unit_set_accel_from_dbg(SYMS_Arena *arena, SYMS_String8 data, SYMS_DwDbgAccel *dbg)
{
  SYMS_DwUnitSetAccel *unit_set_accel = syms_push_array(arena, SYMS_DwUnitSetAccel, 1);
  syms_memzero_struct(unit_set_accel);
  unit_set_accel->format = SYMS_FileFormat_DWARF;
  unit_set_accel->root_count = dbg->unit_count;
  unit_set_accel->roots = syms_push_array_zero(arena, SYMS_DwCompRoot, unit_set_accel->root_count);
  for(SYMS_U64 unit_idx = 0; unit_idx < dbg->unit_count; unit_idx += 1)
  {
    unit_set_accel->roots[unit_idx] = syms_dw_comp_root_from_range(arena, data, dbg, unit_idx, dbg->unit_range_info[unit_idx].frange);
  }
  return unit_set_accel;
}

SYMS_API SYMS_U64
syms_dw_unit_count_from_set(SYMS_DwUnitSetAccel *accel)
{
  return accel->root_count;
}

SYMS_API SYMS_DwCompRoot*
syms_dw_comp_root_from_uid(SYMS_DwUnitSetAccel *accel, SYMS_UnitID uid){
  SYMS_DwCompRoot *root = &syms_dw_nil_comp_root;
  SYMS_U64 num = syms_dw_unit_number_from_uid(accel, uid);
  if(0 < num && num <= accel->root_count)
  {
    root = &accel->roots[uid-1];
  }
  return root;
}

SYMS_API SYMS_UnitID
syms_dw_uid_from_number(SYMS_DwUnitSetAccel *accel, SYMS_U64 n)
{
  return n;
}

SYMS_API SYMS_U64
syms_dw_unit_number_from_uid(SYMS_DwUnitSetAccel *accel, SYMS_UnitID uid){
  return uid;
}

SYMS_API SYMS_UnitInfo
syms_dw_unit_info_from_uid(SYMS_DwUnitSetAccel *unit_set, SYMS_UnitID uid)
{
  SYMS_UnitInfo result = {0};
  SYMS_DwCompRoot *comp_root = syms_dw_comp_root_from_uid(unit_set, uid);
  result.features      = (SYMS_UnitFeature_CompilationUnit|
                          SYMS_UnitFeature_Types|
                          SYMS_UnitFeature_StaticVariables|
                          SYMS_UnitFeature_ExternVariables|
                          SYMS_UnitFeature_Functions);
  result.language      = syms_dw_base_language_from_dw_language((SYMS_DwLanguage)comp_root->language);
  return(result);
}

SYMS_API SYMS_UnitNames
syms_dw_unit_names_from_uid(SYMS_Arena *arena, SYMS_DwUnitSetAccel *unit_set, SYMS_UnitID uid)
{
  SYMS_UnitNames result = {0};
  SYMS_DwCompRoot *comp_root = syms_dw_comp_root_from_uid(unit_set, uid);
  result.source_file   = syms_push_string_copy(arena, comp_root->name);
  result.compiler      = syms_push_string_copy(arena, comp_root->producer);
  result.compile_dir   = syms_push_string_copy(arena, comp_root->compile_dir);
  return(result);
}

SYMS_API void
syms_dw_sort_unit_range_point_array_in_place__merge(SYMS_DwUnitRangePoint *a, SYMS_U64 left, SYMS_U64 right, SYMS_U64 end, SYMS_DwUnitRangePoint *b)
{
  SYMS_U64 left_i = left;
  SYMS_U64 right_i = right;
  for(SYMS_U64 idx = left; idx < end; idx += 1)
  {
    if(left_i < right && (right_i >= end || a[left_i].p <= a[right_i].p))
    {
      b[idx] = a[left_i];
      left_i += 1;
    }
    else
    {
      b[idx] = a[right_i];
      right_i += 1;
    }
  }
}

SYMS_API void
syms_dw_sort_unit_range_point_array_in_place(SYMS_DwUnitRangePoint *a, SYMS_U64 count)
{
  SYMS_ArenaTemp scratch = syms_get_scratch(0, 0);
  SYMS_DwUnitRangePoint *b = syms_push_array_zero(scratch.arena, SYMS_DwUnitRangePoint, count);
  SYMS_DwUnitRangePoint *sort_array = a;
  SYMS_DwUnitRangePoint *scratch_array = b;
  for(SYMS_U64 run_width = 1; run_width < count; run_width *= 2)
  {
    for(SYMS_U64 i = 0; i < count; i += 2*run_width)
    {
      syms_dw_sort_unit_range_point_array_in_place__merge(sort_array, i, SYMS_MIN(i+run_width, count), SYMS_MIN(i + 2*run_width, count), scratch_array);
    }
    SYMS_Swap(SYMS_DwUnitRangePoint *, sort_array, scratch_array);
  }
  if(scratch_array == a)
  {
    syms_memmove(a, sort_array, count*sizeof(*a));
  }
  syms_release_scratch(scratch);
}

SYMS_API SYMS_UnitRangeArray
syms_dw_unit_ranges_from_set(SYMS_Arena *arena, SYMS_String8 data, SYMS_DwDbgAccel *dbg, SYMS_DwUnitSetAccel *unit_set)
{
  SYMS_ArenaTemp scratch = syms_get_scratch(&arena, 1);
  
  SYMS_U64 unit_count            = dbg->unit_count;
  SYMS_U64RangeList *unit_ranges = syms_push_array_zero(scratch.arena, SYMS_U64RangeList, unit_count);
  
  //- rjf: fill ranges that came from the .debug_aranges section
  for(SYMS_U64 unit_idx = 0; unit_idx < unit_count; unit_idx += 1)
  {
    for(SYMS_U64 rng_idx = 0; rng_idx < dbg->unit_range_info[unit_idx].addr_ranges.count; rng_idx += 1)
    {
      syms_u64_range_list_push(scratch.arena, &unit_ranges[unit_idx], dbg->unit_range_info[unit_idx].addr_ranges.ranges[rng_idx]);
    }
  }
  
  //- rjf: fill range list for all compilation unit ranges/low-pc/high-pc data
  //
  // NOTE(rjf): .debug_aranges is apparently not entirely complete, and will not
  // include all the ranges we care about. The LLVM function (at 9/14/2021)
  // DWARFDebugAranges::generate seems to suggest this. We also need to rely on
  // the DW_AT_ranges and DW_AT_low_pc/high_pc attributes to build a complete
  // mapping of all compilation unit ranges.
  //
  for(SYMS_U64 unit_idx = 0; unit_idx < unit_count; unit_idx += 1)
  {
    SYMS_DwCompRoot *root = syms_dw_comp_root_from_uid(unit_set, unit_idx+1);
    SYMS_U64RangeList rngs = syms_dw_range_list_from_high_low_pc_and_ranges_attrib_value(scratch.arena, data, dbg,
                                                                                         root->address_size,
                                                                                         root->base_addr,
                                                                                         root->addrs_base,
                                                                                         root->low_pc,
                                                                                         root->high_pc,
                                                                                         root->ranges_attrib_value);
    syms_u64_range_list_concat(&unit_ranges[unit_idx], &rngs);
  }
  
  //- rjf: calc num of all ranges
  SYMS_U64 range_count = 0;
  for(SYMS_U64 unit_idx = 0; unit_idx < unit_count; unit_idx += 1)
  {
    range_count += unit_ranges[unit_idx].node_count;
  }
  
  //- rjf: bake
  SYMS_UnitRange *ranges = syms_push_array_zero(scratch.arena, SYMS_UnitRange, range_count);
  {
    SYMS_UnitRange *range_ptr = ranges;
    for(SYMS_U64 unit_idx = 0; unit_idx < unit_count; unit_idx += 1)
    {
      for(SYMS_U64RangeNode *n = unit_ranges[unit_idx].first; n != 0; n = n->next)
      {
        range_ptr->uid = unit_idx + 1;
        range_ptr->vrange = n->range;
        range_ptr += 1;
      }
    }
  }
  
  //- rjf: pointify
  SYMS_U64 rng_point_count = range_count*2;
  SYMS_DwUnitRangePoint *rng_points = syms_push_array_zero(scratch.arena, SYMS_DwUnitRangePoint, rng_point_count);
  {
    SYMS_U64 rng_point_write_idx = 0;
    for(SYMS_U64 rng_idx = 0; rng_idx < range_count; rng_idx += 1)
    {
      SYMS_U64Range vrange = ranges[rng_idx].vrange;
      SYMS_UnitID uid = ranges[rng_idx].uid;
      if(vrange.min != 0xffffffffffffffff && vrange.min != 0 && syms_u64_range_size(vrange) != 0)
      {
        rng_points[rng_point_write_idx].p      = vrange.min;
        rng_points[rng_point_write_idx].uid    = uid;
        rng_points[rng_point_write_idx].is_min = 1;
        rng_point_write_idx += 1;
        rng_points[rng_point_write_idx].p      = vrange.max;
        rng_points[rng_point_write_idx].uid    = uid;
        rng_points[rng_point_write_idx].is_min = 0;
        rng_point_write_idx += 1;
      }
    }
    rng_point_count = rng_point_write_idx;
  }
  
  //- rjf: sort pointified ranges
#if 1
  syms_dw_sort_unit_range_point_array_in_place(rng_points, rng_point_count);
#else
  for(SYMS_U64 i = 0; i < rng_point_count; i += 1)
  {
    SYMS_U64 min_idx = 0;
    SYMS_U64 min_p = 0;
    for(SYMS_U64 j = i+1; j < rng_point_count; j += 1)
    {
      if(min_idx == 0 || rng_points[j].p < min_p)
      {
        min_idx = j;
        min_p = rng_points[j].p;
      }
    }
    if(min_idx != 0)
    {
      SYMS_Swap(SYMS_DwUnitRangePoint, rng_points[i], rng_points[min_idx]);
    }
  }
#endif
  
  //- rjf: build final range list, collapse pairs into ranges
  typedef struct SYMS_DwUnitRangeNode SYMS_DwUnitRangeNode;
  struct SYMS_DwUnitRangeNode
  {
    SYMS_DwUnitRangeNode *next;
    SYMS_UnitID uid;
    SYMS_U64Range vrange;
  };
  SYMS_DwUnitRangeNode *first_unit_range_node = 0;
  SYMS_DwUnitRangeNode *last_unit_range_node = 0;
  SYMS_U64 unit_range_node_count = 0;
  {
    if(rng_point_count > 0)
    {
      SYMS_UnitID active_uid = rng_points[0].uid;
      SYMS_U64 active_rng_min = rng_points[0].p;
      SYMS_U64 depth = 1;
      SYMS_U64 last_max = 0;
      for(SYMS_U64 p_idx = 1; p_idx < rng_point_count; p_idx += 1)
      {
        SYMS_B32      push = syms_false;
        SYMS_U64Range push_vrange = {0};
        SYMS_UnitID   push_uid = 0;
        
        //- rjf: point in the same UID
        if(rng_points[p_idx].uid == active_uid)
        {
          // rjf: min => nested UID range, ignore + increase depth
          if(rng_points[p_idx].is_min)
          {
            depth += 1;
          }
          // rjf: max => end of some range, decrement depth and add new range if we're out of all hierarchies
          else
          {
            depth -= 1;
            if(depth == 0)
            {
              push = syms_true;
              push_vrange = syms_make_u64_range(active_rng_min, rng_points[p_idx].p);
              push_uid = active_uid;
              active_uid = 0;
              active_rng_min = 0;
            }
          }
        }
        //- rjf: different uid + new min, end existing range + start a new one
        else if(rng_points[p_idx].is_min)
        {
          if(active_uid != 0)
          {
            push = syms_true;
            push_vrange = syms_make_u64_range(active_rng_min, rng_points[p_idx].p);
            push_uid = active_uid;
          }
          active_uid = rng_points[p_idx].uid;
          active_rng_min = rng_points[p_idx].p;
          depth = 1;
        }
        //- rjf: max for UID we're not focusing --- this generally means we were "interrupted" by
        // another UID -- so take the last max we produced, and make a range out of it
        else if(last_max != 0)
        {
          push = syms_true;
          push_vrange = syms_make_u64_range(last_max, rng_points[p_idx].p);
          push_uid = rng_points[p_idx].uid;
          active_uid = 0;
          active_rng_min = 0;
          depth = 0;
        }
        
        //- rjf: push if we got a new range
        if(push && push_vrange.min != 0xffffffffffffffff && push_vrange.min != 0 && syms_u64_range_size(push_vrange) != 0)
        {
          SYMS_DwUnitRangeNode *n = syms_push_array_zero(scratch.arena, SYMS_DwUnitRangeNode, 1);
          n->uid = push_uid;
          n->vrange = push_vrange;
          SYMS_QueuePush(first_unit_range_node, last_unit_range_node, n);
          unit_range_node_count += 1;
          last_max = push_vrange.max;
        }
      }
    }
  }
  
  //- rjf: bake+fill+return
  SYMS_UnitRangeArray result = {0};
  {
    SYMS_LogOpen(SYMS_LogFeature_DwarfUnitRanges, 0, log);
    result.count = unit_range_node_count;
    result.ranges = syms_push_array_zero(arena, SYMS_UnitRange, result.count);
    SYMS_U64 idx = 0;
    for(SYMS_DwUnitRangeNode *n = first_unit_range_node; n != 0; n = n->next, idx += 1)
    {
      result.ranges[idx].uid = n->uid;
      result.ranges[idx].vrange = n->vrange;
      SYMS_Log("$rng: %i => [0x%" SYMS_PRIx64 ", 0x%" SYMS_PRIx64 ")\n", (int)n->uid, n->vrange.min, n->vrange.max);
    }
    SYMS_LogClose(log);
  }
  syms_release_scratch(scratch);
  return result;
}

SYMS_API SYMS_String8Array
syms_dw_file_table_from_uid(SYMS_Arena *arena, SYMS_String8 data, SYMS_DwDbgAccel *dbg,
                            SYMS_DwUnitSetAccel *unit_set, SYMS_UnitID uid)
{
  SYMS_String8Array result = {0};
  SYMS_DwCompRoot *root = syms_dw_comp_root_from_uid(unit_set, uid);
  if(root->file_table.count != 0)
  {
    SYMS_U64 count = root->file_table.count;
    SYMS_String8 *strings = syms_push_array_zero(arena, SYMS_String8, count);
    
    SYMS_String8 *string_ptr = strings;
    for(SYMS_U64 idx = 0; idx < count; idx += 1, string_ptr += 1)
    {
      SYMS_DwLineFile file = root->file_table.v[idx];
      SYMS_String8 dir_string = root->dir_table.strings[file.dir_idx];
      SYMS_String8 full_string = syms_dw_path_from_dir_and_filename(arena, dir_string, file.file_name);
      *string_ptr = full_string;
    }
    
    result.strings = strings;
    result.count = count;
  }
  return(result);
}

////////////////////////////////
//~ rjf: Tag Reference Data Structure

SYMS_API SYMS_DwTagRefTable
syms_dw_tag_ref_table_make(SYMS_Arena *arena, SYMS_U64 size)
{
  SYMS_DwTagRefTable table = {0};
  table.size = size;
  table.v = syms_push_array_zero(arena, SYMS_DwTagRefNode *, size);
  return table;
}

SYMS_API void
syms_dw_tag_ref_table_insert(SYMS_Arena *arena, SYMS_DwTagRefTable *table, SYMS_SymbolID src, SYMS_SymbolID dst)
{
  SYMS_U64 hash = syms_dw_hash_from_sid(dst);
  SYMS_U64 idx = hash % table->size;
  SYMS_DwTagRefNode *n = syms_push_array_zero(arena, SYMS_DwTagRefNode, 1);
  n->dst = dst;
  n->src = src;
  SYMS_StackPush_N(table->v[idx], n, hash_next);
}

SYMS_API SYMS_SymbolID
syms_dw_tag_ref_table_lookup_src(SYMS_DwTagRefTable table, SYMS_SymbolID dst)
{
  SYMS_SymbolID result = {0};
  SYMS_U64 hash = syms_dw_hash_from_sid(dst);
  SYMS_U64 idx = hash % table.size;
  for(SYMS_DwTagRefNode *n = table.v[idx]; n != 0; n = n->hash_next)
  {
    if(n->dst == dst)
    {
      result = n->src;
      break;
    }
  }
  return result;
}

////////////////////////////////
//~ rjf: Unit Symbol Accelerator

SYMS_API SYMS_U64
syms_dw_primify_table_size(SYMS_U64 v)
{
  SYMS_LOCAL SYMS_U64 primes[] =
  {
    4073, 5821, 7369, 7919, 8971, 10687, 13217, 14639, 16193, 17389, 19373,
    22123, 24517, 26029, 27259, 29633, 32917, 35381, 37139, 37813, 39631, 
    42641, 45263, 47653, 54037, 57119, 59183, 62927, 66103, 69389, 70657,
    72467, 76819, 80051, 84017, 87071, 93179, 97303, 99817, 102397, 104677,
    112967, 116371, 128111, 161407, 178301, 187963, 200003, 249439, 312583,
    411637, 466019, 545959, 745709, 796571, 862177, 918539, 1032683, 1187239,
    1213633, 1299827,
  };
  SYMS_U64 result = 0;
  SYMS_U64 low = 0;
  SYMS_U64 high = SYMS_ARRAY_SIZE(primes);
  for(;;)
  {
    SYMS_U64 mid = (high + low) / 2;
    SYMS_U64 left = mid > 0 ? primes[mid-1] : 4073;
    SYMS_U64 right = primes[mid];
    if(left <= v && v <= right)
    {
      result = left;
      break;
    }
    else if(v < left)
    {
      high = mid;
    }
    else if(right < v)
    {
      low = mid;
    }
  }
  return result;
}

SYMS_API SYMS_U64
syms_dw_predict_good_stub_table_size_from_range_size(SYMS_U64 size)
{
  // NOTE(rjf): currently, we're caching somewhere around ~25% of the tags we parse;
  // so we only need table size for that
  SYMS_F64 pct_cached = 0.2;
  
  SYMS_F64 size_f = (SYMS_F64)size;
  SYMS_F64 a = -0.0000000001700537876;
  SYMS_F64 b = 0.1051854269;
  SYMS_F64 c = 3748.774284;
  SYMS_F64 table_size_f = (a * size_f*size_f) + (b * size_f) + c;
  table_size_f *= pct_cached;
  SYMS_U64 table_size = (SYMS_U64)table_size_f;
  table_size = SYMS_AlignPow2(table_size, 1024);
  table_size = SYMS_MAX(table_size, 4096);
  table_size = syms_dw_primify_table_size(table_size);
  return table_size;
}

SYMS_API SYMS_DwUnitAccel *
syms_dw_unit_accel_from_comp_root(SYMS_Arena *arena, SYMS_String8 data, SYMS_DwDbgAccel *dbg, SYMS_DwCompRoot *comp_root)
{
  SYMS_ProfBegin("syms_dw_unit_accel_from_comp_root");
  SYMS_ArenaTemp scratch = syms_get_scratch(&arena, 1);
  SYMS_DwUnitAccel *unit = syms_push_array(arena, SYMS_DwUnitAccel, 1);
  syms_memzero_struct(unit);
  
  //- rjf: initialize accelerator
  unit->format       = dbg->format;
  unit->uid          = comp_root->index + 1;
  unit->version      = comp_root->version;
  unit->address_size = comp_root->address_size;
  unit->base_addr    = comp_root->base_addr;
  unit->addrs_base   = comp_root->addrs_base;
  unit->language     = comp_root->language;
  unit->abbrev_table.count = comp_root->abbrev_table.count;
  unit->abbrev_table.entries = (SYMS_DwAbbrevTableEntry *)syms_push_string_copy(arena, syms_str8((SYMS_U8 *)comp_root->abbrev_table.entries,
                                                                                                 sizeof(SYMS_DwAbbrevTableEntry) * unit->abbrev_table.count)).str;
  unit->stub_table_size = syms_dw_predict_good_stub_table_size_from_range_size(comp_root->tags_info_range.max - comp_root->tags_info_range.min);
  unit->stub_table = syms_push_array_zero(arena, SYMS_DwTagStubCacheNode *, unit->stub_table_size);
  // TODO(rjf): This is an arbitrary choice of size. Can we predict it? Or tune it to
  // common cases at least?
  unit->ref_table = syms_dw_tag_ref_table_make(arena, 4096);
  // TODO(rjf): This is an arbitrary choice of size. Can we predict it? Or tune it to
  // common cases at least?
  unit->parent_table = syms_dw_tag_ref_table_make(arena, 4096);
  unit->resolve_params = syms_dw_attrib_value_resolve_params_from_comp_root(comp_root);
  
  //- rjf: parse loop
  SYMS_U64 tags_parsed = 0;
  SYMS_U64 cache_nodes_inserted = 0;
  SYMS_DwSymbolIDChunkList top_ids = {0};
  SYMS_DwSymbolIDChunkList proc_ids = {0};
  SYMS_DwSymbolIDChunkList var_ids = {0};
  SYMS_DwSymbolIDChunkList type_ids = {0};
  {
    typedef struct SYMS_DwParseParent SYMS_DwParseParent;
    struct SYMS_DwParseParent
    {
      SYMS_DwParseParent *next;
      SYMS_DwTagStubCacheNode *parent;
      SYMS_DwTagStub parent_stub;
    };
    SYMS_DwParseParent *parent_stack = 0;
    SYMS_U64 tree_depth = 0;
    
    for(SYMS_U64 info_off = comp_root->tags_info_range.min; info_off < comp_root->tags_info_range.max;)
    {
      SYMS_Arena *conflicts[] = { scratch.arena, arena };
      SYMS_ArenaTemp per_tag_scratch = syms_get_scratch(conflicts, SYMS_ARRAY_SIZE(conflicts));
      
      //- rjf: parse tag
      SYMS_DwTag *tag = syms_dw_tag_from_info_offset(per_tag_scratch.arena, data, dbg, comp_root->abbrev_table, comp_root->language, comp_root->version, comp_root->address_size, info_off);
      SYMS_B32 good_tag = tag->abbrev_id != 0;
      info_off = tag->info_range.max;
      if(good_tag)
      {
        tags_parsed += 1;
      }
      
      //- rjf: convert tag to stub
      SYMS_DwTagStub stub = {0};
      if(good_tag)
      {
        stub = syms_dw_stub_from_tag(data, dbg, unit->resolve_params, tag);
      }
      
      //- rjf: classify + store ID into classification lists
      if(good_tag)
      {
        SYMS_SymbolKind symbol_kind = syms_dw_symbol_kind_from_tag_stub(data, dbg, syms_dw_attrib_value_resolve_params_from_comp_root(comp_root), &stub);
        switch(symbol_kind)
        {
          default: break;
          case SYMS_SymbolKind_Procedure:             syms_dw_symbol_id_chunk_list_push(scratch.arena, &proc_ids, stub.sid); break;
          case SYMS_SymbolKind_Type:                  syms_dw_symbol_id_chunk_list_push(scratch.arena, &type_ids, stub.sid); break;
          case SYMS_SymbolKind_ImageRelativeVariable: syms_dw_symbol_id_chunk_list_push(scratch.arena, &var_ids, stub.sid); break;
        }
        if(tree_depth <= 1 ||
           symbol_kind == SYMS_SymbolKind_Procedure ||
           symbol_kind == SYMS_SymbolKind_ImageRelativeVariable ||
           symbol_kind == SYMS_SymbolKind_Type)
        {
          syms_dw_symbol_id_chunk_list_push(scratch.arena, &top_ids, stub.sid); 
        }
      }
      
      //- rjf: determine if this tag should be cached
      SYMS_B32 should_be_cached = 0;
      if(good_tag)
      {
        // NOTE(rjf): only cache nodes when we either don't have any parents at all
        // (in which case we need to make some), or when our active parent was cached
        // also.
        should_be_cached = parent_stack == 0 || parent_stack->parent != 0;
        
        // NOTE(rjf): any of the listed tag kinds will not be cached, and will be re-parsed
        // upon being queried by a caller:
        switch(tag->kind)
        {
          default: break;
          
          case SYMS_DwTagKind_IMPORTED_DECLARATION:
          case SYMS_DwTagKind_UNSPECIFIED_PARAMETERS:
          case SYMS_DwTagKind_VARIANT:
          case SYMS_DwTagKind_COMMON_BLOCK:
          case SYMS_DwTagKind_COMMON_INCLUSION:
          case SYMS_DwTagKind_MODULE:
          case SYMS_DwTagKind_WITH_STMT:
          case SYMS_DwTagKind_ACCESS_DECLARATION:
          case SYMS_DwTagKind_DWARF_PROCEDURE:
          case SYMS_DwTagKind_IMPORTED_MODULE:
          case SYMS_DwTagKind_TEMPLATE_ALIAS:
          
          //case SYMS_DwTagKind_STRUCTURE_TYPE:
          //case SYMS_DwTagKind_CLASS_TYPE:
          //case SYMS_DwTagKind_UNION_TYPE:
          //case SYMS_DwTagKind_ENUMERATION_TYPE:
          //case SYMS_DwTagKind_SUBPROGRAM:
          case SYMS_DwTagKind_POINTER_TYPE:
          case SYMS_DwTagKind_REFERENCE_TYPE:
          case SYMS_DwTagKind_SUBROUTINE_TYPE:
          case SYMS_DwTagKind_TYPEDEF:
          //case SYMS_DwTagKind_SUBPROGRAM:
          {
            should_be_cached = 0;
          }break;
        }
      }
      
      //- rjf: allocate/fill cache node
      SYMS_DwTagStubCacheNode *stub_node = 0;
      if(should_be_cached)
      {
        stub_node = syms_push_array_zero(arena, SYMS_DwTagStubCacheNode, 1);
        stub_node->stub = stub;
      }
      
      //- rjf: link stub cache node into hash table
      if(stub_node != 0)
      {
        cache_nodes_inserted += 1;
        SYMS_U64 hash = syms_dw_hash_from_sid(stub_node->stub.sid);
        SYMS_U64 idx = hash % unit->stub_table_size;
        // printf("%16" SYMS_PRIu64 ", %16" SYMS_PRIu64 ", %16" SYMS_PRIu64 ", %16" SYMS_PRIu64 "\n", stub_node->stub.sid, hash, idx, unit->stub_table_size);
        stub_node->hash_next = unit->stub_table[idx];
        unit->stub_table[idx] = stub_node;
      }
      
      //- rjf: insert stub cache node into tree
      if(stub_node != 0)
      {
        if(parent_stack)
        {
          SYMS_DwTagStubCacheNode *closest_ancestor = 0;
          for(SYMS_DwParseParent *p = parent_stack; p != 0; p = p->next)
          {
            if(p->parent)
            {
              closest_ancestor = p->parent;
              break;
            }
          }
          if(closest_ancestor)
          {
            SYMS_QueuePush(closest_ancestor->first, closest_ancestor->last, stub_node);
            closest_ancestor->children_count += 1;
          }
        }
        if(unit->stub_root == 0)
        {
          unit->stub_root = stub_node;
        }
      }
      
      // NOTE(rjf): Sometimes, DWARF splits info for a single symbol across multiple
      // tags. In this case, we'll have an earlier tag (which may or may not be
      // cached) that is completed by another tag that points to it. This is pretty
      // inconvenient, because we often watn to go the other direction -- from the
      // earlier tag *to* its completion, so we can get full info from either
      // direction. Usually these are only pairs, it seems - there is usually only
      // one specification per tag that has any (I have not seen any cases to the
      // contrary, at least). A tag with the SYMS_DwAttribKind_SPECIFICATION attrib
      // should always come *AFTER* the tag of the associated declaration.
      //
      // For more info, see DWARF V4 spec (June 10, 2010), page 36.
      //
      // In any case, here we've found a tag that is "completing" an earlier tag.
      // That earlier tag may not have been cached, so we cannot simply write to
      // that earlier tag and expect everything to be OK. If that earlier tag was
      // not cached, we'll need to cache the fact that it was referenced, and then
      // any time we get a tag stub (either from the cache or from reparsing), we
      // can functionally query that table and always get correct results,
      // irrespective of what was cached when.
      //
      // In short, we can't get away with not caching something here, because of
      // the fact that we're parsing a relationship between two seemingly-unrelated
      // (at least with respect to the tree structure) symbol IDs.
      //
      // We *can* write to the completed tag stub *if it was cached*, but otherwise
      // caching the reference into the table is necessary.
      //
      if(stub.ref != 0)
      {
        SYMS_DwTagStubCacheNode *ref_node = syms_dw_tag_stub_cache_node_from_sid(unit, stub.ref);
        // NOTE(rjf): if we cached the tag that is being referenced, we can
        // write to its reference slot here to point back at this tag.
        if(ref_node != 0)
        {
          ref_node->stub.ref = stub.sid;
        }
        // NOTE(rjf): otherwise, we'll need to remember this link being formed
        // later so that when we query the completed tag, we'll figure out who
        // completed it.
        else
        {
          syms_dw_tag_ref_table_insert(arena, &unit->ref_table, stub.sid, stub.ref);
        }
      }
      
      //- rjf: fill out parent ref for UDTs that might add a namespace
      if(stub.sid != 0 &&
         parent_stack != 0 &&
         parent_stack->parent_stub.sid != 0 &&
         (parent_stack->parent_stub.kind == SYMS_DwTagKind_STRUCTURE_TYPE ||
          parent_stack->parent_stub.kind == SYMS_DwTagKind_UNION_TYPE ||
          parent_stack->parent_stub.kind == SYMS_DwTagKind_CLASS_TYPE))
      {
        syms_dw_tag_ref_table_insert(arena, &unit->parent_table, parent_stack->parent_stub.sid, stub.sid);
      }
      
      //- rjf: update parent
      SYMS_B32 good_parent = syms_false;
      {
        // NOTE(rjf): good tag with children => push this artifact as parent
        if(good_tag)
        {
          if(tag->has_children)
          {
            SYMS_DwParseParent *n = syms_push_array_zero(scratch.arena, SYMS_DwParseParent, 1);
            SYMS_StackPush(parent_stack, n);
            n->parent = stub_node;
            n->parent_stub = stub;
            tree_depth += 1;
          }
        }
        // NOTE(rjf): bad tag => if there's an active parent, go back up to its parent.
        // otherwise, we are done
        else
        {
          if(parent_stack != 0)
          {
            parent_stack = parent_stack->next;
            tree_depth -= 1;
          }
        }
        good_parent = parent_stack != 0;
      }
      
      syms_release_scratch(per_tag_scratch);
      if(good_parent == 0)
      {
        break;
      }
    }
  }
  
  //- rjf: construct synthetic stub for void type
  {
    SYMS_DwTagStub void_stub = {0};
    void_stub.sid = SYMS_DWARF_VOID_TYPE_ID;
    void_stub.kind = SYMS_DwTagKind_BASE_TYPE;
    SYMS_DwTagStubCacheNode *void_stub_node = syms_push_array_zero(arena, SYMS_DwTagStubCacheNode, 1);
    void_stub_node->stub = void_stub;
    
    //- rjf: link into tree
    SYMS_QueuePush(unit->stub_root->first, unit->stub_root->last, void_stub_node);
    unit->stub_root->children_count += 1;
    
    //- rjf: link into hash table
    SYMS_U64 hash = syms_dw_hash_from_sid(void_stub.sid);
    SYMS_U64 idx = hash % unit->stub_table_size;
    void_stub_node->hash_next = unit->stub_table[idx];
    unit->stub_table[idx] = void_stub_node;
    
    syms_dw_symbol_id_chunk_list_push(scratch.arena, &type_ids, void_stub.sid);
    syms_dw_symbol_id_chunk_list_push(scratch.arena, &top_ids, void_stub.sid);
  }
  
  //- rjf: fill classified top-level symbol arrays
  {
    SYMS_ProfBegin("syms_dw_unit_accel_from_comp_root.top_level_array_build");
    unit->all_top_ids = syms_dw_sid_array_from_chunk_list(arena, top_ids);
    unit->proc_ids    = syms_dw_sid_array_from_chunk_list(arena, proc_ids);
    unit->var_ids     = syms_dw_sid_array_from_chunk_list(arena, var_ids);
    unit->type_ids    = syms_dw_sid_array_from_chunk_list(arena, type_ids);
    SYMS_ProfEnd();
  }
  
  //- rjf: log statistics about which tag kinds were cached
#if RJF_DEBUG && 0
  {
    SYMS_LogOpen(SYMS_LogFeature_DwarfTags, unit->uid, log);
    SYMS_Log("$tag_profile\n");
    
    // NOTE(rjf): pct odd hashes vs. pct even hashes
#if 0
    {
      SYMS_U64 odd = 0;
      SYMS_U64 even = 0;
      SYMS_U64 total = 0;
      for(SYMS_U64 idx = 0; idx < unit->stub_table_size; idx += 1)
      {
        for(SYMS_DwTagStubCacheNode *n = unit->stub_table[idx]; n != 0; n = n->hash_next)
        {
          total += 1;
          SYMS_U64 hash = syms_dw_hash_from_sid(n->stub.sid);
          if(hash & 1)
          {
            odd += 1;
          }
          else
          {
            even += 1;
          }
        }
      }
      printf("%16f %16f\n", (SYMS_F32)odd / total, (SYMS_F32)even / total);
    }
#endif
    
    // NOTE(rjf): size, # of empty buckets, # of buckets total
#if 0
    {
      SYMS_U64 zeroes = 0;
      for(SYMS_U64 idx = 0; idx < unit->stub_table_size; idx += 1)
      {
        if(unit->stub_table[idx] == 0)
        {
          zeroes += 1;
        }
      }
      printf("%16" SYMS_PRIu64 "       %16" SYMS_PRIu64 "       %16" SYMS_PRIu64 "\n", comp_root->tags_info_range.max-comp_root->tags_info_range.min, zeroes,
             unit->stub_table_size);
    }
#endif
    
    // NOTE(rjf): size, ratio of buckets filled, ratio of stubs vs. table size
#if 0
    {
      SYMS_U64 count = 0;
      SYMS_U64 zeroes = 0;
      for(SYMS_U64 idx = 0; idx < unit->stub_table_size; idx += 1)
      {
        if(unit->stub_table[idx] == 0)
        {
          zeroes += 1;
        }
        for(SYMS_DwTagStubCacheNode *n = unit->stub_table[idx]; n != 0; n = n->hash_next)
        {
          count += 1;
        }
      }
      printf("%24" SYMS_PRIu64 ", %24f, %24f, %24f\n",
             comp_root->tags_info_range.max-comp_root->tags_info_range.min,
             1.f - (SYMS_F32)zeroes / (SYMS_F32)unit->stub_table_size,
             (SYMS_F32)count / (SYMS_F32)unit->stub_table_size,
             (SYMS_F32)count / (SYMS_F32)tags_parsed);
    }
#endif
    
    // NOTE(rjf): size, # of collisions
#if 0
    {
      SYMS_U64 collisions = 0;
      for(SYMS_U64 idx = 0; idx < unit->stub_table_size; idx += 1)
      {
        SYMS_U64 collisions_this_slot = 0;
        for(SYMS_DwTagStubCacheNode *n = unit->stub_table[idx]; n != 0; n = n->hash_next)
        {
          if(n->hash_next)
          {
            collisions_this_slot += 1;
          }
        }
        collisions += collisions_this_slot;
      }
      printf("%16" SYMS_PRIu64 "       %16" SYMS_PRIu64 "\n", comp_root->tags_info_range.max-comp_root->tags_info_range.min, collisions);
    }
#endif
    
    // NOTE(rjf): table stats
#if 0
    {
      SYMS_U64 table_size = unit->stub_table_size;
      SYMS_U64 num_cached =0;
      SYMS_U64 zeroes = 0;
      SYMS_U64 n = 0;
      SYMS_U64 worst_case_chain = 0;
      SYMS_F64 avg = 0;
      for(SYMS_U64 idx = 0; idx < unit->stub_table_size; idx += 1)
      {
        SYMS_U64 length = 0;
        for(SYMS_DwTagStubCacheNode *n = unit->stub_table[idx]; n != 0; n = n->hash_next)
        {
          length += 1;
        }
        num_cached += length;
        if(length > 0)
        {
          worst_case_chain = SYMS_MAX(length, worst_case_chain);
          avg *= n;
          avg += length;
          n += 1;
          avg /= n;
        }
        else
        {
          zeroes += 1;
        }
      }
      
      SYMS_F64 load_factor = (SYMS_F64)num_cached / table_size;
      SYMS_F64 avg_load_variance = 0;
      SYMS_F64 var_n = 0;
      {
        for(SYMS_U64 idx = 0; idx < unit->stub_table_size; idx += 1)
        {
          SYMS_U64 length = 0;
          for(SYMS_DwTagStubCacheNode *n = unit->stub_table[idx]; n != 0; n = n->hash_next)
          {
            length += 1;
          }
          SYMS_F64 diff = (SYMS_F64)length - load_factor;
          SYMS_F64 variance = diff*diff;
          avg_load_variance *= var_n;
          avg_load_variance += variance;
          var_n += 1;
          avg_load_variance /= var_n;
        }
      }
      
      printf("range_size: %8" SYMS_PRIu64 "  |  table_size: %7" SYMS_PRIu64 "  |  num_cached: %11" SYMS_PRIu64 " (%8.2f%%)  |  num_parsed: %8" SYMS_PRIu64 "  |  num_zeroes: %7" SYMS_PRIu64 " (%8.2f%%)  |  avg_chain_length: %8f  |  worst_case_chain: %3" SYMS_PRIu64 "  |  avg_load_variance: %8f\n",
             comp_root->tags_info_range.max-comp_root->tags_info_range.min,
             table_size,
             num_cached,
             100.f * (SYMS_F32)num_cached / table_size,
             tags_parsed,
             zeroes,
             100.f * (SYMS_F32)zeroes / table_size,
             avg,
             worst_case_chain,
             avg_load_variance);
      
      // printf("%16" SYMS_PRIu64 "       %16f\n", comp_root->tags_info_range.max-comp_root->tags_info_range.min, avg);
    }
#endif
    
    // NOTE(rjf): size, avg chain length
#if 0
    {
      SYMS_U64 n = 0;
      SYMS_F32 avg = 0;
      for(SYMS_U64 idx = 0; idx < unit->stub_table_size; idx += 1)
      {
        SYMS_U64 length = 0;
        for(SYMS_DwTagStubCacheNode *n = unit->stub_table[idx]; n != 0; n = n->next)
        {
          length += 1;
        }
        if(length > 0)
        {
          avg *= n;
          avg += length;
          n += 1;
          avg /= n;
        }
      }
      printf("%16" SYMS_PRIu64 "       %16f\n", comp_root->tags_info_range.max-comp_root->tags_info_range.min, avg);
    }
#endif
    
    // NOTE(rjf): size, # of tags
#if 0
    {
      printf("%24" SYMS_PRIu64 ", %24" SYMS_PRIu64 ", %24" SYMS_PRIu64 "\n", comp_root->tags_info_range.max-comp_root->tags_info_range.min, tags_parsed, unit->stub_table_size);
    }
#endif
    
    SYMS_U64 tag_kind_counts_size = 65536;
    SYMS_U64 *tag_kind_counts = syms_push_array_zero(scratch.arena, SYMS_U64, tag_kind_counts_size);
    SYMS_U64 total = 0;
    for(SYMS_U64 idx = 0; idx < unit->all_top_ids.count; idx += 1)
    {
      SYMS_DwTagStubCacheNode *cache_node = syms_dw_tag_stub_cache_node_from_sid(unit, unit->all_top_ids.ids[idx]);
      if(cache_node != 0 && cache_node->stub.kind < tag_kind_counts_size)
      {
        tag_kind_counts[cache_node->stub.kind] += 1;
        total += 1;
      }
    }
    
    for(SYMS_U64 tag_kind = 0; tag_kind < tag_kind_counts_size; tag_kind += 1)
    {
      if(tag_kind_counts[tag_kind] != 0)
      {
        SYMS_String8 tag_kind_str = syms_string_from_enum_value(SYMS_DwTagKind, tag_kind);
        SYMS_F32 pct = (SYMS_F32)((SYMS_F64)tag_kind_counts[tag_kind] / (SYMS_F64)total);
        SYMS_Log("  %.*s: %.2f%% [%" SYMS_PRIu64 "]\n", syms_expand_string(tag_kind_str), pct*100, tag_kind_counts[tag_kind]);
      }
    }
    
    SYMS_LogClose(log);
  }
#endif
  
  SYMS_ProfEnd();
  syms_release_scratch(scratch);
  return unit;
}

SYMS_API SYMS_DwUnitAccel*
syms_dw_unit_accel_from_uid(SYMS_Arena *arena, SYMS_String8 data,
                            SYMS_DwDbgAccel *dbg,
                            SYMS_DwUnitSetAccel *unit_set,
                            SYMS_UnitID uid){
  SYMS_DwCompRoot *comp_root = syms_dw_comp_root_from_uid(unit_set, uid);
  SYMS_DwUnitAccel *result = syms_dw_unit_accel_from_comp_root(arena, data, dbg, comp_root);
  return(result);
}

SYMS_API SYMS_UnitID
syms_dw_uid_from_accel(SYMS_DwUnitAccel *unit)
{
  SYMS_UnitID id = unit->uid;
  return id;
}

SYMS_API SYMS_DwTagStubCacheNode *
syms_dw_tag_stub_cache_node_from_sid(SYMS_DwUnitAccel *unit, SYMS_SymbolID sid)
{
  SYMS_DwTagStubCacheNode *result = 0;
  SYMS_U64 hash = syms_dw_hash_from_sid(sid);
  SYMS_U64 idx = hash % unit->stub_table_size;
  for(SYMS_DwTagStubCacheNode *n = unit->stub_table[idx]; n != 0; n = n->hash_next)
  {
    if(n->stub.sid == sid)
    {
      result = n;
      break;
    }
  }
  return result;
}

SYMS_API SYMS_DwTagStub
syms_dw_tag_stub_from_sid(SYMS_String8 data, SYMS_DwDbgAccel *dbg, SYMS_DwUnitAccel *unit, SYMS_SymbolID sid)
{
  SYMS_DwTagStub stub = {0};
  SYMS_ArenaTemp scratch = syms_get_scratch(0, 0);
  SYMS_DwTag *tag = syms_dw_tag_from_info_offset(scratch.arena, data, dbg, unit->abbrev_table, unit->language, unit->version, unit->address_size, sid);
  stub = syms_dw_stub_from_tag(data, dbg, unit->resolve_params, tag);
  if(stub.ref == 0)
  {
    SYMS_SymbolID tag_that_referenced_stub = syms_dw_tag_ref_table_lookup_src(unit->ref_table, sid);
    stub.ref = tag_that_referenced_stub;
  }
  syms_release_scratch(scratch);
  return stub;
}

SYMS_API SYMS_DwTagStub
syms_dw_cached_tag_stub_from_sid__parse_fallback(SYMS_String8 data, SYMS_DwDbgAccel *dbg, SYMS_DwUnitAccel *unit, SYMS_SymbolID sid)
{
  SYMS_DwTagStub stub = {0};
  if(sid != 0)
  {
    SYMS_DwTagStubCacheNode *n = syms_dw_tag_stub_cache_node_from_sid(unit, sid);
    if(n != 0)
    {
      stub = n->stub;
    }
    else
    {
      stub = syms_dw_tag_stub_from_sid(data, dbg, unit, sid);
    }
  }
  return stub;
}

SYMS_API SYMS_DwTagStubList
syms_dw_children_from_tag_stub(SYMS_Arena *arena, SYMS_String8 data, SYMS_DwDbgAccel *dbg, SYMS_DwUnitAccel *unit, SYMS_DwTagStub stub)
{
  SYMS_DwTagStubList list = {0};
  SYMS_DwTagStubCacheNode *cache_node = syms_dw_tag_stub_cache_node_from_sid(unit, stub.sid);
  if(cache_node != 0)
  {
    for(SYMS_DwTagStubCacheNode *child = cache_node->first; child != 0; child = child->next)
    {
      SYMS_DwTagStubNode *n = syms_push_array_zero(arena, SYMS_DwTagStubNode, 1);
      n->stub = child->stub;
      SYMS_QueuePush(list.first, list.last, n);
      list.count += 1;
    }
  }
  else if(stub.children_info_off != 0)
  {
    SYMS_ArenaTemp scratch = syms_get_scratch(&arena, 1);
    SYMS_U64 depth = 0;
    for(SYMS_U64 info_off = stub.children_info_off;;)
    {
      SYMS_DwTag *child_tag = syms_dw_tag_from_info_offset(scratch.arena, data, dbg, unit->abbrev_table, unit->language, unit->version, unit->address_size, info_off);
      info_off = child_tag->info_range.max;
      
      if(depth == 0 && child_tag->kind != SYMS_DwTagKind_NULL)
      {
        SYMS_DwTagStubNode *n = syms_push_array_zero(arena, SYMS_DwTagStubNode, 1);
        n->stub = syms_dw_stub_from_tag(data, dbg, unit->resolve_params, child_tag);
        n->stub.ref = syms_dw_tag_ref_table_lookup_src(unit->ref_table, n->stub.sid);
        SYMS_QueuePush(list.first, list.last, n);
        list.count += 1;
      }
      
      if(child_tag->has_children)
      {
        depth += 1;
      }
      else if(child_tag->kind == SYMS_DwTagKind_NULL)
      {
        if(depth == 0)
        {
          break;
        }
        depth -= 1;
      }
    }
    syms_release_scratch(scratch);
  }
  return list;
}

SYMS_API SYMS_DwAttribList
syms_dw_attrib_list_from_stub(SYMS_Arena *arena, SYMS_String8 data, SYMS_DwDbgAccel *dbg, SYMS_DwLanguage lang, SYMS_DwVersion ver, SYMS_U64 addr_size, SYMS_DwTagStub *stub)
{
  SYMS_DwAttribListParseResult parse = syms_dw_parse_attrib_list_from_info_abbrev_offsets(arena, data, dbg, lang, ver, addr_size, stub->attribs_info_off, stub->attribs_abbrev_off);
  return parse.attribs;
}

SYMS_API SYMS_SymbolIDArray
syms_dw_copy_sid_array_if_needed(SYMS_Arena *arena, SYMS_SymbolIDArray arr)
{
  SYMS_SymbolIDArray result = {0};
  result.count = arr.count;
  result.ids = syms_push_array(arena, SYMS_SymbolID, result.count);
  syms_memmove(result.ids, arr.ids, sizeof(SYMS_SymbolID)*result.count);
  return result;
}

SYMS_API SYMS_SymbolIDArray
syms_dw_proc_sid_array_from_unit(SYMS_Arena *arena, SYMS_DwUnitAccel *unit)
{
  return syms_dw_copy_sid_array_if_needed(arena, unit->proc_ids);
}

SYMS_API SYMS_SymbolIDArray
syms_dw_var_sid_array_from_unit(SYMS_Arena *arena, SYMS_DwUnitAccel *unit)
{
  return syms_dw_copy_sid_array_if_needed(arena, unit->var_ids);
}

SYMS_API SYMS_SymbolIDArray
syms_dw_type_sid_array_from_unit(SYMS_Arena *arena, SYMS_DwUnitAccel *unit)
{
  return syms_dw_copy_sid_array_if_needed(arena, unit->type_ids);
}

////////////////////////////////
//~ rjf: Member Accelerator

SYMS_API SYMS_DwMemsAccel *
syms_dw_mems_accel_from_sid(SYMS_Arena *arena, SYMS_String8 data, SYMS_DwDbgAccel *dbg,
                            SYMS_DwUnitAccel *unit, SYMS_SymbolID sid)
{
  SYMS_ArenaTemp scratch = syms_get_scratch(&arena, 1);
  
  SYMS_DwMemsAccel *accel = (SYMS_DwMemsAccel *)&syms_format_nil;
  SYMS_DwTagStub stub = syms_dw_cached_tag_stub_from_sid__parse_fallback(data, dbg, unit, sid);
  if(stub.kind == SYMS_DwTagKind_STRUCTURE_TYPE ||
     stub.kind == SYMS_DwTagKind_UNION_TYPE ||
     stub.kind == SYMS_DwTagKind_CLASS_TYPE)
  {
    SYMS_DwTagStubList children = syms_dw_children_from_tag_stub(scratch.arena, data, dbg, unit, stub);
    accel = syms_push_array_zero(arena, SYMS_DwMemsAccel, 1);
    accel->format = SYMS_FileFormat_DWARF;
    accel->count = children.count;
    accel->mem_infos = syms_push_array_zero(arena, SYMS_MemInfo, accel->count);
    accel->type_symbols = syms_push_array_zero(arena, SYMS_USID, accel->count);
    accel->full_symbols = syms_push_array_zero(arena, SYMS_USID, accel->count);
    accel->sig_symbols = syms_push_array_zero(arena, SYMS_SymbolID, accel->count);
    
    SYMS_DwTagStubNode *child_n = children.first;
    for(SYMS_U64 idx = 0; idx < accel->count && child_n != 0; child_n = child_n->next)
    {
      SYMS_DwTagStub *child_stub = &child_n->stub;
      
      SYMS_MemInfo  *out_mem_info    = accel->mem_infos + idx;
      SYMS_USID     *out_type_handle = accel->type_symbols + idx;
      SYMS_USID     *out_full_handle = accel->full_symbols + idx;
      SYMS_SymbolID *out_sig_symbol  = accel->sig_symbols + idx;
      
      //- rjf: skip children we don't export in the members accelerator
      if(child_stub->kind == SYMS_DwTagKind_TEMPLATE_TYPE_PARAMETER)
      {
        accel->count -= 1;
        continue;
      }
      idx += 1;
      
      //- rjf: get child attributes
      SYMS_DwAttribList attribs = syms_dw_attrib_list_from_stub(scratch.arena, data, dbg, unit->language, unit->version, unit->address_size, child_stub);
      
      //- rjf: get info from attributes
      SYMS_String8 name = {0};
      SYMS_MemVisibility visibility = SYMS_MemVisibility_Null;
      SYMS_MemFlags flags = 0;
      SYMS_B32 is_virtual = syms_false;
      SYMS_B32 is_external = syms_false;
      SYMS_SymbolID type = syms_dw_sid_from_info_offset(SYMS_DWARF_VOID_TYPE_ID);
      SYMS_U32 offset = 0;
      SYMS_U32 virtual_offset = 0;
      for(SYMS_DwAttribNode *attrib_n = attribs.first;
          attrib_n != 0;
          attrib_n = attrib_n->next)
      {
        SYMS_DwAttrib *attrib = &attrib_n->attrib;
        SYMS_DwAttribValue value = syms_dw_attrib_value_from_form_value(data, dbg, unit->resolve_params,
                                                                        attrib->form_kind,
                                                                        attrib->value_class,
                                                                        attrib->form_value);
        
        switch(attrib->attrib_kind)
        {
          //- rjf: name
          case SYMS_DwAttribKind_NAME:
          {
            name = syms_dw_string_from_attrib_value(data, dbg, value);
          }break;
          
          //- rjf: type
          case SYMS_DwAttribKind_TYPE:
          {
            // TODO(rjf): This doesn't handle cross-unit references @dw_cross_unit
            type = syms_dw_sid_from_info_offset(value.v[0]);
          }break;
          
          //- rjf: visibility
          case SYMS_DwAttribKind_VISIBILITY:
          {
            visibility = syms_dw_mem_visibility_from_access((SYMS_DwAccess)value.v[0]);
          }break;
          
          //- rjf: virtuality
          case SYMS_DwAttribKind_VIRTUALITY:
          {
            is_virtual = (value.v[0] == SYMS_DwVirtuality_Virtual ||
                          value.v[0] == SYMS_DwVirtuality_PureVirtual);
            if(is_virtual)
            {
              flags |= SYMS_MemFlag_Virtual;
            }
          }break;
          
          //- rjf: virtual table offset
          case SYMS_DwAttribKind_VTABLE_ELEM_LOCATION:
          {
            switch(attrib->value_class)
            {
              // NOTE(rjf): I don't think these are ever reported as simple
              // offsets. There's not much we can do about that, even though
              // it'd be sufficient. Instead, we have to evaluate a DWARF
              // expression. But, I'm leaving this code here, as a desperate
              // wish to the stars that someday it'll be hit.
              case SYMS_DwAttribClass_CONST:
              {
                virtual_offset = value.v[0];
              }break;
              
              //- rjf: best-first-guess at DWARF expression evaluation. this
              // won't be totally correct, since evaluation of an arbitrary
              // DWARF expression requires information about a running process,
              // but we can hit the common cases here.
              default:
              {
                SYMS_ArenaTemp scratch = syms_get_scratch(&arena, 1);
                void *debug_info_base = syms_dw_sec_base_from_dbg(data, dbg, SYMS_DwSectionKind_Info);
                void *exprloc_base = (void *)((SYMS_U8 *)debug_info_base + value.v[0]);
                SYMS_U64Range exprloc_rng = syms_make_u64_range(0, value.v[1]);
                
                // TODO(allen): fill this in correctly
                SYMS_U64 text_base = 0;
                SYMS_DwSimpleLoc loc = syms_dw_expr__analyze_fast(exprloc_base, exprloc_rng, text_base);
                if(loc.kind == SYMS_DwSimpleLocKind_Address)
                {
                  // NOTE(rjf): These are reported in slots, but PDB reports in byte offsets,
                  // so this maps to the same space as PDB.
                  virtual_offset = loc.addr * unit->address_size;
                }
                syms_release_scratch(scratch);
              }break;
            }
          }break;
          
          //- rjf: external
          case SYMS_DwAttribKind_EXTERNAL:
          {
            is_external = syms_true;
          }break;
          
          //- rjf: bit offsets
          case SYMS_DwAttribKind_DATA_MEMBER_LOCATION:
          {
            switch(attrib->value_class)
            {
              case SYMS_DwAttribClass_CONST:
              {
                offset = value.v[0];
              }break;
              default:
              {
                // TODO(rjf): Must evaluate a DWARF expression in order to find out the byte offset. @dwarf_expr
              }break;
            }
          }break;
          case SYMS_DwAttribKind_DATA_BIT_OFFSET:
          {
            offset = value.v[0];
          }break;
          
        }
      }
      
      //- rjf: get tag-kind-specific data
      // TODO(rjf): offsets
      SYMS_MemKind kind = SYMS_MemKind_Null;
      SYMS_SymbolID sig_symbol = {0};
      SYMS_USID inferred_type_symbol = {0};
      SYMS_USID inferred_full_symbol = {0};
      switch(child_stub->kind)
      {
        default: break;
        
        case SYMS_DwTagKind_MEMBER:
        {
          SYMS_DwTagStubFlags flags = child_stub->flags;
          kind = SYMS_MemKind_DataField;
          inferred_type_symbol.uid = syms_dw_uid_from_accel(unit);
          inferred_type_symbol.sid = type;
          
          if(child_stub->ref != 0)
          {
            SYMS_DwTagStub child_ref_stub = syms_dw_cached_tag_stub_from_sid__parse_fallback(data, dbg, unit, child_stub->ref);
            inferred_full_symbol.uid = syms_dw_uid_from_accel(unit);
            inferred_full_symbol.sid = child_ref_stub.sid;
            flags |= child_ref_stub.flags;
          }
          if(flags & (SYMS_DwTagStubFlag_HasLocation|SYMS_DwTagStubFlag_HasExternal))
          {
            kind = SYMS_MemKind_StaticData;
          }
        }break;
        
        case SYMS_DwTagKind_SUBPROGRAM:
        {
          sig_symbol = child_stub->sid;
          kind = SYMS_MemKind_Method;
          
          SYMS_DwTagStub child_ref_stub = syms_dw_cached_tag_stub_from_sid__parse_fallback(data, dbg, unit, child_stub->ref);
          if(child_stub->ref != 0)
          {
            inferred_full_symbol.uid = unit->uid;
            inferred_full_symbol.sid = child_ref_stub.sid;
            if(!(child_ref_stub.flags & SYMS_DwTagStubFlag_HasObjectPointerArg))
            {
              kind = SYMS_MemKind_StaticMethod;
            }
          }
          
          //- rjf: check for constructor/destructor via best-guess string match
          {
            SYMS_String8 child_name = syms_dw_attrib_string_from_sid__unstable(data, dbg, unit, SYMS_DwAttribKind_NAME, child_stub->sid);
            SYMS_String8 child_ref_name = syms_dw_attrib_string_from_sid__unstable(data, dbg, unit, SYMS_DwAttribKind_NAME, child_ref_stub.sid);
            SYMS_String8 real_child_name = child_name.size ? child_name : child_ref_name;
            SYMS_B32 child_is_destructor = real_child_name.size > 0 && syms_string_match(real_child_name, syms_str8_lit("~"),
                                                                                         SYMS_StringMatchFlag_RightSideSloppy);
            if(child_is_destructor)
            {
              real_child_name.str += 1;
              real_child_name.size -= 1;
            }
            real_child_name = syms_string_trunc_symbol_heuristic(real_child_name);
            
            SYMS_String8 symbol_name = syms_dw_attrib_string_from_sid__unstable(data, dbg, unit, SYMS_DwAttribKind_NAME, sid);
            symbol_name = syms_string_trunc_symbol_heuristic(symbol_name);
            if(syms_string_match(symbol_name, real_child_name, 0))
            {
              if(child_is_destructor)
              {
                flags |= SYMS_MemFlag_Destructor;
              }
              else
              {
                flags |= SYMS_MemFlag_Constructor;
              }
            }
          }
          
        }break;
        
        case SYMS_DwTagKind_INHERITANCE:
        {
          kind = is_virtual ? SYMS_MemKind_VBaseClassPtr : SYMS_MemKind_BaseClass;
          inferred_type_symbol.sid = type;
          inferred_type_symbol.uid = syms_dw_uid_from_accel(unit);
        }break;
        
        case SYMS_DwTagKind_STRUCTURE_TYPE:
        case SYMS_DwTagKind_UNION_TYPE:
        case SYMS_DwTagKind_ENUMERATION_TYPE:
        case SYMS_DwTagKind_TYPEDEF:
        {
          kind = SYMS_MemKind_NestedType;
          inferred_type_symbol.sid = child_stub->sid;
          inferred_type_symbol.uid = syms_dw_uid_from_accel(unit);
        }break;
      }
      
      //- rjf: fill
      out_mem_info->kind        = kind;
      out_mem_info->name        = name;
      out_mem_info->visibility  = visibility;
      out_mem_info->flags       = flags;
      out_mem_info->off         = offset;
      out_mem_info->virtual_off = virtual_offset;
      *out_type_handle = inferred_type_symbol;
      *out_full_handle = inferred_full_symbol;
      *out_sig_symbol = sig_symbol;
    }
  }
  syms_release_scratch(scratch);
  return accel;
}

SYMS_API SYMS_U64
syms_dw_mem_count_from_mems(SYMS_DwMemsAccel *mems)
{
  return mems->count;
}

SYMS_API SYMS_MemInfo
syms_dw_mem_info_from_number(SYMS_Arena *arena, SYMS_String8 data, SYMS_DwDbgAccel *dbg,
                             SYMS_DwUnitAccel *unit, SYMS_DwMemsAccel *mems, SYMS_U64 n)
{
  SYMS_MemInfo result;
  syms_memzero_struct(&result);
  if(0 < n && n <= mems->count)
  {
    result = mems->mem_infos[n-1];
  }
  return result;
}

SYMS_API SYMS_USID
syms_dw_type_from_mem_number(SYMS_String8 data, SYMS_DwDbgAccel *dbg,
                             SYMS_DwUnitAccel *unit, SYMS_DwMemsAccel *mems,
                             SYMS_U64 n)
{
  SYMS_USID result;
  syms_memzero_struct(&result);
  if(0 < n && n <= mems->count)
  {
    result = mems->type_symbols[n-1];
  }
  return result;
}

SYMS_API SYMS_SigInfo
syms_dw_sig_info_from_mem_number(SYMS_Arena *arena, SYMS_String8 data, SYMS_DwDbgAccel *dbg,
                                 SYMS_DwUnitAccel *unit, SYMS_DwMemsAccel *mems, SYMS_U64 n)
{
  SYMS_SigInfo result;
  syms_memzero_struct(&result);
  if(0 < n && n <= mems->count)
  {
    SYMS_SymbolID sig_symbol = mems->sig_symbols[n-1];
    result = syms_dw_sig_info_from_sid(arena, data, dbg, unit, sig_symbol);
  }
  return result;
}

SYMS_API SYMS_USID
syms_dw_symbol_from_mem_number(SYMS_String8 data, SYMS_DwDbgAccel *dbg,
                               SYMS_DwUnitAccel *unit, SYMS_DwMemsAccel *mems,
                               SYMS_U64 n)
{
  SYMS_USID result;
  syms_memzero_struct(&result);
  if(0 < n && n <= mems->count)
  {
    result = mems->full_symbols[n-1];
  }
  return result;
}

SYMS_API SYMS_EnumMemberArray
syms_dw_enum_member_array_from_sid(SYMS_Arena *arena, SYMS_String8 data, SYMS_DwDbgAccel *dbg,
                                   SYMS_DwUnitAccel *unit, SYMS_SymbolID sid)
{
  SYMS_ArenaTemp scratch = syms_get_scratch(&arena, 1);
  SYMS_EnumMemberArray array;
  syms_memzero_struct(&array);
  
  SYMS_DwTagStub stub = syms_dw_cached_tag_stub_from_sid__parse_fallback(data, dbg, unit, sid);
  if(stub.sid != 0 && stub.kind == SYMS_DwTagKind_ENUMERATION_TYPE)
  {
    SYMS_DwTagStubList children = syms_dw_children_from_tag_stub(scratch.arena, data, dbg, unit, stub);
    
    SYMS_U64 count = 0;
    for(SYMS_DwTagStubNode *child_n = children.first; child_n != 0; child_n = child_n->next)
    {
      SYMS_DwTagStub *child_stub = &child_n->stub;
      if(child_stub->kind == SYMS_DwTagKind_ENUMERATOR)
      {
        count += 1;
      }
    }
    
    SYMS_EnumMember *enum_members = syms_push_array_zero(arena, SYMS_EnumMember, count);
    SYMS_U64 idx = 0;
    for(SYMS_DwTagStubNode *child_n = children.first; child_n != 0; child_n = child_n->next)
    {
      SYMS_DwTagStub *child_stub = &child_n->stub;
      if(child_stub->kind == SYMS_DwTagKind_ENUMERATOR)
      {
        SYMS_String8 name = {0};
        SYMS_U64 val = 0;
        
        SYMS_ArenaTemp temp = syms_arena_temp_begin(scratch.arena);
        SYMS_DwAttribList attribs = syms_dw_attrib_list_from_stub(temp.arena, data, dbg, unit->language, unit->version, unit->address_size, child_stub);
        SYMS_B32 got_name = syms_false;
        SYMS_B32 got_val = syms_false;
        
        for(SYMS_DwAttribNode *attrib_n = attribs.first;
            attrib_n != 0 && (!got_name || !got_val);
            attrib_n = attrib_n->next)
        {
          SYMS_DwAttrib *attrib = &attrib_n->attrib;
          SYMS_DwAttribValue attrib_value = syms_dw_attrib_value_from_form_value(data, dbg, unit->resolve_params,
                                                                                 attrib->form_kind, attrib->value_class, attrib->form_value);
          switch(attrib->attrib_kind)
          {
            case SYMS_DwAttribKind_NAME:
            {
              got_name = syms_true;
              name = syms_dw_string_from_attrib_value(data, dbg, attrib_value);
            }break;
            case SYMS_DwAttribKind_CONST_VALUE:
            {
              got_val = syms_true;
              val = attrib_value.v[0];
            }break;
          }
        }
        
        enum_members[idx].name = syms_push_string_copy(arena, name);
        enum_members[idx].val = val;
        
        syms_arena_temp_end(temp);
        idx += 1;
      }
    }
    
    array.count = count;
    array.enum_members = enum_members;
  }
  
  syms_release_scratch(scratch);
  return array;
}

SYMS_API SYMS_USID
syms_dw_containing_type_from_sid(SYMS_String8 data, SYMS_DwDbgAccel *dbg, SYMS_DwUnitAccel *unit, SYMS_SymbolID sid)
{
  SYMS_USID result = {0};
  result.uid = unit->uid;
  result.sid = syms_dw_tag_ref_table_lookup_src(unit->parent_table, sid);
  if(result.sid == 0)
  {
    SYMS_DwTagStub stub = syms_dw_cached_tag_stub_from_sid__parse_fallback(data, dbg, unit, sid);
    result.sid = syms_dw_tag_ref_table_lookup_src(unit->parent_table, stub.ref);
  }
  return result;
}

SYMS_API SYMS_String8
syms_dw_linkage_name_from_sid(SYMS_Arena *arena, SYMS_String8 data, SYMS_DwDbgAccel *dbg, SYMS_DwUnitAccel *unit, SYMS_SymbolID sid)
{
  SYMS_String8 name = syms_dw_attrib_string_from_sid__unstable_chain(data, dbg, unit, SYMS_DwAttribKind_LINKAGE_NAME, sid);
  SYMS_String8 result = syms_push_string_copy(arena, name);
  return result;
}

////////////////////////////////
//~ rjf: Full Symbol Info Parsing

SYMS_API SYMS_String8
syms_dw_attrib_string_from_sid__unstable(SYMS_String8 data, SYMS_DwDbgAccel *dbg, SYMS_DwUnitAccel *unit, SYMS_DwAttribKind kind, SYMS_SymbolID sid)
{
  SYMS_String8 name = syms_str8_lit("");
  if(sid == SYMS_DWARF_VOID_TYPE_ID)
  {
    name = syms_str8_lit("void");
  }
  else
  {
    SYMS_ArenaTemp scratch = syms_get_scratch(0, 0);
    SYMS_DwTagStub stub = syms_dw_cached_tag_stub_from_sid__parse_fallback(data, dbg, unit, sid);
    SYMS_DwAttribList attribs = syms_dw_attrib_list_from_stub(scratch.arena, data, dbg, unit->language, unit->version, unit->address_size, &stub);
    for(SYMS_DwAttribNode *n = attribs.first; n != 0; n = n->next)
    {
      SYMS_DwAttrib *attrib = &n->attrib;
      if(attrib->attrib_kind == kind)
      {
        SYMS_DwAttribValue value = syms_dw_attrib_value_from_form_value(data, dbg, unit->resolve_params,
                                                                        attrib->form_kind,
                                                                        attrib->value_class,
                                                                        attrib->form_value);
        name = syms_dw_string_from_attrib_value(data, dbg, value);
        break;
      }
    }
    syms_release_scratch(scratch);
  }
  return name;
}

SYMS_API SYMS_String8
syms_dw_attrib_string_from_sid__unstable_chain(SYMS_String8 data, SYMS_DwDbgAccel *dbg,
                                               SYMS_DwUnitAccel *unit,
                                               SYMS_DwAttribKind kind, SYMS_SymbolID sid)
{
  SYMS_String8 result = {0};
  for(SYMS_SymbolID id = sid; id != 0 && result.size == 0;)
  {
    result = syms_dw_attrib_string_from_sid__unstable(data, dbg, unit, kind, id);
    if(result.size == 0)
    {
      SYMS_DwTagStub stub = syms_dw_cached_tag_stub_from_sid__parse_fallback(data, dbg, unit, id);
      if(stub.ref != 0 && stub.flags & SYMS_DwTagStubFlag_HasSpecification)
      {
        id = stub.ref;
      }
      else if(stub.abstract_origin != 0)
      {
        id = stub.abstract_origin;
      }
      else
      {
        break;
      }
    }
  }
  return result;
}

SYMS_API SYMS_SymbolKind
syms_dw_symbol_kind_from_sid(SYMS_String8 data, SYMS_DwDbgAccel *dbg, SYMS_DwUnitAccel *unit, SYMS_SymbolID sid)
{
  SYMS_SymbolKind result = SYMS_SymbolKind_Null;
  SYMS_DwTagStub stub = syms_dw_cached_tag_stub_from_sid__parse_fallback(data, dbg, unit, sid);
  result = syms_dw_symbol_kind_from_tag_stub(data, dbg, unit->resolve_params, &stub);
  return result;
}

SYMS_API SYMS_String8
syms_dw_symbol_name_from_sid(SYMS_Arena *arena, SYMS_String8 data, SYMS_DwDbgAccel *dbg, SYMS_DwUnitAccel *unit, SYMS_SymbolID sid)
{
  SYMS_String8 name = syms_dw_attrib_string_from_sid__unstable_chain(data, dbg, unit, SYMS_DwAttribKind_NAME, sid);
  SYMS_String8 result = syms_push_string_copy(arena, name);
  return result;
}

SYMS_API SYMS_TypeInfo
syms_dw_type_info_from_sid(SYMS_String8 data, SYMS_DwDbgAccel *dbg, SYMS_DwUnitAccel *unit, SYMS_SymbolID sid)
{
  SYMS_ArenaTemp scratch = syms_get_scratch(0, 0);
  SYMS_TypeInfo info = {SYMS_TypeKind_Null};
  
  SYMS_DwTagStub stub = syms_dw_cached_tag_stub_from_sid__parse_fallback(data, dbg, unit, sid);
  if(stub.sid != 0)
  {
    SYMS_SymbolID void_id = syms_dw_sid_from_info_offset(SYMS_DWARF_VOID_TYPE_ID);
    
    //- rjf: get attributes
    SYMS_DwAttribList attribs = syms_dw_attrib_list_from_stub(scratch.arena, data, dbg, unit->language, unit->version, unit->address_size, &stub);
    
    //- rjf: define info from attributes
    SYMS_SymbolID                          type = 0;
    SYMS_UnitID                        type_uid = 0;
    SYMS_DwAttribTypeEncoding          encoding = SYMS_DwAttribTypeEncoding_Null;
    SYMS_U64                               size = 1;
    SYMS_SizeInterpretation size_interpretation = SYMS_SizeInterpretation_Multiplier;
    SYMS_DwCallingConvention    call_convention = SYMS_DwCallingConvention_Normal;
    SYMS_FileID                        loc_file = 0;
    SYMS_U32                           loc_line = 0;
    SYMS_U32                         loc_column = 0;
    SYMS_USID                   containing_type = {0};
    
    //- rjf: loop through DWARF attributes and fill info
    for(SYMS_DwAttribNode *attrib_n = attribs.first;
        attrib_n != 0;
        attrib_n = attrib_n->next)
    {
      SYMS_DwAttrib *attrib = &attrib_n->attrib;
      SYMS_DwAttribValue value = syms_dw_attrib_value_from_form_value(data, dbg, unit->resolve_params, attrib->form_kind, attrib->value_class, attrib->form_value);
      switch(attrib->attrib_kind)
      {
        //- rjf: type references
        case SYMS_DwAttribKind_TYPE:
        {
          type = syms_dw_sid_from_info_offset(value.v[0]);
          // TODO(rjf): This doesn't handle cross-unit references @dw_cross_unit
          type_uid = unit->uid;
        }break;
        
        //- rjf: containing types (used for member-pointers)
        case SYMS_DwAttribKind_CONTAINING_TYPE:
        {
          // TODO(rjf): This doesn't handle cross-unit references @dw_cross_unit
          containing_type.sid = syms_dw_sid_from_info_offset(value.v[0]);
          containing_type.uid = unit->uid;
        }break;
        
        //- rjf: sizes
        case SYMS_DwAttribKind_BYTE_SIZE: {size_interpretation = SYMS_SizeInterpretation_ByteCount;   }goto read_size;
        case SYMS_DwAttribKind_ARR_COUNT: {size_interpretation = SYMS_SizeInterpretation_Multiplier;  }goto read_size;
        read_size:;
        {
          size = value.v[0];
        }break;
        case SYMS_DwAttribKind_BIT_SIZE: 
        {
          size_interpretation = SYMS_SizeInterpretation_ByteCount;
          size = (value.v[0] + 7)/8;
        }break;
        
        //- rjf: misc enums
        case SYMS_DwAttribKind_ENCODING:
        {
          encoding = (SYMS_DwAttribTypeEncoding)value.v[0];
        }break;
        case SYMS_DwAttribKind_CALLING_CONVENTION:
        {
          call_convention = (SYMS_DwCallingConvention)value.v[0];
        }break;
        
        //- rjf: loc
        case SYMS_DwAttribKind_DECL_FILE:   {loc_file = syms_dw_file_id_from_index(value.v[0]); }break;
        case SYMS_DwAttribKind_DECL_LINE:   {loc_line   = (SYMS_U32)value.v[0]; }break;
        case SYMS_DwAttribKind_DECL_COLUMN: {loc_column = (SYMS_U32)value.v[0]; }break;
        
        default: break;
      }
    }
    
    //- rjf: fill
    info.kind                 = syms_dw_type_kind_from_tag_encoding_size(stub.kind, encoding, size);
    info.mods                 = syms_dw_type_modifiers_from_tag_kind(stub.kind);
    info.direct_type.uid      = type_uid;
    info.direct_type.sid      = type;
    info.reported_size_interp = size_interpretation;
    info.reported_size        = size;
    info.src_coord.file_id    = loc_file;
    info.src_coord.line       = loc_line;
    info.src_coord.col        = loc_column;
    switch (info.kind){
      case SYMS_TypeKind_Proc:
      {
        info.call_convention    = syms_dw_base_call_convention_from_dw_calling_convention(call_convention);
      }break;
      case SYMS_TypeKind_MemberPtr:
      {
        info.containing_type    = containing_type;
      }break;
    }
    
    //- rjf: void type direct_type
    if(sid != void_id && type == 0 &&
       (info.mods != 0 ||
        stub.kind == SYMS_DwTagKind_POINTER_TYPE ||
        stub.kind == SYMS_DwTagKind_SUBROUTINE_TYPE))
    {
      info.direct_type.uid = unit->uid;
      info.direct_type.sid = void_id;
    }
    
    //- rjf: type kind fill overrides
    if(sid == void_id)
    {
      info.kind = SYMS_TypeKind_Void;
      info.reported_size_interp = SYMS_SizeInterpretation_ByteCount;
      info.reported_size = 0;
    }
    
    //- rjf: size fill overrides 
    {
      if(info.kind == SYMS_TypeKind_Ptr)
      {
        info.reported_size = unit->address_size;
        info.reported_size_interp = SYMS_SizeInterpretation_ByteCount;
      }
      else if(info.kind == SYMS_TypeKind_MemberPtr)
      {
        info.reported_size_interp = SYMS_SizeInterpretation_ByteCount;
        info.reported_size = 0;
        if(type != 0)
        {
          SYMS_DwTagStub type_stub = syms_dw_cached_tag_stub_from_sid__parse_fallback(data, dbg, unit, type);
          
          // NOTE(rjf): It looks like DWARF doesn't export size information for
          // member pointers, despite the fact that they are not the same size
          // as regular pointers. In experimentation, we've found that with
          // DWARF-producing builds by compilers, method-pointers look to be
          // 16 bytes (not 8, due to multiple inheritance), and regular member
          // pointers look like they're 8 bytes (despite their equivalent being
          // 4 bytes in PDB). These aren't necessarily what they are, but this
          // is what their layout in structs would suggest. So, until we find a
          // better way to do this, we're just going to assume that if we're
          // pointing at a member function, we're 2 * ptr sized, and if we're
          // not, then we're ptr sized.
          if(type_stub.kind == SYMS_DwTagKind_SUBROUTINE_TYPE)
          {
            info.reported_size = unit->address_size * 2;
          }
          else
          {
            info.reported_size = unit->address_size;
          }
        }
      }
    }
    
    //- rjf: for array types: override fill from children
    if(info.kind == SYMS_TypeKind_Array)
    {
      SYMS_DwTagStubList children = syms_dw_children_from_tag_stub(scratch.arena, data, dbg, unit, stub);
      for(SYMS_DwTagStubNode *child_n = children.first; child_n; child_n = child_n->next)
      {
        SYMS_DwTagStub *child = &child_n->stub;
        if(child->kind == SYMS_DwTagKind_SUBRANGE_TYPE)
        {
          SYMS_DwAttribList child_attribs = syms_dw_attrib_list_from_stub(scratch.arena, data, dbg, unit->language, unit->version, unit->address_size, child);
          
          for(SYMS_DwAttribNode *child_attrib_n = child_attribs.first;
              child_attrib_n != 0;
              child_attrib_n = child_attrib_n->next)
          {
            SYMS_DwAttrib *child_attrib = &child_attrib_n->attrib;
            if(child_attrib->attrib_kind == SYMS_DwAttribKind_ARR_COUNT)
            {
              SYMS_DwAttribValue value = syms_dw_attrib_value_from_form_value(data, dbg, unit->resolve_params,
                                                                              child_attrib->form_kind,
                                                                              child_attrib->value_class,
                                                                              child_attrib->form_value);
              info.reported_size = value.v[0];
              info.reported_size_interp = SYMS_SizeInterpretation_Multiplier;
              break;
            }
          }
          break;
        }
      }
    }
  }
  
  syms_release_scratch(scratch);
  return info;
}

SYMS_API SYMS_ConstInfo
syms_dw_const_info_from_sid(SYMS_String8 data, SYMS_DwDbgAccel *dbg, SYMS_DwUnitAccel *unit, SYMS_SymbolID id)
{
  SYMS_ArenaTemp scratch = syms_get_scratch(0, 0);
  SYMS_ConstInfo info;
  syms_memzero_struct(&info);
  
  SYMS_DwTagStub stub = syms_dw_cached_tag_stub_from_sid__parse_fallback(data, dbg, unit, id);
  if(stub.sid != 0)
  {
    //- rjf: get attributes
    SYMS_DwAttribList attribs = syms_dw_attrib_list_from_stub(scratch.arena, data, dbg, unit->language, unit->version, unit->address_size, &stub);
    
    //- rjf: define info from attributes
    SYMS_U64 const_value = 0;
    
    //- rjf: loop through DWARF attributes and fill info
    for(SYMS_DwAttribNode *attrib_n = attribs.first;
        attrib_n != 0;
        attrib_n = attrib_n->next)
    {
      SYMS_DwAttrib *attrib = &attrib_n->attrib;
      SYMS_DwAttribValue value = syms_dw_attrib_value_from_form_value(data, dbg, unit->resolve_params, attrib->form_kind, attrib->value_class, attrib->form_value);
      switch(attrib->attrib_kind)
      {
        //- rjf: const values
        case SYMS_DwAttribKind_CONST_VALUE:
        {
          switch(attrib->value_class)
          {
            case SYMS_DwAttribClass_CONST:
            {
              const_value = value.v[0];
            }break;
            default: /* TODO(rjf): */ break;
          }
        }break;
        
        default: break;
      }
    }
    
    //- rjf: fill
    info.kind = SYMS_TypeKind_UInt64;
    info.val = const_value;
  }
  
  syms_release_scratch(scratch);
  return info;
}

SYMS_API SYMS_USID
syms_dw_type_from_var_sid(SYMS_String8 data, SYMS_DwDbgAccel *dbg, SYMS_DwUnitAccel *unit, SYMS_SymbolID id)
{
  SYMS_ArenaTemp scratch = syms_get_scratch(0, 0);
  SYMS_USID result;
  syms_memzero_struct(&result);
  SYMS_DwTagStub stub = syms_dw_cached_tag_stub_from_sid__parse_fallback(data, dbg, unit, id);
  if(stub.sid != 0)
  {
    SYMS_DwAttribList attribs = syms_dw_attrib_list_from_stub(scratch.arena, data, dbg, unit->language, unit->version, unit->address_size, &stub);
    result.uid = unit->uid;
    result.sid = syms_dw_sid_from_info_offset(SYMS_DWARF_VOID_TYPE_ID);
    for(SYMS_DwAttribNode *attrib_n = attribs.first;
        attrib_n != 0;
        attrib_n = attrib_n->next)
    {
      SYMS_DwAttrib *attrib = &attrib_n->attrib;
      SYMS_DwAttribValue value = syms_dw_attrib_value_from_form_value(data, dbg, unit->resolve_params, attrib->form_kind, attrib->value_class, attrib->form_value);
      if(attrib->attrib_kind == SYMS_DwAttribKind_TYPE)
      {
        // TODO(rjf): This does not handle cross-unit references @dw_cross_unit
        result.sid = syms_dw_sid_from_info_offset(value.v[0]);
        result.uid = unit->uid;
        break;
      }
    }
  }
  syms_release_scratch(scratch);
  return result;
}

SYMS_API SYMS_U64
syms_dw_voff_from_var_sid(SYMS_String8 data, SYMS_DwDbgAccel *dbg, SYMS_DwUnitAccel *unit, SYMS_SymbolID sid)
{
  SYMS_U64 result = 0;
  SYMS_ArenaTemp scratch = syms_get_scratch(0, 0);
  SYMS_DwTagStub stub = syms_dw_cached_tag_stub_from_sid__parse_fallback(data, dbg, unit, sid);
  if(stub.sid != 0)
  {
    SYMS_DwAttribList attribs = syms_dw_attrib_list_from_stub(scratch.arena, data, dbg, unit->language, unit->version, unit->address_size, &stub);
    
    SYMS_DwAttrib *location_attrib = 0;
    for(SYMS_DwAttribNode *attrib_n = attribs.first;
        attrib_n != 0;
        attrib_n = attrib_n->next)
    {
      SYMS_DwAttrib *attrib = &attrib_n->attrib;
      if (attrib->attrib_kind == SYMS_DwAttribKind_LOCATION){
        location_attrib = attrib;
        break;
      }
    }
    
    if(location_attrib != 0)
    {
      SYMS_DwAttribValue value = syms_dw_attrib_value_from_form_value(data, dbg, unit->resolve_params,
                                                                      location_attrib->form_kind,
                                                                      location_attrib->value_class,
                                                                      location_attrib->form_value);
      
      switch (location_attrib->value_class){
        case SYMS_DwAttribClass_CONST:
        {
          result = value.v[0];
        }break;
        
        default:
        {
          SYMS_ArenaTemp temp2 = syms_arena_temp_begin(scratch.arena);
          void *debug_info_base = syms_dw_sec_base_from_dbg(data, dbg, SYMS_DwSectionKind_Info);
          void *exprloc_base = ((SYMS_U8*)debug_info_base + value.v[0]);
          SYMS_U64Range exprloc_range = syms_make_u64_range(0, value.v[1]);
          
          // TODO(allen): fill this in correctly
          SYMS_U64 text_base = 0;
          SYMS_DwSimpleLoc loc = syms_dw_expr__analyze_fast(exprloc_base, exprloc_range, text_base);
          if (loc.kind == SYMS_DwSimpleLocKind_Address){
            result = loc.addr;
          }
          
          syms_arena_temp_end(temp2);
        }break;
      }
    }
  }
  syms_release_scratch(scratch);
  return(result);
}

SYMS_API SYMS_SymbolIDList
syms_dw_children_from_sid_with_kinds(SYMS_Arena *arena, SYMS_String8 data, SYMS_DwDbgAccel *dbg, SYMS_DwUnitAccel *unit, SYMS_SymbolID sid, SYMS_DwTagKind *kinds, SYMS_U64 count)
{
  SYMS_ArenaTemp scratch = syms_get_scratch(&arena, 1);
  SYMS_SymbolIDList list;
  syms_memzero_struct(&list);
  SYMS_DwTagStub stub = syms_dw_cached_tag_stub_from_sid__parse_fallback(data, dbg, unit, sid);
  SYMS_DwTagStubList children = syms_dw_children_from_tag_stub(scratch.arena, data, dbg, unit, stub);
  for(SYMS_DwTagStubNode *n = children.first; n != 0; n = n->next)
  {
    SYMS_B32 matches = 0;
    for(SYMS_U64 kind_idx = 0; kind_idx < count; kind_idx += 1)
    {
      if(n->stub.kind == kinds[kind_idx])
      {
        matches = 1;
        break;
      }
    }
    if(matches)
    {
      SYMS_SymbolIDNode *node = syms_push_array_zero(arena, SYMS_SymbolIDNode, 1);
      node->id = n->stub.sid;
      SYMS_QueuePush(list.first, list.last, node);
      list.count += 1;
    }
  }
  syms_release_scratch(scratch);
  return list;
}

//- rjf: variable locations

SYMS_API SYMS_Location
syms_dw_location_from_sid(SYMS_Arena *arena, SYMS_String8 data, SYMS_DwDbgAccel *dbg, SYMS_DwUnitAccel *unit, SYMS_SymbolID sid, SYMS_DwAttribKind loc_attrib)
{
  SYMS_Location result = {0};
  SYMS_ArenaTemp scratch = syms_get_scratch(&arena, 1);
  
  SYMS_DwTagStub stub = syms_dw_cached_tag_stub_from_sid__parse_fallback(data, dbg, unit, sid);
  if(stub.sid != 0)
  {
    SYMS_DwAttribList attribs = syms_dw_attrib_list_from_stub(scratch.arena, data, dbg, unit->language, unit->version, unit->address_size, &stub);
    for(SYMS_DwAttribNode *n = attribs.first; n != 0; n = n->next)
    {
      SYMS_DwAttrib *attrib = &n->attrib;
      if(attrib->attrib_kind == loc_attrib &&
         attrib->form_kind == SYMS_DwFormKind_EXPRLOC)
      {
        SYMS_DwAttribValue location_value = syms_dw_attrib_value_from_form_value(data, dbg, unit->resolve_params,
                                                                                 attrib->form_kind,
                                                                                 attrib->value_class,
                                                                                 attrib->form_value);
        void *expr_base = syms_dw_sec_base_from_dbg(data, dbg, SYMS_DwSectionKind_Info);
        SYMS_U64Range expr_range = syms_make_u64_range(location_value.v[0], location_value.v[0] + location_value.v[1]);
        SYMS_String8 location = syms_dw_expr__transpile_to_eval(arena, dbg, expr_base, expr_range );
        break;
      }
    }
  }
  
  // TODO(rjf): once SYMS_Location has changed accordingly (returns a string
  // instead of an op-list, wire up `location` to `result`)
  
  syms_release_scratch(scratch);
  return result;
}

SYMS_API SYMS_LocRangeArray
syms_dw_location_ranges_from_sid(SYMS_Arena *arena, SYMS_String8 data, SYMS_DwDbgAccel *dbg,
                                 SYMS_DwUnitAccel *unit, SYMS_SymbolID sid, SYMS_DwAttribKind loc_attrib)
{
  SYMS_LocRangeArray result = {0};
  SYMS_ArenaTemp scratch = syms_get_scratch(&arena, 1);
  
  // rjf: get location range list from sid, if applicable
  SYMS_LocRangeList list = {0};
  SYMS_DwTagStub stub = syms_dw_cached_tag_stub_from_sid__parse_fallback(data, dbg, unit, sid);
  if(stub.sid != 0)
  {
    SYMS_DwAttribList attribs = syms_dw_attrib_list_from_stub(scratch.arena, data, dbg, unit->language, unit->version, unit->address_size, &stub);
    for(SYMS_DwAttribNode *n = attribs.first; n != 0; n = n->next)
    {
      SYMS_DwAttrib *attrib = &n->attrib;
      if(attrib->attrib_kind == loc_attrib &&
         (attrib->value_class == SYMS_DwAttribClass_LOCLISTPTR ||
          attrib->value_class == SYMS_DwAttribClass_LOCLIST))
      {
        SYMS_DwAttribValue location_value = syms_dw_attrib_value_from_form_value(data, dbg, unit->resolve_params,
                                                                                 attrib->form_kind,
                                                                                 attrib->value_class,
                                                                                 attrib->form_value);
        SYMS_U64 comp_unit_base_addr = unit->base_addr;
        switch(location_value.section)
        {
          // NOTE(rjf): .debug_loclists is only available in DWARF V5, so we can use that to
          // determine which parsing path to use.
          case SYMS_DwSectionKind_LocLists:
          {
            list = syms_dw_v5_location_ranges_from_loclist_offset(scratch.arena, data, dbg, location_value.section, unit->address_size, comp_unit_base_addr, location_value.v[0]);
          }break;
          
          // NOTE(rjf): .debug_loclists is only available in DWARF V4 and earlier.
          case SYMS_DwSectionKind_Loc:
          {
            list = syms_dw_v4_location_ranges_from_loc_offset(scratch.arena, data, dbg, unit->address_size, comp_unit_base_addr, location_value.v[0]);
          }break;
        }
        break;
      }
    }
  }
  
  // rjf: convert list => array
  if(list.count != 0)
  {
    result.count = list.count;
    result.loc_ranges = syms_push_array_zero(arena, SYMS_LocRange, result.count);
    SYMS_U64 idx = 0;
    for(SYMS_LocRangeNode *n = list.first; n != 0; n = n->next, idx += 1)
    {
      result.loc_ranges[idx] = n->loc_range;
    }
  }
  
  syms_release_scratch(scratch);
  return result;
}

SYMS_API SYMS_Location
syms_dw_location_from_id(SYMS_Arena *arena, SYMS_String8 data, SYMS_DwDbgAccel *dbg,
                         SYMS_DwUnitAccel *unit, SYMS_LocID loc_id)
{
  SYMS_Location result = {0};
  switch(unit->version)
  {
    case SYMS_DwVersion_V5:{syms_dw_v5_location_from_loclist_id(arena, data, dbg, SYMS_DwSectionKind_LocLists, loc_id);}break;
    case SYMS_DwVersion_V4:{syms_dw_v4_location_from_loc_id(arena, data, dbg, loc_id);}break;
  }
  return result;
}

//- rjf: location information helpers

SYMS_API SYMS_DwAttribKind
syms_dw_attrib_kind_from_proc_loc(SYMS_ProcLoc proc_loc)
{
  SYMS_DwAttribKind attrib = (SYMS_DwAttribKind)0;
  switch(proc_loc)
  {
    case SYMS_ProcLoc_FrameBase:    {attrib = SYMS_DwAttribKind_FRAME_BASE;}break;
    case SYMS_ProcLoc_ReturnAddress:{attrib = SYMS_DwAttribKind_RETURN_ADDR;}break;
  }
  return attrib;
}

SYMS_API SYMS_Location
syms_dw_location_from_var_sid(SYMS_Arena *arena, SYMS_String8 data, SYMS_DwDbgAccel *dbg, SYMS_DwUnitAccel *unit, SYMS_SymbolID sid)
{
  return syms_dw_location_from_sid(arena, data, dbg, unit, sid, SYMS_DwAttribKind_LOCATION);
}

SYMS_API SYMS_LocRangeArray
syms_dw_location_ranges_from_var_sid(SYMS_Arena *arena, SYMS_String8 data, SYMS_DwDbgAccel *dbg,
                                     SYMS_DwUnitAccel *unit, SYMS_SymbolID sid)
{
  return syms_dw_location_ranges_from_sid(arena, data, dbg, unit, sid, SYMS_DwAttribKind_LOCATION);
}

SYMS_API SYMS_Location
syms_dw_location_from_proc_sid(SYMS_Arena *arena, SYMS_String8 data, SYMS_DwDbgAccel *dbg, SYMS_DwUnitAccel *unit, SYMS_SymbolID sid, SYMS_ProcLoc proc_loc)
{
  SYMS_DwAttribKind attrib_kind = syms_dw_attrib_kind_from_proc_loc(proc_loc);
  SYMS_Location location = syms_dw_location_from_sid(arena, data, dbg, unit, sid, attrib_kind);
  return location;
}

SYMS_API SYMS_LocRangeArray
syms_dw_location_ranges_from_proc_sid(SYMS_Arena *arena, SYMS_String8 data, SYMS_DwDbgAccel *dbg,
                                      SYMS_DwUnitAccel *unit, SYMS_SymbolID sid, SYMS_ProcLoc proc_loc)
{
  SYMS_DwAttribKind attrib_kind = syms_dw_attrib_kind_from_proc_loc(proc_loc);
  SYMS_LocRangeArray ranges = syms_dw_location_ranges_from_sid(arena, data, dbg, unit, sid, attrib_kind);
  return ranges;
}

//- rjf: files

SYMS_API SYMS_U64
syms_dw_file_index_from_id(SYMS_FileID id)
{
  return id - 1;
}

SYMS_API SYMS_FileID
syms_dw_file_id_from_index(SYMS_U64 idx)
{
  return idx + 1;
}

SYMS_API SYMS_String8
syms_dw_file_name_from_id(SYMS_Arena *arena, SYMS_DwUnitSetAccel *unit_set, SYMS_UnitID uid, SYMS_FileID file_id)
{
  SYMS_String8 result = {0};
  
  //- rjf: get comp root
  SYMS_DwCompRoot *root = syms_dw_comp_root_from_uid(unit_set, uid);
  
  //- rjf: get .debug_line file struct for this file ID
  SYMS_U64 file_idx = syms_dw_file_index_from_id(file_id);
  SYMS_DwLineFile file = {0};
  SYMS_B32 good_file = syms_false;
  if(0 <= file_idx && file_idx < root->file_table.count)
  {
    file = root->file_table.v[file_idx];
    good_file = syms_true;
  }
  
  //- rjf: get directory string for this file
  SYMS_String8 dir = {0};
  if(good_file && 0 <= file.dir_idx && file.dir_idx < root->dir_table.count)
  {
    dir = root->dir_table.strings[file.dir_idx];
  }
  
  //- rjf: build final string
  result = syms_dw_path_from_dir_and_filename(arena, dir, file.file_name);
  
  return result;
}

//- rjf: procedures

SYMS_API SYMS_U64RangeArray
syms_dw_scope_vranges_from_sid(SYMS_Arena *arena, SYMS_String8 data, SYMS_DwDbgAccel *dbg,
                               SYMS_DwUnitAccel *unit, SYMS_SymbolID sid)
{
  SYMS_U64RangeList list;
  syms_memzero_struct(&list);
  SYMS_ArenaTemp scratch = syms_get_scratch(&arena, 1);
  
  //- rjf: id => stub
  SYMS_DwTagStub stub = syms_dw_cached_tag_stub_from_sid__parse_fallback(data, dbg, unit, sid);
  
  //- rjf: get range list
  if(stub.sid != 0)
  {
    SYMS_DwAttribList attribs = syms_dw_attrib_list_from_stub(scratch.arena, data, dbg, unit->language, unit->version, unit->address_size, &stub);
    
    SYMS_U64 low_pc = 0;
    SYMS_U64 high_pc = 0;
    SYMS_B32 high_pc_is_relative = 0;
    SYMS_DwAttribValue ranges_attrib_value = {SYMS_DwSectionKind_Null};
    
    for(SYMS_DwAttribNode *attrib_n = attribs.first;
        attrib_n != 0;
        attrib_n = attrib_n->next)
    {
      SYMS_DwAttrib *attrib = &attrib_n->attrib;
      SYMS_DwAttribValue value = syms_dw_attrib_value_from_form_value(data, dbg, unit->resolve_params, attrib->form_kind, attrib->value_class, attrib->form_value);
      
      if(attrib->attrib_kind == SYMS_DwAttribKind_RANGES)
      {
        ranges_attrib_value = value;
      }
      else if(attrib->attrib_kind == SYMS_DwAttribKind_LOW_PC)
      {
        low_pc = value.v[0];
      }
      else if(attrib->attrib_kind == SYMS_DwAttribKind_HIGH_PC)
      {
        high_pc = value.v[0];
        high_pc_is_relative = (attrib->value_class & SYMS_DwAttribClass_ADDRESS) == 0;
      }
    }
    
    if(high_pc_is_relative)
    {
      high_pc = low_pc + high_pc;
    }
    list = syms_dw_range_list_from_high_low_pc_and_ranges_attrib_value(scratch.arena, data, dbg,
                                                                       unit->address_size,
                                                                       unit->base_addr,
                                                                       unit->addrs_base,
                                                                       low_pc,
                                                                       high_pc,
                                                                       ranges_attrib_value);
  }
  
  SYMS_U64RangeArray result;
  syms_memzero_struct(&result);
  result = syms_u64_range_array_from_list(arena, &list);
  syms_release_scratch(scratch);
  return result;
}

SYMS_API SYMS_SigInfo
syms_dw_sig_info_from_sid(SYMS_Arena *arena, SYMS_String8 data, SYMS_DwDbgAccel *dbg,
                          SYMS_DwUnitAccel *unit, SYMS_SymbolID id)
{
  SYMS_ArenaTemp scratch = syms_get_scratch(&arena, 1);
  
  SYMS_SymbolIDList param_types;
  SYMS_SymbolID return_type;
  syms_memzero_struct(&param_types);
  syms_memzero_struct(&return_type);
  return_type = syms_dw_sid_from_info_offset(SYMS_DWARF_VOID_TYPE_ID);
  SYMS_SymbolID this_ptr_type = {0};
  
  //- rjf: get info from id
  SYMS_DwTagStub stub = syms_dw_cached_tag_stub_from_sid__parse_fallback(data, dbg, unit, id);
  if(stub.sid != 0)
  {
    SYMS_DwAttribList attribs = syms_dw_attrib_list_from_stub(scratch.arena, data, dbg, unit->language, unit->version, unit->address_size, &stub);
    
    //- rjf: grab return type
    for(SYMS_DwAttribNode *attrib_n = attribs.first;
        attrib_n != 0;
        attrib_n = attrib_n->next)
    {
      SYMS_DwAttrib *attrib = &attrib_n->attrib;
      SYMS_DwAttribValue value = syms_dw_attrib_value_from_form_value(data, dbg, unit->resolve_params, attrib->form_kind, attrib->value_class, attrib->form_value);
      if(attrib->attrib_kind == SYMS_DwAttribKind_TYPE)
      {
        return_type = syms_dw_sid_from_info_offset(value.v[0]);
        break;
      }
    }
    
    //- rjf: figure out if this is a member function
    SYMS_DwTagStub ref_stub = syms_dw_cached_tag_stub_from_sid__parse_fallback(data, dbg, unit, stub.ref);
    SYMS_DwTagStubFlags flags = stub.flags | ref_stub.flags;
    
    //- rjf: grab children
    SYMS_DwTagStubList children = syms_dw_children_from_tag_stub(scratch.arena, data, dbg, unit, stub);
    
    //- rjf: grab param types
    int child_idx = 0;
    for(SYMS_DwTagStubNode *child_n = children.first; child_n != 0; child_n = child_n->next, child_idx += 1)
    {
      SYMS_DwTagStub *child = &child_n->stub;
      if(child->kind == SYMS_DwTagKind_FORMAL_PARAMETER)
      {
        SYMS_DwAttribList child_attribs = syms_dw_attrib_list_from_stub(scratch.arena, data, dbg, unit->language, unit->version, unit->address_size, child);
        SYMS_SymbolID param_type = syms_dw_sid_from_info_offset(SYMS_DWARF_VOID_TYPE_ID);
        for(SYMS_DwAttribNode *child_attrib_n = child_attribs.first;
            child_attrib_n != 0;
            child_attrib_n = child_attrib_n->next)
        {
          SYMS_DwAttrib *attrib = &child_attrib_n->attrib;
          if(attrib->attrib_kind == SYMS_DwAttribKind_TYPE)
          {
            SYMS_DwAttribValue value = syms_dw_attrib_value_from_form_value(data, dbg, unit->resolve_params, attrib->form_kind, attrib->value_class, attrib->form_value);
            param_type = syms_dw_sid_from_info_offset(value.v[0]);
            break;
          }
        }
        
        if(child_idx == 0 && flags & SYMS_DwTagStubFlag_HasObjectPointerArg)
        {
          this_ptr_type = param_type;
        }
        else
        {
          syms_push_sid_to_list(scratch.arena, &param_types, param_type);
        }
      }
    }
  }
  
  //- rjf: build/fill result
  SYMS_SigInfo result;
  syms_memzero_struct(&result);
  result.uid = unit->uid;
  result.param_type_ids = syms_sid_array_from_list(arena, &param_types);
  result.return_type_id = return_type;
  result.this_type_id = this_ptr_type;
  
  syms_release_scratch(scratch);
  return result;
}

SYMS_API SYMS_UnitIDAndSig
syms_dw_proc_sig_handle_from_sid(SYMS_String8 data, SYMS_DwDbgAccel *dbg, SYMS_DwUnitAccel *unit,
                                 SYMS_SymbolID sid){
  SYMS_UnitIDAndSig result = {unit->uid, {sid}};
  return result;
}

SYMS_API SYMS_SigInfo
syms_dw_sig_info_from_handle(SYMS_Arena *arena, SYMS_String8 data, SYMS_DwDbgAccel *dbg, SYMS_DwUnitAccel *unit, SYMS_SigHandle handle){
  SYMS_SymbolID id = {handle.v};
  return syms_dw_sig_info_from_sid(arena, data, dbg, unit, id);
}

SYMS_API SYMS_SymbolIDArray
syms_dw_scope_children_from_sid(SYMS_Arena *arena, SYMS_String8 data, SYMS_DwDbgAccel *dbg, SYMS_DwUnitAccel *unit, SYMS_SymbolID id)
{
  SYMS_DwTagKind kinds[] =
  {
    SYMS_DwTagKind_FORMAL_PARAMETER,
    SYMS_DwTagKind_VARIABLE,
    SYMS_DwTagKind_LEXICAL_BLOCK,
    SYMS_DwTagKind_STRUCTURE_TYPE,
    SYMS_DwTagKind_UNION_TYPE,
    SYMS_DwTagKind_CLASS_TYPE,
    SYMS_DwTagKind_ENUMERATION_TYPE,
    SYMS_DwTagKind_SUBPROGRAM,
    SYMS_DwTagKind_INLINED_SUBROUTINE,
  };
  SYMS_ArenaTemp scratch = syms_get_scratch(&arena, 1);
  SYMS_SymbolIDList list = syms_dw_children_from_sid_with_kinds(scratch.arena, data, dbg, unit, id, kinds, SYMS_ARRAY_SIZE(kinds));
  SYMS_SymbolIDArray array = syms_sid_array_from_list(arena, &list);
  syms_release_scratch(scratch);
  return array;
}

//- rjf: line info

SYMS_API void
syms_dw_line_vm_reset(SYMS_DwLineVMState *state, SYMS_B32 default_is_stmt){
  state->address         = 0;
  state->op_index        = 0;
  state->file_index      = 1;
  state->line            = 1;
  state->column          = 0;
  state->is_stmt         = default_is_stmt;
  state->basic_block     = syms_false;
  state->prologue_end    = syms_false;
  state->epilogue_begin  = syms_false;
  state->isa             = 0;
  state->discriminator   = 0;
}

SYMS_API void
syms_dw_line_vm_advance(SYMS_DwLineVMState *state, SYMS_U64 advance,
                        SYMS_U64 min_inst_len, SYMS_U64 max_ops_for_inst){
  SYMS_U64 op_index = state->op_index + advance;
  state->address += min_inst_len*(op_index/max_ops_for_inst);
  state->op_index = op_index % max_ops_for_inst;
}

SYMS_API SYMS_DwLineSeqNode *
syms_dw_push_line_seq(SYMS_Arena* arena, SYMS_DwLineTableParseResult *parsed_tbl)
{
  SYMS_DwLineSeqNode *new_seq = syms_push_array_zero(arena, SYMS_DwLineSeqNode, 1);
  SYMS_QueuePush(parsed_tbl->first_seq, parsed_tbl->last_seq, new_seq);
  parsed_tbl->seq_count += 1;
  SYMS_Log("<new sequence>\n");
  return new_seq;
}

SYMS_API SYMS_DwLineNode *
syms_dw_push_line(SYMS_Arena *arena, SYMS_DwLineTableParseResult *tbl, SYMS_DwLineVMState *vm_state, SYMS_B32 start_of_sequence)
{
  SYMS_DwLineNode *n = 0;
  if(vm_state->busted_seq == 0)
  {
    SYMS_DwLineSeqNode *seq = tbl->last_seq;
    if(seq == 0 || start_of_sequence)
    {
      if(seq && seq->count == 1)
      {
        SYMS_Log("ERROR! do not emit sequences with only one line...\n");
      }
      seq = syms_dw_push_line_seq(arena, tbl);
    }
    
    n = syms_push_array_zero(arena, SYMS_DwLineNode, 1);
    SYMS_QueuePush(seq->first, seq->last, n);
    seq->count += 1;
    n->line.src_coord.file_id = syms_dw_file_id_from_index(vm_state->file_index);
    n->line.src_coord.line    = vm_state->line;
    n->line.src_coord.col     = vm_state->column;
    n->line.voff              = vm_state->address;
    SYMS_Log(" line: [%i] %i:%i => %llx\n",
             vm_state->file_index,vm_state->line, vm_state->column, vm_state->address);
  }
  return n;
}

SYMS_API SYMS_DwLineTableParseResult
syms_dw_parsed_line_table_from_comp_root(SYMS_Arena *arena, SYMS_String8 data, SYMS_DwDbgAccel *dbg, SYMS_DwCompRoot *root)
{
  SYMS_LogOpen(SYMS_LogFeature_LineTable, root->index + 1, dwarf_line_vm);
  
  SYMS_DwMode mode = syms_dw_mode_from_sec(dbg->section_map, SYMS_DwSectionKind_Line);
  void *base = syms_dw_sec_base_from_dbg(data, dbg, SYMS_DwSectionKind_Line);
  SYMS_U64Range line_info_range = syms_dw_sec_range_from_dbg(dbg, SYMS_DwSectionKind_Line);
  SYMS_U64 read_off_start = root->line_off - line_info_range.min;
  SYMS_U64 read_off = read_off_start;
  
  SYMS_DwLineVMHeader vm_header;
  read_off += syms_dw_read_line_vm_header(arena, base, line_info_range, read_off, data, mode, dbg, root, &vm_header);
  
  //- rjf: prep state for VM
  SYMS_DwLineVMState vm_state;
  syms_memzero_struct(&vm_state);
  syms_dw_line_vm_reset(&vm_state, vm_header.default_is_stmt);
  
  //- rjf: VM loop; build output list
  SYMS_DwLineTableParseResult result = {0};
  SYMS_B32 end_of_seq = 0;
  SYMS_B32 error = syms_false;
  for (;!error && read_off < vm_header.unit_opl;){
    //- rjf: parse opcode
    SYMS_U8 opcode = 0;
    read_off += syms_based_range_read_struct(base, line_info_range, read_off, &opcode);
    
    //- rjf: do opcode action
    switch (opcode){
      default:
      {
        //- rjf: special opcode case
        if(opcode >= vm_header.opcode_base)
        {
          SYMS_U32 adjusted_opcode = (SYMS_U32)(opcode - vm_header.opcode_base);
          SYMS_U32 op_advance = adjusted_opcode / vm_header.line_range;
          SYMS_S32 line_inc = (SYMS_S32)vm_header.line_base + ((SYMS_S32)adjusted_opcode) % (SYMS_S32)vm_header.line_range;
          // TODO(allen): can we just call dw_advance_line_vm_state_pc
          SYMS_U64 addr_inc = vm_header.min_inst_len * ((vm_state.op_index+op_advance) / vm_header.max_ops_for_inst);
          vm_state.address += addr_inc;
          vm_state.op_index = (vm_state.op_index + op_advance) % vm_header.max_ops_for_inst;
          vm_state.line = (SYMS_U32)((SYMS_S32)vm_state.line + line_inc);
          vm_state.basic_block = syms_false;
          vm_state.prologue_end = syms_false;
          vm_state.epilogue_begin = syms_false;
          vm_state.discriminator = 0;
          
          SYMS_Log("[special opcode] inc line by %i, inc addr by %i\n", line_inc, (int)addr_inc);
          syms_dw_push_line(arena, &result, &vm_state, end_of_seq);
          end_of_seq = 0;
          
#if 0
          // NOTE(rjf): DWARF has dummy lines at the end of groups of line ranges, where we'd like
          // to break line info into sequences.
          if(vm_state.line == 0)
          {
            end_of_seq = 1;
          }
#endif
        }
        //- NOTE(nick): Skipping unknown opcode. This is a valid case and
        // it works because compiler stores operand lengths that we can read
        // to skip unknown opcode
        else{
          if (opcode > 0 && opcode <= vm_header.num_opcode_lens){
            SYMS_U8 num_operands = vm_header.opcode_lens[opcode - 1];
            for (SYMS_U8 i = 0; i < num_operands; i += 1){
              SYMS_U64 operand = 0;
              read_off += syms_based_range_read_uleb128(base, line_info_range, read_off, &operand);
            }
          }
          else{
            error = syms_true;
            goto exit;
          }
        }
      }break;
      
      //- NOTE(nick): standard opcodes
      
      case SYMS_DwStdOpcode_COPY:
      {
        SYMS_Log("copy\n");
        syms_dw_push_line(arena, &result, &vm_state, end_of_seq);
        end_of_seq = 0;
        vm_state.discriminator   = 0;
        vm_state.basic_block     = syms_false;
        vm_state.prologue_end    = syms_false;
        vm_state.epilogue_begin  = syms_false;
      }break;
      
      case SYMS_DwStdOpcode_ADVANCE_PC:
      {
        SYMS_U64 advance = 0;
        read_off += syms_based_range_read_uleb128(base, line_info_range, read_off, &advance);
        syms_dw_line_vm_advance(&vm_state, advance, vm_header.min_inst_len, vm_header.max_ops_for_inst);
        SYMS_Log("advance pc by %i, now %" SYMS_PRIx64 "\n", (int)advance, vm_state.address);
      }break;
      
      case SYMS_DwStdOpcode_ADVANCE_LINE:
      {
        SYMS_S64 s = 0;
        read_off += syms_based_range_read_sleb128(base, line_info_range, read_off, &s);
        vm_state.line += s;
        SYMS_Log("advance line by %i\n", (int)s);
      }break;
      
      case SYMS_DwStdOpcode_SET_FILE:
      {
        SYMS_U64 file_index = 0;
        read_off += syms_based_range_read_uleb128(base, line_info_range, read_off, &file_index);
        vm_state.file_index = file_index;
        SYMS_Log("set file to %i\n", (int)file_index);
      }break;
      
      case SYMS_DwStdOpcode_SET_COLUMN:
      {
        SYMS_U64 column = 0;
        read_off += syms_based_range_read_uleb128(base, line_info_range, read_off, &column);
        vm_state.column = column;
        SYMS_Log("set column to %i\n", (int)column);
      }break;
      
      case SYMS_DwStdOpcode_NEGATE_STMT:
      {
        vm_state.is_stmt = !vm_state.is_stmt;
        SYMS_Log("negate is_stmt (now is %i)\n", (int)vm_state.is_stmt);
      }break;
      
      case SYMS_DwStdOpcode_SET_BASIC_BLOCK:
      {
        vm_state.basic_block = syms_true;
        SYMS_Log("set basic_block to true\n");
      }break;
      
      case SYMS_DwStdOpcode_CONST_ADD_PC:
      {
        SYMS_U64 advance = (0xffu - vm_header.opcode_base)/vm_header.line_range;
        syms_dw_line_vm_advance(&vm_state, advance, vm_header.min_inst_len, vm_header.max_ops_for_inst);
        SYMS_Log("const add pc with %i\n", (int)advance);
      }break;
      
      case SYMS_DwStdOpcode_FIXED_ADVANCE_PC:
      {
        SYMS_U16 operand = 0;
        read_off += syms_based_range_read_struct(base, line_info_range, read_off, &operand);
        vm_state.address += operand;
        vm_state.op_index = 0;
        SYMS_Log("fixed add pc with %i\n", (int)operand);
      }break;
      
      case SYMS_DwStdOpcode_SET_PROLOGUE_END:
      {
        vm_state.prologue_end = syms_true;
        SYMS_Log("set prologue end\n");
      }break;
      
      case SYMS_DwStdOpcode_SET_EPILOGUE_BEGIN:
      {
        vm_state.epilogue_begin = syms_true;
        SYMS_Log("set epilogue begin\n");
      }break;
      
      case SYMS_DwStdOpcode_SET_ISA:
      {
        SYMS_U64 v = 0;
        read_off += syms_based_range_read_uleb128(base, line_info_range, read_off, &v);
        vm_state.isa = v;
        SYMS_Log("set isa to %i\n", (int)v);
      }break;
      
      //- NOTE(nick): extended opcodes
      case SYMS_DwStdOpcode_EXTENDED_OPCODE:
      {
        SYMS_U64 length = 0;
        read_off += syms_based_range_read_uleb128(base, line_info_range, read_off, &length);
        SYMS_U64 start_off = read_off;
        SYMS_U8 extended_opcode = 0;
        read_off += syms_based_range_read_struct(base, line_info_range, read_off, &extended_opcode);
        
        switch (extended_opcode){
          case SYMS_DwExtOpcode_END_SEQUENCE:
          {
            vm_state.end_sequence = syms_true;
            syms_dw_push_line(arena, &result, &vm_state, 0);
            SYMS_Log("<end sequence>\n");
            syms_dw_line_vm_reset(&vm_state, vm_header.default_is_stmt);
            end_of_seq = 1;
          }break;
          
          case SYMS_DwExtOpcode_SET_ADDRESS:
          {
            SYMS_U64 address = 0;
            read_off += syms_based_range_read(base, line_info_range, read_off, root->address_size, &address);
            vm_state.address = address;
            vm_state.op_index = 0;
            vm_state.busted_seq = !(dbg->acceptable_vrange.min <= address && address < dbg->acceptable_vrange.max);
            SYMS_Log("set address to %" SYMS_PRIx64 "\n", address);
          }break;
          
          case SYMS_DwExtOpcode_DEFINE_FILE:
          {
            SYMS_String8 file_name = syms_based_range_read_string(base, line_info_range, read_off);
            SYMS_U64 dir_index = 0;
            SYMS_U64 modify_time = 0;
            SYMS_U64 file_size = 0;
            read_off += file_name.size + 1;
            read_off += syms_based_range_read_uleb128(base, line_info_range, read_off, &dir_index);
            read_off += syms_based_range_read_uleb128(base, line_info_range, read_off, &modify_time);
            read_off += syms_based_range_read_uleb128(base, line_info_range, read_off, &file_size);
            
            // TODO(rjf): Not fully implemented. By the DWARF V4 spec, the above is
            // all that needs to be parsed, but the rest of the work that needs to
            // happen here---allowing this file to be used by further opcodes---is
            // not implemented.
            //
            // See the DWARF V4 spec (June 10, 2010), page 122.
            error = syms_true;
            SYMS_Log("UNHANDLED DEFINE FILE!!!\n");
          }break;
          
          case SYMS_DwExtOpcode_SET_DISCRIMINATOR:
          {
            SYMS_U64 v = 0;
            read_off += syms_based_range_read_uleb128(base, line_info_range, read_off, &v);
            vm_state.discriminator = v;
            SYMS_Log("set discriminator to %" SYMS_PRIx64 "\n", v);
          }break;
          
          default: break;
        }
        
        SYMS_U64 num_skip = read_off - (start_off + length);
        read_off += num_skip;
        if (syms_based_range_ptr(base, line_info_range, read_off) == 0 || 
            start_off + length > read_off){
          error = syms_true;
        }
        
      }break;
    }
  }
  exit:;
  
  SYMS_LogClose(dwarf_line_vm);
  
  return(result);
}

SYMS_API SYMS_LineParseOut
syms_dw_line_parse_from_uid(SYMS_Arena *arena, SYMS_String8 data, SYMS_DwDbgAccel *dbg, SYMS_DwUnitSetAccel *unit_set, SYMS_UnitID uid)
{
  SYMS_ArenaTemp scratch = syms_get_scratch(&arena, 1);
  SYMS_DwCompRoot *root = syms_dw_comp_root_from_uid(unit_set, uid);
  SYMS_DwLineTableParseResult parsed_tbl = syms_dw_parsed_line_table_from_comp_root(scratch.arena, data, dbg, root);
  SYMS_LineParseOut result = {0};
  {
    //- rjf: calculate # of lines we have
    SYMS_U64 line_count = 0;
    SYMS_U64 seq_count = 0;
    for(SYMS_DwLineSeqNode *seq = parsed_tbl.first_seq; seq != 0; seq = seq->next)
    {
      if(seq->count > 1)
      {
        line_count += seq->count;
        seq_count += 1;
      }
    }
    
    //- rjf: set up arrays
    SYMS_U64 *sequence_index_array = syms_push_array_zero(arena, SYMS_U64, parsed_tbl.seq_count + 1);
    SYMS_Line *line_array = syms_push_array_zero(arena, SYMS_Line, line_count);
    
    //- rjf: fill arrays
    SYMS_U64 line_idx = 0;
    SYMS_U64 seq_idx = 0;
    for (SYMS_DwLineSeqNode *seq = parsed_tbl.first_seq;
         seq != 0;
         seq = seq->next, seq_idx += 1){
      sequence_index_array[seq_idx] = line_idx;
      for (SYMS_DwLineNode *line = seq->first;
           line != 0;
           line = line->next, line_idx += 1){
        line_array[line_idx] = line->line;
      }
    }
    sequence_index_array[seq_idx] = line_idx;
    
    //- fill result
    result.line_table.sequence_count = parsed_tbl.seq_count;
    result.line_table.sequence_index_array = sequence_index_array;
    result.line_table.line_count = line_count;
    result.line_table.line_array = line_array;
  }
  syms_release_scratch(scratch);
  return result;
}

SYMS_API SYMS_U64
syms_dw_read_line_file(void *line_base, SYMS_U64Range line_rng, SYMS_U64 line_off, SYMS_DwMode mode, SYMS_DwDbgAccel *dbg, SYMS_String8 data, SYMS_DwCompRoot *unit, SYMS_U8 address_size, SYMS_U64 format_count, SYMS_U64Range *formats, SYMS_DwLineFile *line_file_out)
{
  syms_memzero_struct(line_file_out);
  
  SYMS_DwAttribValueResolveParams resolve_params = syms_dw_attrib_value_resolve_params_from_comp_root(unit);
  SYMS_U64 line_off_start = line_off;
  for (SYMS_U64 format_idx = 0; format_idx < format_count; ++format_idx) 
  {
    SYMS_DwLNCT lnct = (SYMS_DwLNCT)formats[format_idx].min;
    SYMS_DwFormKind form_kind = (SYMS_DwFormKind)formats[format_idx].max;
    SYMS_DwAttribValue form_value;
    syms_memzero_struct(&form_value);
    line_off += syms_dw_based_range_read_attrib_form_value(line_base, line_rng, line_off, mode, address_size, form_kind, 0, &form_value);
    switch (lnct) 
    {
      case SYMS_DwLNCT_PATH: 
      {
        SYMS_ASSERT_PARANOID(form_kind == SYMS_DwFormKind_STRING || form_kind == SYMS_DwFormKind_LINE_STRP ||
                             form_kind == SYMS_DwFormKind_STRP || form_kind == SYMS_DwFormKind_STRP_SUP ||
                             form_kind == SYMS_DwFormKind_STRX || form_kind == SYMS_DwFormKind_STRX1 ||
                             form_kind == SYMS_DwFormKind_STRX2 || form_kind == SYMS_DwFormKind_STRX3 ||
                             form_kind == SYMS_DwFormKind_STRX4);
        SYMS_DwAttribValue attrib_value = syms_dw_attrib_value_from_form_value(data, dbg, resolve_params, form_kind, SYMS_DwAttribClass_STRING, form_value);
        line_file_out->file_name = syms_dw_string_from_attrib_value(data, dbg, attrib_value);
      } break;
      
      case SYMS_DwLNCT_DIRECTORY_INDEX: 
      {
        SYMS_ASSERT_PARANOID(form_kind == SYMS_DwFormKind_DATA1 || form_kind == SYMS_DwFormKind_DATA2 ||
                             form_kind == SYMS_DwFormKind_UDATA);
        SYMS_DwAttribValue attrib_value = syms_dw_attrib_value_from_form_value(data, dbg, resolve_params, form_kind, SYMS_DwAttribClass_BLOCK, form_value);
        line_file_out->dir_idx = attrib_value.v[0];
      } break;
      
      case SYMS_DwLNCT_TIMESTAMP: 
      {
        SYMS_ASSERT_PARANOID(form_kind == SYMS_DwFormKind_UDATA || form_kind == SYMS_DwFormKind_DATA4 ||
                             form_kind == SYMS_DwFormKind_DATA8 || form_kind == SYMS_DwFormKind_BLOCK);
        SYMS_DwAttribValue attrib_value = syms_dw_attrib_value_from_form_value( data, dbg, resolve_params, form_kind, SYMS_DwAttribClass_CONST, form_value);
        line_file_out->modify_time = attrib_value.v[0];
      } break;
      
      case SYMS_DwLNCT_SIZE: 
      {
        SYMS_ASSERT_PARANOID(form_kind == SYMS_DwFormKind_UDATA || form_kind == SYMS_DwFormKind_DATA1 ||
                             form_kind == SYMS_DwFormKind_DATA2 || form_kind == SYMS_DwFormKind_DATA4 ||
                             form_kind == SYMS_DwFormKind_DATA8);
        SYMS_DwAttribValue attrib_value = syms_dw_attrib_value_from_form_value(data, dbg, resolve_params, form_kind, SYMS_DwAttribClass_BLOCK, form_value);
        line_file_out->file_size = attrib_value.v[0];
      } break;
      
      case SYMS_DwLNCT_MD5: 
      {
        SYMS_ASSERT_PARANOID(form_kind == SYMS_DwFormKind_DATA16);
        SYMS_DwAttribValue attrib_value = syms_dw_attrib_value_from_form_value(data, dbg, resolve_params, form_kind, SYMS_DwAttribClass_BLOCK, form_value);
        line_file_out->md5_digest[0] = attrib_value.v[0];
        line_file_out->md5_digest[1] = attrib_value.v[1];
      } break;
      
      default: 
      {
        SYMS_ASSERT_PARANOID(SYMS_DwLNCT_USER_LO < lnct && lnct < SYMS_DwLNCT_USER_HI);
      } break;
    }
  }
  SYMS_U64 result = line_off - line_off_start;
  return result;
}

SYMS_API SYMS_U64
syms_dw_read_line_vm_header(SYMS_Arena *arena, void *line_base, SYMS_U64Range line_rng, SYMS_U64 line_off, SYMS_String8 data, SYMS_DwMode mode, SYMS_DwDbgAccel *dbg, SYMS_DwCompRoot *unit, SYMS_DwLineVMHeader *header_out)
{
  SYMS_U64 line_off_start = line_off;
  
  SYMS_ArenaTemp scratch = syms_get_scratch(&arena, 1);
  
  //- rjf: parse unit length
  header_out->unit_length = 0;
  {
    line_off += syms_dw_based_range_read_length(line_base, line_rng, line_off, &header_out->unit_length);
  }
  
  header_out->unit_opl = line_off + header_out->unit_length;
  
  //- rjf: parse version and header length
  line_off += syms_based_range_read_struct(line_base, line_rng, line_off, &header_out->version);
  
  if (header_out->version == SYMS_DwVersion_V5) {
    line_off += syms_based_range_read_struct(line_base, line_rng, line_off, &header_out->address_size);
    line_off += syms_based_range_read_struct(line_base, line_rng, line_off, &header_out->segment_selector_size);
  }
  
  {
    SYMS_U64 off_size = syms_dw_offset_size_from_mode(mode);
    line_off += syms_based_range_read(line_base, line_rng, line_off, off_size, &header_out->header_length);
  }
  
  //- rjf: calculate program offset
  header_out->program_off = line_off + header_out->header_length;
  
  //- rjf: parse minimum instruction length
  {
    line_off += syms_based_range_read_struct(line_base, line_rng, line_off, &header_out->min_inst_len);
  }
  
  //- rjf: parse max ops for instruction
  switch (header_out->version) 
  {
    case SYMS_DwVersion_V5:
    case SYMS_DwVersion_V4: 
    {
      line_off += syms_based_range_read_struct(line_base, line_rng, line_off, &header_out->max_ops_for_inst);
      SYMS_ASSERT_PARANOID(header_out->max_ops_for_inst > 0);
    } break;
    case SYMS_DwVersion_V3:
    case SYMS_DwVersion_V2:
    case SYMS_DwVersion_V1: 
    {
      header_out->max_ops_for_inst = 1;
    } break;
    default: break;
  }
  
  //- rjf: parse rest of program info
  syms_based_range_read_struct(line_base, line_rng, line_off + 0 * sizeof(SYMS_U8), &header_out->default_is_stmt);
  syms_based_range_read_struct(line_base, line_rng, line_off + 1 * sizeof(SYMS_U8), &header_out->line_base);
  syms_based_range_read_struct(line_base, line_rng, line_off + 2 * sizeof(SYMS_U8), &header_out->line_range);
  syms_based_range_read_struct(line_base, line_rng, line_off + 3 * sizeof(SYMS_U8), &header_out->opcode_base);
  line_off += 4 * sizeof(SYMS_U8);
  SYMS_ASSERT_PARANOID(header_out->opcode_base != 0 && header_out->opcode_base > 0);
  
  //- rjf: calculate opcode length array
  header_out->num_opcode_lens = header_out->opcode_base - 1u;
  header_out->opcode_lens = (SYMS_U8 *)syms_based_range_ptr(line_base, line_rng, line_off);
  line_off += header_out->num_opcode_lens * sizeof(SYMS_U8);
  
  if (header_out->version == SYMS_DwVersion_V5) {
    //- parse directory names
    SYMS_U8 directory_entry_format_count = 0;
    line_off += syms_based_range_read_struct(line_base, line_rng, line_off, &directory_entry_format_count);
    SYMS_ASSERT_PARANOID(directory_entry_format_count == 1);
    SYMS_U64Range *directory_entry_formats =
      syms_push_array(scratch.arena, SYMS_U64Range, directory_entry_format_count);
    for (SYMS_U8 format_idx = 0; format_idx < directory_entry_format_count; ++format_idx) 
    {
      SYMS_U64 content_type_code = 0, form_code = 0;
      line_off += syms_based_range_read_uleb128(line_base, line_rng, line_off, &content_type_code);
      line_off += syms_based_range_read_uleb128(line_base, line_rng, line_off, &form_code);
      directory_entry_formats[format_idx] = syms_make_u64_range(content_type_code, form_code);
    }
    SYMS_U64 directories_count = 0;
    line_off += syms_based_range_read_uleb128(line_base, line_rng, line_off, &directories_count);
    header_out->dir_table.count = directories_count;
    header_out->dir_table.strings = syms_push_array(arena, SYMS_String8, header_out->dir_table.count);
    for (SYMS_U64 dir_idx = 0; dir_idx < directories_count; ++dir_idx) 
    {
      SYMS_DwLineFile line_file;
      line_off += syms_dw_read_line_file(line_base,
                                         line_rng,
                                         line_off,
                                         mode,
                                         dbg,
                                         data,
                                         unit,
                                         header_out->address_size,
                                         directory_entry_format_count,
                                         directory_entry_formats,
                                         &line_file);
      // imperically directory format defines just names, but format is structured so it can export md5, time stamp for directories
      
      SYMS_String8 file_name = syms_push_string_copy(arena, line_file.file_name);
      header_out->dir_table.strings[dir_idx] = file_name;
    }
    //- parse file table
    SYMS_U8 file_name_entry_format_count = 0;
    line_off += syms_based_range_read_struct(line_base, line_rng, line_off, &file_name_entry_format_count);
    SYMS_U64Range *file_name_entry_formats = syms_push_array(scratch.arena, SYMS_U64Range, file_name_entry_format_count);
    for (SYMS_U8 format_idx = 0; format_idx < file_name_entry_format_count; ++format_idx) 
    {
      SYMS_U64 content_type_code = 0, form_code = 0;
      line_off += syms_based_range_read_uleb128(line_base, line_rng, line_off, &content_type_code);
      line_off += syms_based_range_read_uleb128(line_base, line_rng, line_off, &form_code);
      file_name_entry_formats[format_idx] = syms_make_u64_range(content_type_code, form_code);
    }
    SYMS_U64 file_names_count = 0;
    line_off += syms_based_range_read_uleb128(line_base, line_rng, line_off, &file_names_count);
    header_out->file_table.count = file_names_count;
    header_out->file_table.v = syms_push_array(arena, SYMS_DwLineFile, header_out->file_table.count);
    //SYMS_ASSERT_IMPLIES(file_name_entry_format_count == 0, file_names_count == 0);
    for (SYMS_U64 file_idx = 0; file_idx < file_names_count; ++file_idx) 
    {
      line_off += syms_dw_read_line_file(line_base,
                                         line_rng,
                                         line_off,
                                         mode,
                                         dbg,
                                         data,
                                         unit,
                                         header_out->address_size,
                                         file_name_entry_format_count,
                                         file_name_entry_formats,
                                         &header_out->file_table.v[file_idx]);
    }
  }
  else 
  {
    SYMS_String8List dir_list;
    syms_memzero_struct(&dir_list);
    
    syms_string_list_push(scratch.arena, &dir_list, unit->compile_dir);
    for (;;) 
    {
      SYMS_String8 dir = syms_based_range_read_string(line_base, line_rng, line_off);
      line_off += dir.size + 1;
      if (dir.size == 0) 
      {
        break;
      }
      syms_string_list_push(scratch.arena, &dir_list, dir);
    }
    
    SYMS_DwLineVMFileList file_list;
    syms_memzero_struct(&file_list);
    
    //- rjf: push 0-index file (compile file)
    {
      SYMS_DwLineVMFileNode *node = syms_push_array(scratch.arena, SYMS_DwLineVMFileNode, 1);
      syms_memzero_struct(node);
      node->file.file_name = unit->name;
      SYMS_QueuePush(file_list.first, file_list.last, node);
      file_list.node_count += 1;
    }
    
    for (;;) 
    {
      SYMS_String8 file_name = syms_based_range_read_string(line_base, line_rng, line_off);
      SYMS_U64 dir_index = 0;
      SYMS_U64 modify_time = 0;
      SYMS_U64 file_size = 0;
      line_off += file_name.size + 1;
      if (file_name.size == 0) 
      {
        break;
      }
      line_off += syms_based_range_read_uleb128(line_base, line_rng, line_off, &dir_index);
      line_off += syms_based_range_read_uleb128(line_base, line_rng, line_off, &modify_time);
      line_off += syms_based_range_read_uleb128(line_base, line_rng, line_off, &file_size);
      
      SYMS_DwLineVMFileNode *node = syms_push_array(scratch.arena, SYMS_DwLineVMFileNode, 1);
      node->file.file_name = file_name;
      node->file.dir_idx = dir_index;
      node->file.modify_time = modify_time;
      node->file.file_size = file_size;
      SYMS_QueuePush(file_list.first, file_list.last, node);
      file_list.node_count += 1;
    }
    
    //- rjf: build dir table
    {
      header_out->dir_table.count = dir_list.node_count;
      header_out->dir_table.strings = syms_push_array(arena, SYMS_String8, header_out->dir_table.count);
      SYMS_String8Node *n = dir_list.first;
      for(SYMS_U64 idx = 0; n != 0 && idx < header_out->dir_table.count; idx += 1, n = n->next) 
      {
        SYMS_String8 string = syms_push_string_copy(arena, n->string);
        header_out->dir_table.strings[idx] = string;
      }
    }
    
    //- rjf: build file table
    {
      header_out->file_table.count = file_list.node_count;
      header_out->file_table.v = syms_push_array(arena, SYMS_DwLineFile, header_out->file_table.count);
      SYMS_U64 file_idx = 0;
      SYMS_DwLineVMFileNode *file_node = file_list.first;
      for(; file_node != 0; file_idx += 1, file_node = file_node->next) 
      {
        SYMS_String8 file_name = syms_push_string_copy(arena, file_node->file.file_name);
        header_out->file_table.v[file_idx].file_name = file_name;
        header_out->file_table.v[file_idx].dir_idx = file_node->file.dir_idx;
        header_out->file_table.v[file_idx].modify_time = file_node->file.modify_time;
        header_out->file_table.v[file_idx].file_size = file_node->file.file_size;
      }
    }
  }
  syms_release_scratch(scratch);
  
  SYMS_U64 result = line_off - line_off_start;
  return result;
}

//- rjf: name maps

SYMS_API SYMS_DwMapAccel*
syms_dw_type_map_from_dbg(SYMS_Arena *arena, SYMS_String8 data, SYMS_DwDbgAccel *dbg)
{
  SYMS_DwMapAccel *result = (SYMS_DwMapAccel *)&syms_format_nil;
  if(syms_dw_sec_is_present(dbg, SYMS_DwSectionKind_PubTypes))
  {
    result = syms_push_array(arena, SYMS_DwMapAccel, 1);
    syms_memzero_struct(result);
    result->format = SYMS_FileFormat_DWARF;
    result->tbl = syms_dw_v4_pub_strings_table_from_section_kind(arena, data, dbg, SYMS_DwSectionKind_PubTypes);
  }
  return result;
}

SYMS_API SYMS_DwMapAccel*
syms_dw_image_symbol_map_from_dbg(SYMS_Arena *arena, SYMS_String8 data, SYMS_DwDbgAccel *dbg)
{
  SYMS_DwMapAccel *result = (SYMS_DwMapAccel *)&syms_format_nil;
  if(syms_dw_sec_is_present(dbg, SYMS_DwSectionKind_PubNames))
  {
    result = syms_push_array(arena, SYMS_DwMapAccel, 1);
    syms_memzero_struct(result);
    result->format = SYMS_FileFormat_DWARF;
    result->tbl = syms_dw_v4_pub_strings_table_from_section_kind(arena, data, dbg, SYMS_DwSectionKind_PubNames);
  }
  return result;
}

SYMS_API SYMS_USIDList
syms_dw_usid_list_from_string(SYMS_Arena *arena, SYMS_DwMapAccel *map, SYMS_String8 string)
{
  return syms_dw_v4_usid_list_from_pub_table_string(arena, map->tbl, string);
}

#endif // SYMS_DWARF_PARSER_C
