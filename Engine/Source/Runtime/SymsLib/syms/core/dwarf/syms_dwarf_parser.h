// Copyright Epic Games, Inc. All Rights Reserved.
/* date = May 11th 2021 4:56 pm */

#ifndef SYMS_DWARF_PARSER_H
#define SYMS_DWARF_PARSER_H

// NOTE(rjf): Some rules about the spaces of offsets and ranges:
//
// - Every stored/passed offset is relative to the base of its section.
// - Every stored/passed range has endpoints relative to the base of their section.
// - Upon calling a syms_based_range_* function, these offsets need to be
//   converted into range-relative.

////////////////////////////////
//~ rjf: Constants

#define SYMS_DWARF_VOID_TYPE_ID 0xffffffffffffffffull

////////////////////////////////
//~ rjf: Helper Types

typedef struct SYMS_DwUnitRangePoint SYMS_DwUnitRangePoint;
struct SYMS_DwUnitRangePoint
{
  SYMS_U64 p;
  SYMS_UnitID uid;
  SYMS_B32 is_min;
};

typedef struct SYMS_DwSymbolIDChunk SYMS_DwSymbolIDChunk;
struct SYMS_DwSymbolIDChunk
{
  SYMS_DwSymbolIDChunk *next;
  SYMS_U64 count;
  SYMS_SymbolID ids[1022];
};

typedef struct SYMS_DwSymbolIDChunkList SYMS_DwSymbolIDChunkList;
struct SYMS_DwSymbolIDChunkList
{
  SYMS_U64 chunk_count;
  SYMS_U64 total_id_count;
  SYMS_DwSymbolIDChunk *first;
  SYMS_DwSymbolIDChunk *last;
};

////////////////////////////////
//~ rjf: Files + External Debug References

typedef struct SYMS_DwExtDebugRef SYMS_DwExtDebugRef;
struct SYMS_DwExtDebugRef
{
  // NOTE(rjf): .dwo => an external DWARF V5 .dwo file
  SYMS_String8 dwo_path;
  SYMS_U64 dwo_id;
};

////////////////////////////////
//~ rjf: Abbrev Table

typedef struct SYMS_DwAbbrevTableEntry SYMS_DwAbbrevTableEntry;
struct SYMS_DwAbbrevTableEntry
{
  SYMS_U64 id;
  SYMS_U64 off;
};

typedef struct SYMS_DwAbbrevTable SYMS_DwAbbrevTable;
struct SYMS_DwAbbrevTable
{
  SYMS_U64 count;
  SYMS_DwAbbrevTableEntry *entries;
};

////////////////////////////////
//~ rjf: Dbg Accel

typedef struct SYMS_DwSection SYMS_DwSection;
struct SYMS_DwSection
{
  SYMS_U64Range range;
};

typedef struct SYMS_DwUnitRangeInfo SYMS_DwUnitRangeInfo;
struct SYMS_DwUnitRangeInfo
{
  SYMS_UnitID uid;
  SYMS_U64Range frange;            // rjf: .debug_info range in file
  SYMS_U64RangeArray addr_ranges;  // rjf: .debug_aranges
};

typedef struct SYMS_DwDbgAccel SYMS_DwDbgAccel;
struct SYMS_DwDbgAccel
{
  SYMS_FileFormat format;
  SYMS_Arch arch;
  SYMS_U64 vbase;
  SYMS_SecInfoArray sections;
  SYMS_U64 text_section_idx;
  SYMS_U64Range acceptable_vrange;
  SYMS_DwSection *section_map; // NOTE(rjf): treat as [SYMS_DwSectionKind_COUNT] array
  SYMS_U64 unit_count;
  SYMS_DwUnitRangeInfo *unit_range_info;
  SYMS_B32 is_dwo;
};

////////////////////////////////
//~ rjf: Basic Line Info

typedef struct SYMS_DwLineFile SYMS_DwLineFile;
struct SYMS_DwLineFile
{
  SYMS_String8 file_name;
  SYMS_U64 dir_idx;
  SYMS_U64 modify_time;
  SYMS_U64 md5_digest[2];
  SYMS_U64 file_size;
};

typedef struct SYMS_DwLineVMFileNode SYMS_DwLineVMFileNode;
struct SYMS_DwLineVMFileNode
{
  SYMS_DwLineVMFileNode *next;
  SYMS_DwLineFile file;
};

typedef struct SYMS_DwLineVMFileList SYMS_DwLineVMFileList;
struct SYMS_DwLineVMFileList
{
  SYMS_U64 node_count;
  SYMS_DwLineVMFileNode *first;
  SYMS_DwLineVMFileNode *last;
};

typedef struct SYMS_DwLineVMFileArray SYMS_DwLineVMFileArray;
struct SYMS_DwLineVMFileArray
{
  SYMS_U64 count;
  SYMS_DwLineFile *v;
};

////////////////////////////////
//~ rjf: Abbrevs

typedef enum SYMS_DwAbbrevKind
{
  SYMS_DwAbbrevKind_Null,
  SYMS_DwAbbrevKind_Tag,
  SYMS_DwAbbrevKind_Attrib,
  SYMS_DwAbbrevKind_AttribSequenceEnd,
  SYMS_DwAbbrevKind_DIEBegin,
  SYMS_DwAbbrevKind_DIEEnd,
}
SYMS_DwAbbrevKind;

typedef SYMS_U32 SYMS_DwAbbrevFlags;
enum{
  SYMS_DwAbbrevFlag_HasImplicitConst = (1<<0),
  SYMS_DwAbbrevFlag_HasChildren      = (1<<1),
};

typedef struct SYMS_DwAbbrev SYMS_DwAbbrev;
struct SYMS_DwAbbrev
{
  SYMS_DwAbbrevKind kind;
  SYMS_U64Range abbrev_range;
  SYMS_U64 sub_kind;
  SYMS_U64 id;
  SYMS_U64 const_value;
  SYMS_DwAbbrevFlags flags;
};

////////////////////////////////
//~ rjf: Attribs

typedef struct SYMS_DwAttribValueResolveParams SYMS_DwAttribValueResolveParams;
struct SYMS_DwAttribValueResolveParams
{
  SYMS_DwVersion version;
  SYMS_DwLanguage language;
  SYMS_U64 addr_size;                // NOTE(rjf): size in bytes of containing compilation unit's addresses
  SYMS_U64 containing_unit_info_off; // NOTE(rjf): containing compilation unit's offset into the .debug_info section
  SYMS_U64 debug_addrs_base;         // NOTE(rjf): containing compilation unit's offset into the .debug_addrs section       (DWARF V5 ONLY)
  SYMS_U64 debug_rnglists_base;      // NOTE(rjf): containing compilation unit's offset into the .debug_rnglists section    (DWARF V5 ONLY)
  SYMS_U64 debug_str_offs_base;      // NOTE(rjf): containing compilation unit's offset into the .debug_str_offsets section (DWARF V5 ONLY)
  SYMS_U64 debug_loclists_base;      // NOTE(rjf): containing compilation unit's offset into the .debug_loclists section    (DWARF V5 ONLY)
};

typedef struct SYMS_DwAttribValue SYMS_DwAttribValue;
struct SYMS_DwAttribValue
{
  SYMS_DwSectionKind section;
  SYMS_U64 v[2];
};

typedef struct SYMS_DwAttrib SYMS_DwAttrib;
struct SYMS_DwAttrib
{
  SYMS_U64 info_off;
  SYMS_U64 abbrev_id;
  SYMS_DwAttribKind attrib_kind;
  SYMS_DwFormKind form_kind;
  SYMS_DwAttribClass value_class;
  SYMS_DwAttribValue form_value;
};

typedef struct SYMS_DwAttribArray SYMS_DwAttribArray;
struct SYMS_DwAttribArray
{
  SYMS_DwAttrib *v;
  SYMS_U64 count;
};

typedef struct SYMS_DwAttribNode SYMS_DwAttribNode;
struct SYMS_DwAttribNode
{
  SYMS_DwAttribNode *next;
  SYMS_DwAttrib attrib;
};

typedef struct SYMS_DwAttribList SYMS_DwAttribList;
struct SYMS_DwAttribList
{
  SYMS_DwAttribNode *first;
  SYMS_DwAttribNode *last;
  SYMS_U64 count;
};

typedef struct SYMS_DwAttribListParseResult SYMS_DwAttribListParseResult;
struct SYMS_DwAttribListParseResult
{
  SYMS_DwAttribList attribs;
  SYMS_U64 max_info_off;
  SYMS_U64 max_abbrev_off;
};

////////////////////////////////
//~ rjf: Compilation Units + Accelerators

typedef struct SYMS_DwCompRoot SYMS_DwCompRoot;
struct SYMS_DwCompRoot
{
  // NOTE(rjf): Header Data
  SYMS_U64 size;
  SYMS_DwCompUnitKind kind;
  SYMS_DwVersion version;
  SYMS_U64 address_size;
  SYMS_U64 abbrev_off;
  SYMS_U64 info_off;
  SYMS_U64 index;
  SYMS_U64Range tags_info_range;
  SYMS_DwAbbrevTable abbrev_table;
  
  // NOTE(rjf): [parsed from DWARF attributes] Offsets For More Info (DWARF V5 ONLY)
  SYMS_U64 rnglist_base; // NOTE(rjf): Offset into the .debug_rnglists section where this comp unit's data is.
  SYMS_U64 loclist_base; // NOTE(rjf): Offset into the .debug_loclists section where this comp unit's data is.
  SYMS_U64 addrs_base;   // NOTE(rjf): Offset into the .debug_addr section where this comp unit's data is.
  SYMS_U64 stroffs_base; // NOTE(rjf): Offset into the .debug_str_offsets section where this comp unit's data is.
  
  // NOTE(rjf): [parsed from DWARF attributes] General Info
  SYMS_String8 name;
  SYMS_String8 producer;
  SYMS_String8 compile_dir;
  SYMS_String8 external_dwo_name;
  SYMS_U64 dwo_id;
  SYMS_DwLanguage language;
  SYMS_U64 name_case;
  SYMS_B32 use_utf8;
  SYMS_U64 line_off;
  SYMS_U64 low_pc;
  SYMS_U64 high_pc;
  SYMS_DwAttribValue ranges_attrib_value;
  SYMS_U64 base_addr;
  
  // NOTE(rjf): Line/File Info For This Comp Unit
  SYMS_String8Array dir_table;
  SYMS_DwLineVMFileArray file_table;
};

typedef struct SYMS_DwUnitSetAccelBucket SYMS_DwUnitSetAccelBucket;
struct SYMS_DwUnitSetAccelBucket
{
  SYMS_DwUnitSetAccelBucket *next;
  SYMS_U64 comp_root_idx;
  SYMS_DwCompRoot root;
};

typedef struct SYMS_DwUnitSetAccel SYMS_DwUnitSetAccel;
struct SYMS_DwUnitSetAccel
{
  SYMS_FileFormat format;
  SYMS_U64 root_count;
  SYMS_DwCompRoot *roots;
};

////////////////////////////////
//~ rjf: Tags

typedef struct SYMS_DwTag SYMS_DwTag;
struct SYMS_DwTag
{
  SYMS_DwTag *next_sibling;
  SYMS_DwTag *first_child;
  SYMS_DwTag *last_child;
  SYMS_DwTag *parent;
  SYMS_U64Range info_range;
  SYMS_U64Range abbrev_range;
  SYMS_B32 has_children;
  SYMS_U64 abbrev_id;
  SYMS_DwTagKind kind;
  SYMS_U64 attribs_info_off;
  SYMS_U64 attribs_abbrev_off;
  SYMS_DwAttribList attribs;
};

typedef SYMS_U32 SYMS_DwTagStubFlags;
enum
{
  SYMS_DwTagStubFlag_HasObjectPointerArg  = (1<<0),
  SYMS_DwTagStubFlag_HasLocation          = (1<<1),
  SYMS_DwTagStubFlag_HasExternal          = (1<<2),
  SYMS_DwTagStubFlag_HasSpecification     = (1<<3),
};

typedef struct SYMS_DwTagStub SYMS_DwTagStub;
struct SYMS_DwTagStub
{
  SYMS_SymbolID sid;
  SYMS_DwTagKind kind;
  SYMS_DwTagStubFlags flags;
  SYMS_U64 children_info_off;
  SYMS_U64 attribs_info_off;
  SYMS_U64 attribs_abbrev_off;
  
  // NOTE(rjf): SYMS_DwAttribKind_SPECIFICATION is tacked onto definitions that
  // are filling out more info about a "prototype". That attribute is a reference
  // that points back at the declaration tag. The declaration tag has the
  // SYMS_DwAttribKind_DECLARATION attribute, which is sort of like the reverse
  // of that, except there's no reference. So what we're doing here is just storing
  // a reference on both, that point back to each other, so it's always easy to
  // get from decl => spec, or from spec => decl.
  SYMS_SymbolID ref;
  
  // NOTE(rjf): SYMS_DwAttribKind_ABSTRACT_ORIGIN is tacked onto some definitions
  // that are used to specify information more specific to inlining, while wanting
  // to refer to an "abstract" function DIE, that is not specific to any inline
  // sites. The DWARF generator will not duplicate information across these, so
  // we will occasionally need to look at an abstract origin to get abstract
  // information, like name/linkage-name/etc.
  SYMS_SymbolID abstract_origin;
  
  SYMS_U64 _unused_;
};

typedef struct SYMS_DwTagStubNode SYMS_DwTagStubNode;
struct SYMS_DwTagStubNode
{
  SYMS_DwTagStubNode *next;
  SYMS_DwTagStub stub;
};

typedef struct SYMS_DwTagStubList SYMS_DwTagStubList;
struct SYMS_DwTagStubList
{
  SYMS_DwTagStubNode *first;
  SYMS_DwTagStubNode *last;
  SYMS_U64 count;
};

////////////////////////////////
//~ rjf: Line Info VM Types

typedef struct SYMS_DwLineVMHeader SYMS_DwLineVMHeader;
struct SYMS_DwLineVMHeader
{
  SYMS_U64 unit_length;
  SYMS_U64 unit_opl;
  SYMS_U16 version;
  SYMS_U8 address_size; // NOTE(nick): duplicates size from the compilation unit but is needed to support stripped exe that just have .debug_line and .debug_line_str.
  SYMS_U8 segment_selector_size;
  SYMS_U64 header_length;
  SYMS_U64 program_off;
  SYMS_U8 min_inst_len;
  SYMS_U8 max_ops_for_inst;
  SYMS_U8 default_is_stmt;
  SYMS_S8 line_base;
  SYMS_U8 line_range;
  SYMS_U8 opcode_base;
  SYMS_U64 num_opcode_lens;
  SYMS_U8 *opcode_lens;
  SYMS_String8Array dir_table;
  SYMS_DwLineVMFileArray file_table;
};

typedef struct SYMS_DwLineVMState SYMS_DwLineVMState;
struct SYMS_DwLineVMState
{
  SYMS_U64 address;  // NOTE(nick): Address of a machine instruction.
  SYMS_U32 op_index; // NOTE(nick): This is used by the VLIW instructions to indicate index of operation inside the instruction.
  
  // NOTE(nick): Line table doesn't contain full path to a file, instead
  // DWARF encodes path as two indices. First index will point into a directory
  // table,  and second points into a file name table.
  SYMS_U32 file_index;
  
  SYMS_U32 line;
  SYMS_U32 column;
  
  SYMS_B32 is_stmt;      // NOTE(nick): Indicates that "va" points to place suitable for a breakpoint.
  SYMS_B32 basic_block;  // NOTE(nick): Indicates that the "va" is inside a basic block.
  
  // NOTE(nick): Indicates that "va" points to place where function starts.
  // Usually prologue is the place where compiler emits instructions to 
  // prepare stack for a function.
  SYMS_B32 prologue_end;
  
  SYMS_B32 epilogue_begin;  // NOTE(nick): Indicates that "va" points to section where function exits and unwinds stack.
  SYMS_U64 isa;             // NOTE(nick): Instruction set that is used.
  SYMS_U64 discriminator;   // NOTE(nick): Arbitrary id that indicates to which block these instructions belong.
  SYMS_B32 end_sequence;    // NOTE(nick): Indicates that "va" points to the first instruction in the instruction block that follows.
  
  // NOTE(rjf): it looks like LTO might sometimes zero out high PC and low PCs, causing a
  // swath of line info to map to a range starting at 0. This causes overlapping ranges
  // which we do not want to report. So this B32 will turn on emission.
  SYMS_B32 busted_seq;
};

typedef struct SYMS_DwLineNode SYMS_DwLineNode;
struct SYMS_DwLineNode
{
  SYMS_DwLineNode *next;
  SYMS_Line line;
};

typedef struct SYMS_DwLineSeqNode SYMS_DwLineSeqNode;
struct SYMS_DwLineSeqNode
{
  SYMS_DwLineSeqNode *next;
  SYMS_U64 count;
  SYMS_DwLineNode *first;
  SYMS_DwLineNode *last;
};

typedef struct SYMS_DwLineTableParseResult SYMS_DwLineTableParseResult;
struct SYMS_DwLineTableParseResult
{
  SYMS_U64 seq_count;
  SYMS_DwLineSeqNode *first_seq;
  SYMS_DwLineSeqNode *last_seq;
};

////////////////////////////////
//~ rjf: .debug_pubnames and .debug_pubtypes

typedef struct SYMS_DwPubStringsBucket SYMS_DwPubStringsBucket;
struct SYMS_DwPubStringsBucket
{
  SYMS_DwPubStringsBucket *next;
  SYMS_String8 string;
  SYMS_SymbolID sid;
  SYMS_UnitID uid;
};

typedef struct SYMS_DwPubStringsTable SYMS_DwPubStringsTable;
struct SYMS_DwPubStringsTable
{
  SYMS_U64 size;
  SYMS_DwPubStringsBucket **buckets;
};

////////////////////////////////
//~ rjf: Tag Reference (Specification Attributes etc.) Data Structure

typedef struct SYMS_DwTagRefNode SYMS_DwTagRefNode;
struct SYMS_DwTagRefNode
{
  SYMS_DwTagRefNode *hash_next;
  SYMS_SymbolID dst;
  SYMS_SymbolID src;
};

typedef struct SYMS_DwTagRefTable SYMS_DwTagRefTable;
struct SYMS_DwTagRefTable
{
  SYMS_U64 size;
  SYMS_DwTagRefNode **v;
};

////////////////////////////////
//~ rjf: Unit Accelerator

typedef struct SYMS_DwTagStubCacheNode SYMS_DwTagStubCacheNode;
struct SYMS_DwTagStubCacheNode
{
  SYMS_DwTagStubCacheNode *hash_next;
  SYMS_DwTagStubCacheNode *next;
  SYMS_DwTagStubCacheNode *first;
  SYMS_DwTagStubCacheNode *last;
  SYMS_U64 children_count;
  SYMS_DwTagStub stub;
};

typedef struct SYMS_DwUnitAccel SYMS_DwUnitAccel;
struct SYMS_DwUnitAccel
{
  SYMS_FileFormat format;
  
  //- rjf: header information (from comp root)
  SYMS_UnitID uid;
  SYMS_DwVersion version;
  SYMS_U64 address_size;
  SYMS_U64 base_addr;
  SYMS_U64 addrs_base;
  SYMS_DwLanguage language;
  SYMS_DwAbbrevTable abbrev_table;
  
  //- rjf: tag stub hash table
  SYMS_U64 stub_table_size;
  SYMS_DwTagStubCacheNode **stub_table;
  
  //- rjf: tag reference tables
  SYMS_DwTagRefTable ref_table;
  SYMS_DwTagRefTable parent_table;
  
  //- rjf: tag stub tree
  SYMS_DwTagStubCacheNode *stub_root;
  
  //- rjf: top-level symbol IDs
  SYMS_SymbolIDArray all_top_ids; // TODO(rjf): this will be removed
  SYMS_SymbolIDArray proc_ids;
  SYMS_SymbolIDArray var_ids;
  SYMS_SymbolIDArray type_ids;
  
  // TODO(rjf): hack hack hack, merge with above info
  SYMS_DwAttribValueResolveParams resolve_params;
};

////////////////////////////////
//~ rjf: Members Accelerator

typedef struct SYMS_DwMemsAccel SYMS_DwMemsAccel;
struct SYMS_DwMemsAccel
{
  SYMS_FileFormat format;
  SYMS_U64 count;
  SYMS_MemInfo *mem_infos;
  SYMS_USID *type_symbols;
  SYMS_USID *full_symbols;
  SYMS_SymbolID *sig_symbols;
};

////////////////////////////////
//~ rjf: Name Mapping Accelerator

typedef struct SYMS_DwMapAccel SYMS_DwMapAccel;
struct SYMS_DwMapAccel
{
  SYMS_FileFormat format;
  SYMS_DwPubStringsTable tbl;
};

typedef struct SYMS_DwLinkMapAccel SYMS_DwLinkMapAccel;
struct SYMS_DwLinkMapAccel
{
  SYMS_FileFormat format;
  // TODO(allen): 
};

SYMS_C_LINKAGE_BEGIN //-

////////////////////////////////
//~ rjf: Basic Helpers

SYMS_API SYMS_U64           syms_dw_hash_from_string(SYMS_String8 string);
SYMS_API SYMS_U64           syms_dw_hash_from_sid(SYMS_SymbolID sid);

SYMS_API SYMS_SymbolID      syms_dw_sid_from_info_offset(SYMS_U64 info_offset);
SYMS_API SYMS_DwAttribClass syms_dw_pick_attrib_value_class(SYMS_DwLanguage lang, SYMS_DwVersion ver, SYMS_DwAttribKind attrib, SYMS_DwFormKind form_kind);
SYMS_API SYMS_SymbolKind    syms_dw_symbol_kind_from_tag_stub(SYMS_String8 data, SYMS_DwDbgAccel *dbg, SYMS_DwAttribValueResolveParams resolve_params, SYMS_DwTagStub *stub);

SYMS_API SYMS_SecInfoArray syms_dw_copy_sec_info_array(SYMS_Arena *arena, SYMS_SecInfoArray array);

SYMS_API SYMS_String8 syms_dw_path_from_dir_and_filename(SYMS_Arena *arena, SYMS_String8 dir, SYMS_String8 filename);
SYMS_API void syms_dw_symbol_id_chunk_list_push(SYMS_Arena *arena, SYMS_DwSymbolIDChunkList *list, SYMS_SymbolID sid);
SYMS_API SYMS_SymbolIDArray syms_dw_sid_array_from_chunk_list(SYMS_Arena *arena, SYMS_DwSymbolIDChunkList list);

////////////////////////////////
//~ rjf: DWARF-Specific Based Range Reads

SYMS_API SYMS_U64 syms_dw_based_range_read_length(void *base, SYMS_U64Range range, SYMS_U64 offset, SYMS_U64 *out_value);
SYMS_API SYMS_U64 syms_dw_based_range_read_abbrev_tag(void *base, SYMS_U64Range range, SYMS_U64 offset, SYMS_DwAbbrev *out_abbrev);
SYMS_API SYMS_U64 syms_dw_based_range_read_abbrev_attrib_info(void *base, SYMS_U64Range range, SYMS_U64 offset, SYMS_DwAbbrev *out_abbrev);
SYMS_API SYMS_U64 syms_dw_based_range_read_attrib_form_value(void *base, SYMS_U64Range range, SYMS_U64 offset, SYMS_DwMode mode, SYMS_U64 address_size, SYMS_DwFormKind form_kind, SYMS_U64 implicit_const, SYMS_DwAttribValue *form_value_out);

////////////////////////////////
//~ rjf: Debug Info Accelerator (DbgAccel) code

//- rjf: debug info and interaction with bin accels
SYMS_API SYMS_B32         syms_dw_elf_bin_accel_is_dbg(SYMS_ElfBinAccel *bin_accel);
SYMS_API SYMS_B32         syms_dw_mach_bin_accel_is_dbg(SYMS_MachBinAccel *bin_accel);
SYMS_API SYMS_DwDbgAccel *syms_dw_dbg_accel_from_sec_info_array(SYMS_Arena *arena, SYMS_String8 data, SYMS_U64 vbase, SYMS_Arch arch, SYMS_SecInfoArray sections);
SYMS_API SYMS_DwDbgAccel *syms_dw_dbg_accel_from_elf_bin(SYMS_Arena *arena, SYMS_String8 data, SYMS_ElfBinAccel *bin);
SYMS_API SYMS_DwDbgAccel *syms_dw_dbg_accel_from_mach_bin(SYMS_Arena *arena, SYMS_String8 data, SYMS_MachBinAccel *bin);
SYMS_API SYMS_ExtFileList syms_dw_ext_file_list_from_dbg(SYMS_Arena *arena, SYMS_String8 data,
                                                         SYMS_DwDbgAccel *dbg);
SYMS_API SYMS_SecInfoArray syms_dw_sec_info_array_from_dbg(SYMS_Arena *arena, SYMS_String8 data, SYMS_DwDbgAccel *dbg);
SYMS_API SYMS_ExtMatchKey  syms_dw_ext_match_key_from_dbg(SYMS_String8 data, SYMS_DwDbgAccel *dbg);
SYMS_API SYMS_U64          syms_dw_default_vbase_from_dbg(SYMS_DwDbgAccel *dbg);

//- rjf: top-level unit info
SYMS_API SYMS_UnitID syms_dw_uid_from_foff(SYMS_DwDbgAccel *dbg, SYMS_U64 foff);

//- rjf: important DWARF section base/range accessors
SYMS_API SYMS_DwMode syms_dw_mode_from_sec(SYMS_DwSection *section_map, SYMS_DwSectionKind kind);

SYMS_API SYMS_B32      syms_dw_sec_is_present(SYMS_DwDbgAccel *dbg, SYMS_DwSectionKind kind);
SYMS_API void*         syms_dw_sec_base_from_dbg(SYMS_String8 data, SYMS_DwDbgAccel *dbg,
                                                 SYMS_DwSectionKind kind);
SYMS_API SYMS_U64Range syms_dw_sec_range_from_dbg(SYMS_DwDbgAccel *dbg, SYMS_DwSectionKind kind);

////////////////////////////////
//~ rjf: Abbrev Table

SYMS_API SYMS_DwAbbrevTable syms_dw_make_abbrev_table(SYMS_Arena *arena, SYMS_String8 data, SYMS_DwDbgAccel *dbg,
                                                      SYMS_U64 start_abbrev_off);
SYMS_API SYMS_U64           syms_dw_abbrev_offset_from_abbrev_id(SYMS_DwAbbrevTable table, SYMS_U64 abbrev_id);

////////////////////////////////
//~ rjf: Miscellaneous DWARF Section Parsing

//- rjf: .debug_ranges (DWARF V4)
SYMS_API SYMS_U64RangeList      syms_dw_v4_range_list_from_range_offset(SYMS_Arena *arena, SYMS_String8 data, SYMS_DwDbgAccel *dbg, SYMS_U64 addr_size, SYMS_U64 comp_unit_base_addr, SYMS_U64 range_off);

//- rjf: .debug_loc (DWARF V4)
SYMS_API SYMS_LocRangeList      syms_dw_v4_location_ranges_from_loc_offset(SYMS_Arena *arena, SYMS_String8 data, SYMS_DwDbgAccel *dbg, SYMS_U64 addr_size, SYMS_U64 comp_unit_base_addr, SYMS_U64 offset);
SYMS_API SYMS_Location          syms_dw_v4_location_from_loc_id(SYMS_Arena *arena, SYMS_String8 data, SYMS_DwDbgAccel *dbg, SYMS_LocID loc);

//- rjf: .debug_pubtypes + .debug_pubnames (DWARF V4)
SYMS_API SYMS_DwPubStringsTable syms_dw_v4_pub_strings_table_from_section_kind(SYMS_Arena *arena, SYMS_String8 data,
                                                                               SYMS_DwDbgAccel *dbg,
                                                                               SYMS_DwSectionKind section_kind);
SYMS_API SYMS_USIDList          syms_dw_v4_usid_list_from_pub_table_string(SYMS_Arena *arena,
                                                                           SYMS_DwPubStringsTable tbl,
                                                                           SYMS_String8 string);

//- rjf: .debug_str_offsets (DWARF V5)
SYMS_API SYMS_U64 syms_dw_v5_offset_from_offs_section_base_index(SYMS_String8 data, SYMS_DwDbgAccel *dbg, SYMS_DwSectionKind section, SYMS_U64 base, SYMS_U64 index);

//- rjf: .debug_addr (DWARF V5)
SYMS_API SYMS_U64 syms_dw_v5_addr_from_addrs_section_base_index(SYMS_String8 data, SYMS_DwDbgAccel *dbg, SYMS_DwSectionKind section, SYMS_U64 base, SYMS_U64 index);

//- rjf: .debug_rnglists parsing (DWARF V5)
SYMS_API SYMS_U64 syms_dw_v5_sec_offset_from_rnglist_or_loclist_section_base_index(SYMS_String8 data, SYMS_DwDbgAccel *dbg, SYMS_DwSectionKind section_kind, SYMS_U64 base, SYMS_U64 index);
SYMS_API SYMS_U64RangeList syms_dw_v5_range_list_from_rnglist_offset(SYMS_Arena *arena, SYMS_String8 data, SYMS_DwDbgAccel *dbg, SYMS_DwSectionKind section, SYMS_U64 addr_size, SYMS_U64 addr_section_base, SYMS_U64 offset);

//- rjf: .debug_loclists parsing (DWARF V5)
SYMS_API SYMS_LocRangeList syms_dw_v5_location_ranges_from_loclist_offset(SYMS_Arena *arena, SYMS_String8 data, SYMS_DwDbgAccel *dbg, SYMS_DwSectionKind section, SYMS_U64 addr_size, SYMS_U64 addr_section_base, SYMS_U64 offset);
SYMS_API SYMS_Location syms_dw_v5_location_from_loclist_id(SYMS_Arena *arena, SYMS_String8 data, SYMS_DwDbgAccel *dbg, SYMS_DwSectionKind section, SYMS_LocID id);

////////////////////////////////
//~ rjf: Attrib Value Parsing

SYMS_API SYMS_DwAttribValueResolveParams syms_dw_attrib_value_resolve_params_from_comp_root(SYMS_DwCompRoot *root);
SYMS_API SYMS_DwAttribValue syms_dw_attrib_value_from_form_value(SYMS_String8 data, SYMS_DwDbgAccel *dbg,
                                                                 SYMS_DwAttribValueResolveParams resolve_params,
                                                                 SYMS_DwFormKind form_kind,
                                                                 SYMS_DwAttribClass value_class,
                                                                 SYMS_DwAttribValue form_value);
SYMS_API SYMS_String8       syms_dw_string_from_attrib_value(SYMS_String8 data, SYMS_DwDbgAccel *dbg,
                                                             SYMS_DwAttribValue value);
SYMS_API SYMS_U64RangeList syms_dw_range_list_from_high_low_pc_and_ranges_attrib_value(SYMS_Arena *arena, SYMS_String8 data, SYMS_DwDbgAccel *dbg, SYMS_U64 address_size, SYMS_U64 comp_unit_base_addr, SYMS_U64 addr_section_base, SYMS_U64 low_pc, SYMS_U64 high_pc, SYMS_DwAttribValue ranges_value);

////////////////////////////////
//~ rjf: Tag Parsing

SYMS_API SYMS_DwAttribListParseResult syms_dw_parse_attrib_list_from_info_abbrev_offsets(SYMS_Arena *arena, SYMS_String8 data, SYMS_DwDbgAccel *dbg, SYMS_DwLanguage lang, SYMS_DwVersion ver, SYMS_U64 address_size, SYMS_U64 info_off, SYMS_U64 abbrev_off);
SYMS_API SYMS_DwTag *syms_dw_tag_from_info_offset(SYMS_Arena *arena,
                                                  SYMS_String8 data, SYMS_DwDbgAccel *dbg,
                                                  SYMS_DwAbbrevTable abbrev_table,
                                                  SYMS_DwLanguage lang,
                                                  SYMS_DwVersion ver,
                                                  SYMS_U64 address_size,
                                                  SYMS_U64 info_offset);
SYMS_API SYMS_DwTagStub syms_dw_stub_from_tag(SYMS_String8 data, SYMS_DwDbgAccel *dbg, SYMS_DwAttribValueResolveParams resolve_params,
                                              SYMS_DwTag *tag);

////////////////////////////////
//~ rjf: Unit Set Accelerator

SYMS_API SYMS_U64            syms_dw_v5_header_offset_from_table_offset(SYMS_String8 data, SYMS_DwDbgAccel *dbg,
                                                                        SYMS_DwSectionKind section,
                                                                        SYMS_U64 table_off);
SYMS_API SYMS_DwCompRoot     syms_dw_comp_root_from_range(SYMS_Arena *arena,
                                                          SYMS_String8 data, SYMS_DwDbgAccel *dbg,
                                                          SYMS_U64 index, SYMS_U64Range range);
SYMS_API SYMS_DwExtDebugRef  syms_dw_ext_debug_ref_from_comp_root(SYMS_DwCompRoot *root);
SYMS_API SYMS_DwUnitSetAccel*syms_dw_unit_set_accel_from_dbg(SYMS_Arena *arena, SYMS_String8 data,
                                                             SYMS_DwDbgAccel *dbg);
SYMS_API SYMS_U64            syms_dw_unit_count_from_set(SYMS_DwUnitSetAccel *unit_set);
SYMS_API SYMS_DwCompRoot*    syms_dw_comp_root_from_uid(SYMS_DwUnitSetAccel *unit_set, SYMS_UnitID uid);
SYMS_API SYMS_UnitID         syms_dw_uid_from_number(SYMS_DwUnitSetAccel *unit_set, SYMS_U64 n);
SYMS_API SYMS_U64            syms_dw_unit_number_from_uid(SYMS_DwUnitSetAccel *unit_set, SYMS_UnitID uid);
SYMS_API SYMS_UnitInfo       syms_dw_unit_info_from_uid(SYMS_DwUnitSetAccel *unit_set, SYMS_UnitID uid);
SYMS_API SYMS_UnitNames      syms_dw_unit_names_from_uid(SYMS_Arena *arena, SYMS_DwUnitSetAccel *unit_set,
                                                         SYMS_UnitID uid);
SYMS_API void                syms_dw_sort_unit_range_point_array_in_place__merge(SYMS_DwUnitRangePoint *a, SYMS_U64 left, SYMS_U64 right, SYMS_U64 end, SYMS_DwUnitRangePoint *b);
SYMS_API void                syms_dw_sort_unit_range_point_array_in_place(SYMS_DwUnitRangePoint *a, SYMS_U64 count);
SYMS_API SYMS_UnitRangeArray syms_dw_unit_ranges_from_set(SYMS_Arena *arena,
                                                          SYMS_String8 data, SYMS_DwDbgAccel *dbg,
                                                          SYMS_DwUnitSetAccel *unit_set);
SYMS_API SYMS_String8Array   syms_dw_file_table_from_uid(SYMS_Arena *arena,
                                                         SYMS_String8 data, SYMS_DwDbgAccel *dbg,
                                                         SYMS_DwUnitSetAccel *unit_set, SYMS_UnitID uid);

////////////////////////////////
//~ rjf: Tag Reference Data Structure

SYMS_API SYMS_DwTagRefTable syms_dw_tag_ref_table_make(SYMS_Arena *arena, SYMS_U64 size);
SYMS_API void syms_dw_tag_ref_table_insert(SYMS_Arena *arena, SYMS_DwTagRefTable *table, SYMS_SymbolID src, SYMS_SymbolID dst);
SYMS_API SYMS_SymbolID syms_dw_tag_ref_table_lookup_src(SYMS_DwTagRefTable table, SYMS_SymbolID dst);

////////////////////////////////
//~ rjf: Unit Symbol Accelerator

SYMS_API SYMS_U64                   syms_dw_primify_table_size(SYMS_U64 v);
SYMS_API SYMS_U64                   syms_dw_predict_good_stub_table_size_from_range_size(SYMS_U64 size);
SYMS_API SYMS_DwUnitAccel*          syms_dw_unit_accel_from_comp_root(SYMS_Arena *arena, SYMS_String8 data,
                                                                      SYMS_DwDbgAccel *dbg,
                                                                      SYMS_DwCompRoot *comp_root);
SYMS_API SYMS_DwUnitAccel*          syms_dw_unit_accel_from_uid(SYMS_Arena *arena, SYMS_String8 data,
                                                                SYMS_DwDbgAccel *dbg,
                                                                SYMS_DwUnitSetAccel *unit_set,
                                                                SYMS_UnitID uid);
SYMS_API SYMS_UnitID                syms_dw_uid_from_accel(SYMS_DwUnitAccel *unit);
SYMS_API SYMS_DwTagStubCacheNode *  syms_dw_tag_stub_cache_node_from_sid(SYMS_DwUnitAccel *unit, SYMS_SymbolID sid);
SYMS_API SYMS_DwTagStub             syms_dw_tag_stub_from_sid(SYMS_String8 data, SYMS_DwDbgAccel *dbg, SYMS_DwUnitAccel *unit, SYMS_SymbolID sid);
SYMS_API SYMS_DwTagStub             syms_dw_cached_tag_stub_from_sid__parse_fallback(SYMS_String8 data, SYMS_DwDbgAccel *dbg, SYMS_DwUnitAccel *unit, SYMS_SymbolID sid);
SYMS_API SYMS_DwTagStubList         syms_dw_children_from_tag_stub(SYMS_Arena *arena, SYMS_String8 data, SYMS_DwDbgAccel *dbg, SYMS_DwUnitAccel *unit, SYMS_DwTagStub stub);
SYMS_API SYMS_DwAttribList          syms_dw_attrib_list_from_stub(SYMS_Arena *arena, SYMS_String8 data, SYMS_DwDbgAccel *dbg, SYMS_DwLanguage lang, SYMS_DwVersion ver, SYMS_U64 addr_size, SYMS_DwTagStub *stub);
SYMS_API SYMS_SymbolIDArray         syms_dw_copy_sid_array_if_needed(SYMS_Arena *arena, SYMS_SymbolIDArray arr);
SYMS_API SYMS_SymbolIDArray         syms_dw_proc_sid_array_from_unit(SYMS_Arena *arena, SYMS_DwUnitAccel *unit);
SYMS_API SYMS_SymbolIDArray         syms_dw_var_sid_array_from_unit(SYMS_Arena *arena, SYMS_DwUnitAccel *unit);
SYMS_API SYMS_SymbolIDArray         syms_dw_type_sid_array_from_unit(SYMS_Arena *arena, SYMS_DwUnitAccel *unit);

////////////////////////////////
//~ rjf: Members

SYMS_API SYMS_DwMemsAccel* syms_dw_mems_accel_from_sid(SYMS_Arena *arena, SYMS_String8 data,
                                                       SYMS_DwDbgAccel *dbg, SYMS_DwUnitAccel *unit,
                                                       SYMS_SymbolID sid);
SYMS_API SYMS_U64          syms_dw_mem_count_from_mems(SYMS_DwMemsAccel *mems);
SYMS_API SYMS_MemInfo      syms_dw_mem_info_from_number(SYMS_Arena *arena, SYMS_String8 data,
                                                        SYMS_DwDbgAccel *dbg, SYMS_DwUnitAccel *unit,
                                                        SYMS_DwMemsAccel *mems, SYMS_U64 n);
SYMS_API SYMS_USID         syms_dw_type_from_mem_number(SYMS_String8 data, SYMS_DwDbgAccel *dbg,
                                                        SYMS_DwUnitAccel *unit, SYMS_DwMemsAccel *mems,
                                                        SYMS_U64 n);
SYMS_API SYMS_SigInfo      syms_dw_sig_info_from_mem_number(SYMS_Arena *arena, SYMS_String8 data, SYMS_DwDbgAccel *dbg,
                                                            SYMS_DwUnitAccel *unit, SYMS_DwMemsAccel *mems,
                                                            SYMS_U64 n);
SYMS_API SYMS_USID         syms_dw_symbol_from_mem_number(SYMS_String8 data, SYMS_DwDbgAccel *dbg,
                                                          SYMS_DwUnitAccel *unit, SYMS_DwMemsAccel *mems,
                                                          SYMS_U64 n);
SYMS_API SYMS_EnumMemberArray syms_dw_enum_member_array_from_sid(SYMS_Arena *arena, SYMS_String8 data, SYMS_DwDbgAccel *dbg,
                                                                 SYMS_DwUnitAccel *unit, SYMS_SymbolID sid);
SYMS_API SYMS_USID          syms_dw_containing_type_from_sid(SYMS_String8 data, SYMS_DwDbgAccel *dbg, SYMS_DwUnitAccel *unit, SYMS_SymbolID sid);
SYMS_API SYMS_String8       syms_dw_linkage_name_from_sid(SYMS_Arena *arena, SYMS_String8 data, SYMS_DwDbgAccel *dbg, SYMS_DwUnitAccel *unit, SYMS_SymbolID sid);

////////////////////////////////
//~ rjf: Full Symbol Info Parsing

SYMS_API SYMS_String8      syms_dw_attrib_string_from_sid__unstable(SYMS_String8 data, SYMS_DwDbgAccel *dbg,
                                                                    SYMS_DwUnitAccel *unit,
                                                                    SYMS_DwAttribKind kind, SYMS_SymbolID sid);
SYMS_API SYMS_String8      syms_dw_attrib_string_from_sid__unstable_chain(SYMS_String8 data, SYMS_DwDbgAccel *dbg,
                                                                          SYMS_DwUnitAccel *unit,
                                                                          SYMS_DwAttribKind kind, SYMS_SymbolID sid);
SYMS_API SYMS_SymbolKind   syms_dw_symbol_kind_from_sid(SYMS_String8 data, SYMS_DwDbgAccel *dbg,
                                                        SYMS_DwUnitAccel *unit, SYMS_SymbolID sid);
SYMS_API SYMS_String8      syms_dw_symbol_name_from_sid(SYMS_Arena *arena, SYMS_String8 data, SYMS_DwDbgAccel *dbg,
                                                        SYMS_DwUnitAccel *unit, SYMS_SymbolID sid);
SYMS_API SYMS_TypeInfo     syms_dw_type_info_from_sid(SYMS_String8 data, SYMS_DwDbgAccel *dbg,
                                                      SYMS_DwUnitAccel *unit, SYMS_SymbolID sid);
SYMS_API SYMS_ConstInfo    syms_dw_const_info_from_sid(SYMS_String8 data, SYMS_DwDbgAccel *dbg,
                                                       SYMS_DwUnitAccel *unit, SYMS_SymbolID sid);
SYMS_API SYMS_USID         syms_dw_type_from_var_sid(SYMS_String8 data, SYMS_DwDbgAccel *dbg,
                                                     SYMS_DwUnitAccel *unit, SYMS_SymbolID sid);
SYMS_API SYMS_U64          syms_dw_voff_from_var_sid(SYMS_String8 data, SYMS_DwDbgAccel *dbg,
                                                     SYMS_DwUnitAccel *unit, SYMS_SymbolID sid);
SYMS_API SYMS_SymbolIDList syms_dw_children_from_sid_with_kinds(SYMS_Arena *arena, SYMS_String8 data, SYMS_DwDbgAccel *dbg, SYMS_DwUnitAccel *unit, SYMS_SymbolID sid, SYMS_DwTagKind *kinds, SYMS_U64 count);

//- rjf: location information
SYMS_API SYMS_Location     syms_dw_location_from_sid(SYMS_Arena *arena, SYMS_String8 data, SYMS_DwDbgAccel *dbg, SYMS_DwUnitAccel *unit, SYMS_SymbolID sid, SYMS_DwAttribKind loc_attrib);
SYMS_API SYMS_LocRangeArray syms_dw_location_ranges_from_sid(SYMS_Arena *arena, SYMS_String8 data, SYMS_DwDbgAccel *dbg,
                                                             SYMS_DwUnitAccel *unit, SYMS_SymbolID sid, SYMS_DwAttribKind loc_attrib);
SYMS_API SYMS_Location     syms_dw_location_from_id(SYMS_Arena *arena, SYMS_String8 data, SYMS_DwDbgAccel *dbg,
                                                    SYMS_DwUnitAccel *unit, SYMS_LocID loc_id);

//- rjf: location information helpers
SYMS_API SYMS_DwAttribKind syms_dw_attrib_kind_from_proc_loc(SYMS_ProcLoc proc_loc);
SYMS_API SYMS_Location     syms_dw_location_from_var_sid(SYMS_Arena *arena, SYMS_String8 data, SYMS_DwDbgAccel *dbg, SYMS_DwUnitAccel *unit, SYMS_SymbolID sid);
SYMS_API SYMS_LocRangeArray syms_dw_location_ranges_from_var_sid(SYMS_Arena *arena, SYMS_String8 data, SYMS_DwDbgAccel *dbg,
                                                                 SYMS_DwUnitAccel *unit, SYMS_SymbolID sid);
SYMS_API SYMS_Location     syms_dw_location_from_proc_sid(SYMS_Arena *arena, SYMS_String8 data, SYMS_DwDbgAccel *dbg, SYMS_DwUnitAccel *unit, SYMS_SymbolID sid, SYMS_ProcLoc proc_loc);
SYMS_API SYMS_LocRangeArray syms_dw_location_ranges_from_proc_sid(SYMS_Arena *arena, SYMS_String8 data, SYMS_DwDbgAccel *dbg,
                                                                  SYMS_DwUnitAccel *unit, SYMS_SymbolID sid, SYMS_ProcLoc proc_loc);

//- rjf: files
SYMS_API SYMS_U64          syms_dw_file_index_from_id(SYMS_FileID file_id);
SYMS_API SYMS_FileID       syms_dw_file_id_from_index(SYMS_U64 idx);
SYMS_API SYMS_String8      syms_dw_file_name_from_id(SYMS_Arena *arena, SYMS_DwUnitSetAccel *unit_set,
                                                     SYMS_UnitID uid, SYMS_FileID file_id);

//- rjf: procedures
SYMS_API SYMS_U64RangeArray syms_dw_scope_vranges_from_sid(SYMS_Arena *arena, SYMS_String8 data,
                                                           SYMS_DwDbgAccel *dbg, SYMS_DwUnitAccel *unit,
                                                           SYMS_SymbolID sid);
SYMS_API SYMS_SigInfo      syms_dw_sig_info_from_sid(SYMS_Arena *arena, SYMS_String8 data, SYMS_DwDbgAccel *dbg,
                                                     SYMS_DwUnitAccel *unit, SYMS_SymbolID sid);
SYMS_API SYMS_UnitIDAndSig syms_dw_proc_sig_handle_from_sid(SYMS_String8 data, SYMS_DwDbgAccel *dbg,
                                                            SYMS_DwUnitAccel *unit, SYMS_SymbolID sid);
SYMS_API SYMS_SigInfo      syms_dw_sig_info_from_handle(SYMS_Arena *arena, SYMS_String8 data, SYMS_DwDbgAccel *dbg, SYMS_DwUnitAccel *unit, SYMS_SigHandle handle);
SYMS_API SYMS_SymbolIDArray syms_dw_scope_children_from_sid(SYMS_Arena *arena, SYMS_String8 data, SYMS_DwDbgAccel *dbg, SYMS_DwUnitAccel *unit, SYMS_SymbolID id);

//- rjf: line info
SYMS_API void              syms_dw_line_vm_reset(SYMS_DwLineVMState *state, SYMS_B32 default_is_stmt);
SYMS_API void              syms_dw_line_vm_advance(SYMS_DwLineVMState *state, SYMS_U64 advance,
                                                   SYMS_U64 min_inst_len, SYMS_U64 max_ops_for_inst);

SYMS_API SYMS_DwLineSeqNode *syms_dw_push_line_seq(SYMS_Arena* arena, SYMS_DwLineTableParseResult *parsed_tbl);
SYMS_API SYMS_DwLineNode *syms_dw_push_line(SYMS_Arena *arena, SYMS_DwLineTableParseResult *tbl, SYMS_DwLineVMState *vm_state, SYMS_B32 start_of_sequence);
SYMS_API SYMS_DwLineTableParseResult syms_dw_parsed_line_table_from_comp_root(SYMS_Arena *arena, SYMS_String8 data,
                                                                              SYMS_DwDbgAccel *dbg, SYMS_DwCompRoot *root);
SYMS_API SYMS_LineParseOut syms_dw_line_parse_from_uid(SYMS_Arena *arena, SYMS_String8 data, SYMS_DwDbgAccel *dbg,
                                                       SYMS_DwUnitSetAccel *set, SYMS_UnitID uid);
SYMS_API SYMS_U64 syms_dw_read_line_file(void *line_base, SYMS_U64Range line_rng, SYMS_U64 line_off,
                                         SYMS_DwMode mode, SYMS_DwDbgAccel *dbg,  SYMS_String8 data, SYMS_DwCompRoot *unit,
                                         SYMS_U8 address_size, 
                                         SYMS_U64 format_count, SYMS_U64Range *formats,
                                         SYMS_DwLineFile *line_file_out);
SYMS_API SYMS_U64 syms_dw_read_line_vm_header(SYMS_Arena *arena, void *line_base, SYMS_U64Range line_rng,
                                              SYMS_U64 line_off, SYMS_String8 data, SYMS_DwMode mode, SYMS_DwDbgAccel *dbg,
                                              SYMS_DwCompRoot *unit, SYMS_DwLineVMHeader *header_out);

//- rjf: name maps
SYMS_API SYMS_DwMapAccel*       syms_dw_type_map_from_dbg(SYMS_Arena *arena, SYMS_String8 data, SYMS_DwDbgAccel *dbg);
SYMS_API SYMS_DwMapAccel*       syms_dw_image_symbol_map_from_dbg(SYMS_Arena *arena, SYMS_String8 data, SYMS_DwDbgAccel *dbg);
SYMS_API SYMS_USIDList          syms_dw_usid_list_from_string(SYMS_Arena *arena,
                                                              SYMS_DwMapAccel *map, SYMS_String8 string);

SYMS_C_LINKAGE_END

#endif // SYMS_DWARF_PARSER_H
