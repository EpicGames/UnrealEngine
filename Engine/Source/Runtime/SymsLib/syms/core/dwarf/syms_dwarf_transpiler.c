// Copyright Epic Games, Inc. All Rights Reserved.

////////////////////////////////
//~ NOTE(allen): Dwarf Transpiler Functions

SYMS_API SYMS_RegID
syms_reg_id_from_dw_reg(SYMS_Arch arch, SYMS_U64 reg_idx){
  SYMS_U64 count = 0;
  SYMS_RegID *tbl = 0;
  switch (arch){
    case SYMS_Arch_X86: tbl = syms_dw_reg_table_x86; count = SYMS_ARRAY_SIZE(syms_dw_reg_table_x86); break;
    case SYMS_Arch_X64: tbl = syms_dw_reg_table_x64; count = SYMS_ARRAY_SIZE(syms_dw_reg_table_x64); break;
  }
  SYMS_RegID result = 0;
  if (reg_idx < count){
    result = tbl[reg_idx];
  }
  return(result);
}

SYMS_API SYMS_U64
syms_reg_off_from_dw_reg(SYMS_Arch arch, SYMS_U64 reg_idx){
  SYMS_U64 count = 0;
  SYMS_RegID *tbl = 0;
  SYMS_RegSection *sec_tbl = 0;
  switch (arch){
    case SYMS_Arch_X86:
    {
      tbl = syms_dw_reg_table_x86;
      count = SYMS_ARRAY_SIZE(syms_dw_reg_table_x86);
      sec_tbl = syms_reg_section_X86;
    }break;
    case SYMS_Arch_X64:
    {
      tbl = syms_dw_reg_table_x64;
      count = SYMS_ARRAY_SIZE(syms_dw_reg_table_x64);
      sec_tbl = syms_reg_section_X64;
    }break;
  }
  SYMS_U64 result = 0;
  if (reg_idx < count){
    SYMS_RegID reg_id = tbl[reg_idx];
    result = sec_tbl[reg_id].off;
  }
  return(result);
}

SYMS_API SYMS_DwEvalIRGraphNode*
syms_dw_expr__ir_push_node(SYMS_Arena *arena, SYMS_DwEvalIRGraph *graph,
                           SYMS_EvalOp op, SYMS_EvalOpParams params){
  SYMS_DwEvalIRGraphNode *result = syms_push_array_zero(arena, SYMS_DwEvalIRGraphNode, 1);
  SYMS_QueuePush(graph->first, graph->last, result);
  result->op = op;
  result->params = params;
  return(result);
}

SYMS_API SYMS_DwEvalIRGraphNode*
syms_dw_expr__ir_encode_u(SYMS_Arena *arena, SYMS_DwEvalIRGraph *graph, SYMS_U64 u){
  // TODO(allen): deduplicate with op encode
  SYMS_EvalOp op = SYMS_EvalOp_ConstU64;
  if (u <= SYMS_U8_MAX){
    op = SYMS_EvalOp_ConstU8;
  }
  else if (u <= SYMS_U16_MAX){
    op = SYMS_EvalOp_ConstU16;
  }
  else if (u <= SYMS_U32_MAX){
    op = SYMS_EvalOp_ConstU32;
  }
  
  SYMS_DwEvalIRGraphNode *result = syms_dw_expr__ir_push_node(arena, graph, op, syms_eval_op_params(u));
  return(result);
}

SYMS_API SYMS_DwEvalIRGraphNode*
syms_dw_expr__ir_encode_s(SYMS_Arena *arena, SYMS_DwEvalIRGraph *graph, SYMS_S64 s){
  // TODO(allen): deduplicate with op encode
  SYMS_U64 size = 64;
  SYMS_EvalOp op = SYMS_EvalOp_ConstU64;
  if (SYMS_S8_MIN <= s && s <= SYMS_S8_MAX){
    op = SYMS_EvalOp_ConstU8;
    size = 8;
  }
  else if (SYMS_S16_MIN <= s && s <= SYMS_S16_MAX){
    op = SYMS_EvalOp_ConstU16;
    size = 16;
  }
  else if (SYMS_S32_MIN <= s && s <= SYMS_S32_MAX){
    op = SYMS_EvalOp_ConstU32;
    size = 32;
  }
  
  SYMS_DwEvalIRGraphNode *result = syms_dw_expr__ir_push_node(arena, graph, op, syms_eval_op_params(s));
  if (size < 64){
    syms_dw_expr__ir_push_node(arena, graph, SYMS_EvalOp_TruncSigned, syms_eval_op_params(size));
  }
  return(result);
}

SYMS_API SYMS_DwEvalIRGraph
syms_dw_expr__ir_graph_from_dw_expr(SYMS_Arena *arena, SYMS_DwDbgAccel *dbg,
                                    void *expr_base, SYMS_U64Range expr_range){
  SYMS_DwEvalIRGraph graph = {0};
  
  SYMS_Arch arch = dbg->arch;
  
  // scratch
  SYMS_ArenaTemp scratch = syms_get_scratch(&arena, 1);
  
  SYMS_DwEvalJumpPatch *jump_patches = 0;
  
  SYMS_U64 off_count = expr_range.max - expr_range.min;
  SYMS_DwEvalIRGraphNode **off_tbl = syms_push_array_zero(scratch.arena, SYMS_DwEvalIRGraphNode*,
                                                          off_count);
  
  // setup decoder
  SYMS_U64 cursor = 0;
  
  //- step 1: decode
  for (;;){
    
    // decode op
    SYMS_U64 op_offset = cursor;
    SYMS_U8 op = 0;
    if (syms_based_range_read(expr_base, expr_range, op_offset, 1, &op)){
      SYMS_U64 after_op_off = cursor + 1;
      
      // step params
      SYMS_U64 size_param = 0;
      SYMS_B8  is_signed = 0;
      
      // new graph node
      SYMS_DwEvalIRGraphNode *new_node = 0;
      
      // step
      SYMS_U64 step_cursor = after_op_off;
      switch (op){
        
        //// literal encodings ////
        
        case SYMS_DwOp_LIT0:  case SYMS_DwOp_LIT1:  case SYMS_DwOp_LIT2:
        case SYMS_DwOp_LIT3:  case SYMS_DwOp_LIT4:  case SYMS_DwOp_LIT5:
        case SYMS_DwOp_LIT6:  case SYMS_DwOp_LIT7:  case SYMS_DwOp_LIT8:
        case SYMS_DwOp_LIT9:  case SYMS_DwOp_LIT10: case SYMS_DwOp_LIT11:
        case SYMS_DwOp_LIT12: case SYMS_DwOp_LIT13: case SYMS_DwOp_LIT14:
        case SYMS_DwOp_LIT15: case SYMS_DwOp_LIT16: case SYMS_DwOp_LIT17:
        case SYMS_DwOp_LIT18: case SYMS_DwOp_LIT19: case SYMS_DwOp_LIT20:
        case SYMS_DwOp_LIT21: case SYMS_DwOp_LIT22: case SYMS_DwOp_LIT23:
        case SYMS_DwOp_LIT24: case SYMS_DwOp_LIT25: case SYMS_DwOp_LIT26:
        case SYMS_DwOp_LIT27: case SYMS_DwOp_LIT28: case SYMS_DwOp_LIT29:
        case SYMS_DwOp_LIT30: case SYMS_DwOp_LIT31:
        {
          new_node = syms_dw_expr__ir_encode_u(arena, &graph, op - SYMS_DwOp_LIT0);
        }break;
        
        case SYMS_DwOp_CONST1U:size_param = 1; goto const_n;
        case SYMS_DwOp_CONST2U:size_param = 2; goto const_n;
        case SYMS_DwOp_CONST4U:size_param = 4; goto const_n;
        case SYMS_DwOp_CONST8U:size_param = 8; goto const_n;
        case SYMS_DwOp_CONST1S:size_param = 1; is_signed = 1; goto const_n;
        case SYMS_DwOp_CONST2S:size_param = 2; is_signed = 1; goto const_n;
        case SYMS_DwOp_CONST4S:size_param = 4; is_signed = 1; goto const_n;
        case SYMS_DwOp_CONST8S:size_param = 8; is_signed = 1; goto const_n;
        const_n:
        {
          SYMS_U64 x = 0;
          step_cursor += syms_based_range_read(expr_base, expr_range, step_cursor, size_param, &x);
          if (is_signed){
            new_node = syms_dw_expr__ir_encode_s(arena, &graph, (SYMS_S64)x);
          }
          else{
            new_node = syms_dw_expr__ir_encode_u(arena, &graph, x);
          }
        }break;
        
        case SYMS_DwOp_ADDR:
        {
          SYMS_U64 raw_offset = 0;
          step_cursor += syms_based_range_read(expr_base, expr_range, step_cursor, 8, &raw_offset);
          
          SYMS_SecInfo *text_sec = dbg->sections.sec_info + dbg->text_section_idx;
          SYMS_U64 text_voff = text_sec->vrange.min;
          
          SYMS_U64 offset = raw_offset + text_voff;
          
          if (offset < 0xFFFFFFFF){
            new_node = syms_dw_expr__ir_push_node(arena, &graph, SYMS_EvalOp_ModuleOff, syms_eval_op_params(offset));
          }
          else{
            new_node = syms_dw_expr__ir_push_node(arena, &graph, SYMS_EvalOp_ModuleOff,
                                                  syms_eval_op_params(0xFFFFFFFF));
            syms_dw_expr__ir_encode_u(arena, &graph, offset - 0xFFFFFFFF);
            syms_dw_expr__ir_push_node(arena, &graph, SYMS_EvalOp_Add, syms_eval_op_params(SYMS_EvalTypeGroup_U));
          }
        }break;
        
        case SYMS_DwOp_CONSTU:
        {
          SYMS_U64 x = 0;
          step_cursor += syms_based_range_read_uleb128(expr_base, expr_range, step_cursor, &x);
          new_node = syms_dw_expr__ir_encode_u(arena, &graph, x);
        }break;
        
        case SYMS_DwOp_CONSTS:
        {
          SYMS_S64 x = 0;
          step_cursor += syms_based_range_read_sleb128(expr_base, expr_range, step_cursor, &x);
          new_node = syms_dw_expr__ir_encode_s(arena, &graph, x);
        }break;
        
        
        //// register based addressing ////
        
        case SYMS_DwOp_FBREG:
        {
          SYMS_U64 offset = 0;
          step_cursor += syms_based_range_read_sleb128(expr_base, expr_range, step_cursor, (SYMS_S64*)&offset);
          
          if (offset < 256){
            new_node = syms_dw_expr__ir_push_node(arena, &graph, SYMS_EvalOp_FrameOff,
                                                  syms_eval_op_params((SYMS_U64)offset));
          }
          else{
            new_node = syms_dw_expr__ir_push_node(arena, &graph, SYMS_EvalOp_FrameOff, syms_eval_op_params(255));
            syms_dw_expr__ir_encode_u(arena, &graph, offset - 255);
            syms_dw_expr__ir_push_node(arena, &graph, SYMS_EvalOp_Add, syms_eval_op_params(SYMS_EvalTypeGroup_U));
          }
        }break;
        
        case SYMS_DwOp_BREG0:  case SYMS_DwOp_BREG1:  case SYMS_DwOp_BREG2:
        case SYMS_DwOp_BREG3:  case SYMS_DwOp_BREG4:  case SYMS_DwOp_BREG5:
        case SYMS_DwOp_BREG6:  case SYMS_DwOp_BREG7:  case SYMS_DwOp_BREG8:
        case SYMS_DwOp_BREG9:  case SYMS_DwOp_BREG10: case SYMS_DwOp_BREG11:
        case SYMS_DwOp_BREG12: case SYMS_DwOp_BREG13: case SYMS_DwOp_BREG14:
        case SYMS_DwOp_BREG15: case SYMS_DwOp_BREG16: case SYMS_DwOp_BREG17:
        case SYMS_DwOp_BREG18: case SYMS_DwOp_BREG19: case SYMS_DwOp_BREG20:
        case SYMS_DwOp_BREG21: case SYMS_DwOp_BREG22: case SYMS_DwOp_BREG23:
        case SYMS_DwOp_BREG24: case SYMS_DwOp_BREG25: case SYMS_DwOp_BREG26:
        case SYMS_DwOp_BREG27: case SYMS_DwOp_BREG28: case SYMS_DwOp_BREG29:
        case SYMS_DwOp_BREG30: case SYMS_DwOp_BREG31:
        {
          SYMS_S64 offset = 0;
          step_cursor += syms_based_range_read_sleb128(expr_base, expr_range, step_cursor, &offset);
          
          SYMS_U64 reg_idx = op - SYMS_DwOp_REG0;
          
          SYMS_RegID reg_id = syms_reg_id_from_dw_reg(arch, reg_idx);
          // TODO(allen): change size based on arch
          SYMS_U8 size = 8;
          
          new_node = syms_dw_expr__ir_push_node(arena, &graph, SYMS_EvalOp_RegRead, syms_eval_op_params_2u8(reg_id, size));
          syms_dw_expr__ir_encode_u(arena, &graph, offset);
          syms_dw_expr__ir_push_node(arena, &graph, SYMS_EvalOp_Add, syms_eval_op_params(SYMS_EvalTypeGroup_U));
        }break;
        
        case SYMS_DwOp_BREGX:
        {
          SYMS_U64 reg_idx = 0;
          step_cursor += syms_based_range_read_uleb128(expr_base, expr_range, step_cursor, &reg_idx);
          SYMS_S64 offset = 0;
          step_cursor += syms_based_range_read_sleb128(expr_base, expr_range, step_cursor, &offset);
          
          SYMS_RegID reg_id = syms_reg_id_from_dw_reg(arch, reg_idx);
          // TODO(allen): change size based on arch
          SYMS_U8 size = 8;
          
          new_node = syms_dw_expr__ir_push_node(arena, &graph, SYMS_EvalOp_RegRead, syms_eval_op_params_2u8(reg_id, size));
          
          syms_dw_expr__ir_encode_u(arena, &graph, offset);
          syms_dw_expr__ir_push_node(arena, &graph, SYMS_EvalOp_Add, syms_eval_op_params(SYMS_EvalTypeGroup_U));
        }break;
        
        //// stack operations ////
        
        case SYMS_DwOp_DUP:
        {
          new_node = syms_dw_expr__ir_push_node(arena, &graph, SYMS_EvalOp_Pick, syms_eval_op_params(0));
        }break;
        
        case SYMS_DwOp_DROP:
        {
          new_node = syms_dw_expr__ir_push_node(arena, &graph, SYMS_EvalOp_Pop, syms_eval_op_params(0));
        }break;
        
        case SYMS_DwOp_PICK:
        {
          SYMS_U64 idx = 0;
          step_cursor += syms_based_range_read(expr_base, expr_range, step_cursor, 1, &idx);
          // TODO: if (idx >= 256) no-convert
          new_node = syms_dw_expr__ir_push_node(arena, &graph, SYMS_EvalOp_Pick, syms_eval_op_params(idx));
        }break;
        
        case SYMS_DwOp_OVER:
        {
          new_node = syms_dw_expr__ir_push_node(arena, &graph, SYMS_EvalOp_Pick, syms_eval_op_params(1));
        }break;
        
        case SYMS_DwOp_SWAP:
        {
          new_node = syms_dw_expr__ir_push_node(arena, &graph, SYMS_EvalOp_Insert, syms_eval_op_params(1));
        }break;
        
        case SYMS_DwOp_ROT:
        {
          new_node = syms_dw_expr__ir_push_node(arena, &graph, SYMS_EvalOp_Insert, syms_eval_op_params(2));
        }break;
        
        case SYMS_DwOp_DEREF:
        {
          new_node = syms_dw_expr__ir_push_node(arena, &graph, SYMS_EvalOp_MemRead, syms_eval_op_params(8));
        }break;
        
        case SYMS_DwOp_DEREF_SIZE:
        {
          SYMS_U64 raw_size = 0;
          step_cursor += syms_based_range_read(expr_base, expr_range, step_cursor, 1, &raw_size);
          SYMS_U64 size = SYMS_ClampTop(raw_size, 8);
          new_node = syms_dw_expr__ir_push_node(arena, &graph, SYMS_EvalOp_MemRead, syms_eval_op_params(size));
        }break;
        
        case SYMS_DwOp_XDEREF:
        case SYMS_DwOp_XDEREF_SIZE:
        {
          // NOT SUPPORTED
        }break;
        
        case SYMS_DwOp_PUSH_OBJECT_ADDRESS:
        {
          new_node = syms_dw_expr__ir_push_node(arena, &graph, SYMS_EvalOp_ObjectOff, syms_eval_op_params(0));
        }break;
        
        case SYMS_DwOp_FORM_TLS_ADDRESS:
        {
          // TODO(allen): 
        }break;
        
        case SYMS_DwOp_CALL_FRAME_CFA:
        {
          new_node = syms_dw_expr__ir_push_node(arena, &graph, SYMS_EvalOp_CFA, syms_eval_op_params(0));
        }break;
        
        //// arithmetic and logical operations ////
        
        case SYMS_DwOp_ABS:
        {
          new_node = syms_dw_expr__ir_push_node(arena, &graph, SYMS_EvalOp_Abs,
                                                syms_eval_op_params(SYMS_EvalTypeGroup_S));
        }break;
        
        case SYMS_DwOp_AND:
        {
          new_node = syms_dw_expr__ir_push_node(arena, &graph, SYMS_EvalOp_BitAnd,
                                                syms_eval_op_params(SYMS_EvalTypeGroup_U));
        }break;
        
        case SYMS_DwOp_DIV:
        {
          new_node = syms_dw_expr__ir_push_node(arena, &graph, SYMS_EvalOp_Div,
                                                syms_eval_op_params(SYMS_EvalTypeGroup_S));
        }break;
        
        case SYMS_DwOp_MINUS:
        {
          new_node = syms_dw_expr__ir_push_node(arena, &graph, SYMS_EvalOp_Sub,
                                                syms_eval_op_params(SYMS_EvalTypeGroup_U));
        }break;
        
        case SYMS_DwOp_MOD:
        {
          new_node = syms_dw_expr__ir_push_node(arena, &graph, SYMS_EvalOp_Mod,
                                                syms_eval_op_params(SYMS_EvalTypeGroup_U));
        }break;
        
        case SYMS_DwOp_MUL:
        {
          new_node = syms_dw_expr__ir_push_node(arena, &graph, SYMS_EvalOp_Mul,
                                                syms_eval_op_params(SYMS_EvalTypeGroup_U));
        }break;
        
        case SYMS_DwOp_NEG:
        {
          new_node = syms_dw_expr__ir_push_node(arena, &graph, SYMS_EvalOp_Neg,
                                                syms_eval_op_params(SYMS_EvalTypeGroup_S));
        }break;
        
        case SYMS_DwOp_NOT:
        {
          new_node = syms_dw_expr__ir_push_node(arena, &graph, SYMS_EvalOp_BitNot,
                                                syms_eval_op_params(SYMS_EvalTypeGroup_U));
        }break;
        
        case SYMS_DwOp_OR:
        {
          new_node = syms_dw_expr__ir_push_node(arena, &graph, SYMS_EvalOp_BitOr,
                                                syms_eval_op_params(SYMS_EvalTypeGroup_U));
        }break;
        
        case SYMS_DwOp_PLUS:
        {
          syms_dw_expr__ir_push_node(arena, &graph, SYMS_EvalOp_Add, syms_eval_op_params(SYMS_EvalTypeGroup_U));
        }break;
        
        case SYMS_DwOp_PLUS_UCONST:
        {
          SYMS_U64 y = 0;
          step_cursor += syms_based_range_read_uleb128(expr_base, expr_range, step_cursor, &y);
          new_node = syms_dw_expr__ir_encode_u(arena, &graph, y);
          syms_dw_expr__ir_push_node(arena, &graph, SYMS_EvalOp_Add, syms_eval_op_params(SYMS_EvalTypeGroup_U));
        }break;
        
        case SYMS_DwOp_SHL:
        {
          new_node = syms_dw_expr__ir_push_node(arena, &graph, SYMS_EvalOp_LShift,
                                                syms_eval_op_params(SYMS_EvalTypeGroup_U));
        }break;
        
        case SYMS_DwOp_SHR:
        {
          new_node = syms_dw_expr__ir_push_node(arena, &graph, SYMS_EvalOp_RShift,
                                                syms_eval_op_params(SYMS_EvalTypeGroup_U));
        }break;
        
        case SYMS_DwOp_SHRA:
        {
          new_node = syms_dw_expr__ir_push_node(arena, &graph, SYMS_EvalOp_RShift,
                                                syms_eval_op_params(SYMS_EvalTypeGroup_S));
        }break;
        
        case SYMS_DwOp_XOR:
        {
          new_node = syms_dw_expr__ir_push_node(arena, &graph, SYMS_EvalOp_BitXor,
                                                syms_eval_op_params(SYMS_EvalTypeGroup_U));
        }break;
        
        //// control flow operations ////
        
        case SYMS_DwOp_LE:
        {
          new_node = syms_dw_expr__ir_push_node(arena, &graph, SYMS_EvalOp_LsEq,
                                                syms_eval_op_params(SYMS_EvalTypeGroup_S));
        }break;
        
        case SYMS_DwOp_GE:
        {
          new_node = syms_dw_expr__ir_push_node(arena, &graph, SYMS_EvalOp_GrEq,
                                                syms_eval_op_params(SYMS_EvalTypeGroup_S));
        }break;
        
        case SYMS_DwOp_EQ:
        {
          new_node = syms_dw_expr__ir_push_node(arena, &graph, SYMS_EvalOp_EqEq,
                                                syms_eval_op_params(SYMS_EvalTypeGroup_S));
        }break;
        
        case SYMS_DwOp_LT:
        {
          new_node = syms_dw_expr__ir_push_node(arena, &graph, SYMS_EvalOp_Less,
                                                syms_eval_op_params(SYMS_EvalTypeGroup_S));
        }break;
        
        case SYMS_DwOp_GT:
        {
          new_node = syms_dw_expr__ir_push_node(arena, &graph, SYMS_EvalOp_Grtr,
                                                syms_eval_op_params(SYMS_EvalTypeGroup_S));
        }break;
        
        case SYMS_DwOp_NE:
        {
          new_node = syms_dw_expr__ir_push_node(arena, &graph, SYMS_EvalOp_NtEq,
                                                syms_eval_op_params(SYMS_EvalTypeGroup_S));
        }break;
        
        case SYMS_DwOp_SKIP:
        {
          SYMS_S16 d = 0;
          step_cursor += syms_based_range_read(expr_base, expr_range, step_cursor, 2, &d);
          new_node = syms_dw_expr__ir_push_node(arena, &graph, SYMS_EvalOp_Skip, syms_eval_op_params(0));
          
          // save a jump patch
          SYMS_U64 dst_off = step_cursor + d;
          
          SYMS_DwEvalJumpPatch *new_patch = syms_push_array_zero(scratch.arena, SYMS_DwEvalJumpPatch, 1);
          SYMS_StackPush(jump_patches, new_patch);
          new_patch->node = new_node;
          new_patch->cnext_dw_off = dst_off;
        }break;
        
        case SYMS_DwOp_BRA:
        {
          SYMS_S16 d = 0;
          step_cursor += syms_based_range_read(expr_base, expr_range, step_cursor, 2, &d);
          new_node = syms_dw_expr__ir_push_node(arena, &graph, SYMS_EvalOp_Cond, syms_eval_op_params(0));
          
          // save a jump patch
          SYMS_U64 dst_off = step_cursor + d;
          
          SYMS_DwEvalJumpPatch *new_patch = syms_push_array_zero(scratch.arena, SYMS_DwEvalJumpPatch, 1);
          SYMS_StackPush(jump_patches, new_patch);
          new_patch->node = new_node;
          new_patch->cnext_dw_off = dst_off;
        }break;
        
        case SYMS_DwOp_CALL2:
        case SYMS_DwOp_CALL4:
        case SYMS_DwOp_CALL_REF:
        {
          // TODO: NOT SUPPORTED
        }break;
        
        //// special operations ////
        
        case SYMS_DwOp_NOP:
        {
          new_node = syms_dw_expr__ir_push_node(arena, &graph, (SYMS_EvalOp)SYMS_EvalIRExtKind_Noop,
                                                syms_eval_op_params(0));
        }break;
        
        //// register location descriptions ////
        
        case SYMS_DwOp_REG0:  case SYMS_DwOp_REG1:  case SYMS_DwOp_REG2:
        case SYMS_DwOp_REG3:  case SYMS_DwOp_REG4:  case SYMS_DwOp_REG5:
        case SYMS_DwOp_REG6:  case SYMS_DwOp_REG7:  case SYMS_DwOp_REG8:
        case SYMS_DwOp_REG9:  case SYMS_DwOp_REG10: case SYMS_DwOp_REG11:
        case SYMS_DwOp_REG12: case SYMS_DwOp_REG13: case SYMS_DwOp_REG14:
        case SYMS_DwOp_REG15: case SYMS_DwOp_REG16: case SYMS_DwOp_REG17:
        case SYMS_DwOp_REG18: case SYMS_DwOp_REG19: case SYMS_DwOp_REG20:
        case SYMS_DwOp_REG21: case SYMS_DwOp_REG22: case SYMS_DwOp_REG23:
        case SYMS_DwOp_REG24: case SYMS_DwOp_REG25: case SYMS_DwOp_REG26:
        case SYMS_DwOp_REG27: case SYMS_DwOp_REG28: case SYMS_DwOp_REG29:
        case SYMS_DwOp_REG30: case SYMS_DwOp_REG31:
        {
          SYMS_U64 reg_idx = op - SYMS_DwOp_REG0;
          SYMS_U64 off = syms_reg_off_from_dw_reg(arch, reg_idx);
          // TODO(allen): size from architecture
          SYMS_U8 size = 8;
          SYMS_U64 x = off | (size << 8);
          new_node = syms_dw_expr__ir_encode_u(arena, &graph, x);
          // TODO(allen): register mode
        }break;
        
        case SYMS_DwOp_REGX:
        {
          SYMS_U64 reg_idx = 0;
          step_cursor += syms_based_range_read(expr_base, expr_range, step_cursor, size_param, &reg_idx);
          SYMS_U64 off = syms_reg_off_from_dw_reg(arch, reg_idx);
          // TODO(allen): size from architecture
          SYMS_U8 size = 8;
          SYMS_U64 x = off | (size << 8);
          new_node = syms_dw_expr__ir_encode_u(arena, &graph, x);
          // TODO(allen): register mode
        }break;
        
        //// implicit location descriptions ////
        
        case SYMS_DwOp_IMPLICIT_VALUE:
        {
          SYMS_U64 size = 0;
          step_cursor += syms_based_range_read(expr_base, expr_range, step_cursor, size_param, &size);
          // TODO(allen): size > 8 ?
          
          SYMS_U64 x = 0;
          syms_memmove(&x, (SYMS_U8*)expr_base + expr_range.min + step_cursor, size);
          step_cursor += size;
          
          new_node = syms_dw_expr__ir_encode_u(arena, &graph, x);
          // TODO(allen): value mode
        }break;
        
        case SYMS_DwOp_STACK_VALUE:
        {
          // TODO(allen): value mode
        }break;
        
        //// composite location descriptions ////
        
        case SYMS_DwOp_PIECE:
        case SYMS_DwOp_BIT_PIECE:
        {
          // TODO(allen): ???? how do we even handle this?
        }break;
        
        //// final fallback ////
        
        default:
        {
          // NOT SUPPORTED
        }break;
      }
      
      // save the new node
      off_tbl[cursor] = new_node;
      
      // increment cursor
      cursor = step_cursor;
    }
    
    if (cursor >= off_count){
      break;
    }
  }
  
  //- step 2: resolve jump patches
  for (SYMS_DwEvalJumpPatch *patch = jump_patches;
       patch != 0;
       patch = patch->next){
    SYMS_DwEvalIRGraphNode *dst_node = 0;
    
    SYMS_U64 cnext_dw_off = patch->cnext_dw_off;
    if (cnext_dw_off < off_count){
      dst_node = off_tbl[cnext_dw_off];
    }
    
    if (dst_node != 0){
      patch->node->cnext = dst_node;
    }
    else{
      // TODO(allen): cannot transpile
    }
  }
  
  syms_release_scratch(scratch);
  
  // finish filling in result graph
  graph.tbl = off_tbl;
  graph.count = off_count;
  
  return(graph);
}

SYMS_API void
syms_dw_expr__ir_graph_stack_push(SYMS_Arena *arena, SYMS_DwEvalIRGraphStack *stack,
                                  SYMS_DwEvalIRGraphNode *node){
  SYMS_DwEvalIRGraphStackNode *stack_node = stack->free;
  if (stack_node == 0){
    stack_node = syms_push_array(arena, SYMS_DwEvalIRGraphStackNode, 1);
  }
  else{
    SYMS_StackPop(stack->free);
  }
  SYMS_StackPush(stack->stack, stack_node);
  stack_node->node = node;
  stack_node->stage = 0;
}

SYMS_API void
syms_dw_expr__ir_graph_stack_pop(SYMS_DwEvalIRGraphStack *stack){
  SYMS_DwEvalIRGraphStackNode *stack_node = stack->stack;
  if (stack_node != 0){
    SYMS_StackPop(stack->stack);
    SYMS_StackPush(stack->free, stack_node);
  }
}

SYMS_API SYMS_B32
syms_dw_expr__ir_contains_cycle(SYMS_DwEvalIRGraphNode *entry_node){
  SYMS_B32 result = syms_false;
  
  SYMS_ArenaTemp scratch = syms_get_scratch(0,0);
  SYMS_DwEvalIRGraphStack stack = {0};
  
  entry_node->in_stack = syms_true;
  syms_dw_expr__ir_graph_stack_push(scratch.arena, &stack, entry_node);
  
  for (;;){
    SYMS_DwEvalIRGraphStackNode *stack_node = stack.stack;
    if (stack_node == 0){
      break;
    }
    
    SYMS_DwEvalIRGraphNode *node = stack_node->node;
    
    switch (stack_node->stage){
      case 0:
      {
        if (node->op != SYMS_EvalOp_Skip){
          SYMS_DwEvalIRGraphNode *next_node = node->next;
          if (next_node != 0){
            if (next_node->in_stack){
              result = syms_true;
              goto done;
            }
            else{
              next_node->in_stack = syms_true;
              syms_dw_expr__ir_graph_stack_push(scratch.arena, &stack, next_node);
            }
          }
        }
      }break;
      
      case 1:
      {
        SYMS_DwEvalIRGraphNode *next_node = node->cnext;
        if (next_node != 0){
          if (next_node->in_stack){
            result = syms_true;
            goto done;
          }
          else{
            next_node->in_stack = syms_true;
            syms_dw_expr__ir_graph_stack_push(scratch.arena, &stack, next_node);
          }
        }
      }break;
      
      case 2:
      {
        node->in_stack = syms_false;
        syms_dw_expr__ir_graph_stack_pop(&stack);
      }break;
    }
    
    stack_node->stage += 1;
  }
  
  done:;
  
  syms_release_scratch(scratch);
  
  return(result);
}

SYMS_API SYMS_DwEvalIRGraphBlock*
syms_dw_expr__ir_blocks_from_graph(SYMS_Arena *arena, SYMS_DwEvalIRGraph graph){
  SYMS_DwEvalIRGraphBlock *result = 0;
  
  
  
  return(result);
}

SYMS_API SYMS_String8
syms_dw_expr__transpile_to_eval(SYMS_Arena *arena, SYMS_DwDbgAccel *dbg,
                                void *expr_base, SYMS_U64Range expr_range){
  SYMS_ArenaTemp scratch = syms_get_scratch(&arena, 1);
  
  SYMS_DwEvalIRGraph graph = syms_dw_expr__ir_graph_from_dw_expr(scratch.arena, dbg, expr_base, expr_range);
  
  SYMS_B32 contains_cycle = syms_dw_expr__ir_contains_cycle(graph.first);
  
  if (!contains_cycle){
    SYMS_DwEvalIRGraphBlock *block = syms_dw_expr__ir_blocks_from_graph(arena, graph);
    
  }
  
  syms_release_scratch(scratch);
  
  SYMS_String8 result = {0};
  return(result);
}
