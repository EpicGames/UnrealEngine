// Copyright Epic Games, Inc. All Rights Reserved.
/* date = March 21st 2022 11:41 am */

#ifndef SYMS_DWARF_TRANSPILER_H
#define SYMS_DWARF_TRANSPILER_H

// NOTE(allen): The transpiler is organized separately from the rest of dwarf expression
// decoding and analysis (in syms_dwarf_expr) because this part requires explicit access to
// more of the full debug info, and we want the rest of the expression analysis stuff to work
// even without debug info.

////////////////////////////////
//~ NOTE(allen): Dwarf Expression to Eval Transpiler Types

typedef struct SYMS_DwEvalIRGraphNode{
  // contents
  struct SYMS_DwEvalIRGraphNode *next;
  struct SYMS_DwEvalIRGraphNode *cnext;
  SYMS_U32 op;
  SYMS_EvalOpParams params;
  
  // analysis
  SYMS_B8 in_stack;
} SYMS_DwEvalIRGraphNode;

typedef struct SYMS_DwEvalIRGraph{
  SYMS_DwEvalIRGraphNode *first;
  SYMS_DwEvalIRGraphNode *last;
  
  SYMS_DwEvalIRGraphNode **tbl;
  SYMS_U64 count;
} SYMS_DwEvalIRGraph;

typedef struct SYMS_DwEvalJumpPatch{
  struct SYMS_DwEvalJumpPatch *next;
  SYMS_DwEvalIRGraphNode *node;
  SYMS_U64 cnext_dw_off;
} SYMS_DwEvalJumpPatch;

typedef struct SYMS_DwEvalIRGraphStackNode{
  struct SYMS_DwEvalIRGraphStackNode *next;
  SYMS_DwEvalIRGraphNode *node;
  SYMS_U64 stage;
} SYMS_DwEvalIRGraphStackNode;

typedef struct SYMS_DwEvalIRGraphStack{
  SYMS_DwEvalIRGraphStackNode *stack;
  SYMS_DwEvalIRGraphStackNode *free;
} SYMS_DwEvalIRGraphStack;

typedef struct SYMS_DwEvalIRGraphBlock{
  // content
  struct SYMS_DwEvalIRGraphBlock *next;
  struct SYMS_DwEvalIRGraphBlock *cnext;
  
  SYMS_DwEvalIRGraphNode *first;
  SYMS_DwEvalIRGraphNode *last;
  
  // analysis
  struct SYMS_DwEvalIRGraphBlock *order;
} SYMS_DwEvalIRGraphBlock;

////////////////////////////////
//~ NOTE(allen): Dwarf Transpiler Functions

SYMS_API SYMS_RegID  syms_reg_id_from_dw_reg(SYMS_Arch arch, SYMS_U64 reg_idx);
SYMS_API SYMS_U64    syms_reg_off_from_dw_reg(SYMS_Arch arch, SYMS_U64 reg_idx);

SYMS_API SYMS_DwEvalIRGraphNode* syms_dw_expr__ir_push_node(SYMS_Arena *arena, SYMS_DwEvalIRGraph *graph,
                                                            SYMS_EvalOp op, SYMS_EvalOpParams params);

SYMS_API SYMS_DwEvalIRGraphNode* syms_dw_expr__ir_encode_u(SYMS_Arena *arena, SYMS_DwEvalIRGraph *graph,
                                                           SYMS_U64 u);
SYMS_API SYMS_DwEvalIRGraphNode* syms_dw_expr__ir_encode_s(SYMS_Arena *arena, SYMS_DwEvalIRGraph *graph,
                                                           SYMS_S64 s);

SYMS_API SYMS_DwEvalIRGraph syms_dw_expr__ir_graph_from_dw_expr(SYMS_Arena *arena, SYMS_DwDbgAccel *dbg,
                                                                void *base, SYMS_U64Range range);

SYMS_API void syms_dw_expr__ir_graph_stack_push(SYMS_Arena *arena, SYMS_DwEvalIRGraphStack *stack,
                                                SYMS_DwEvalIRGraphNode *node);
SYMS_API void syms_dw_expr__ir_graph_stack_pop(SYMS_DwEvalIRGraphStack *stack);
SYMS_API SYMS_B32 syms_dw_expr__ir_contains_cycle(SYMS_DwEvalIRGraphNode *node);

SYMS_API SYMS_DwEvalIRGraphBlock* syms_dw_expr__ir_blocks_from_graph(SYMS_Arena *arena,
                                                                     SYMS_DwEvalIRGraph graph);

SYMS_API SYMS_String8 syms_dw_expr__transpile_to_eval(SYMS_Arena *arena, SYMS_DwDbgAccel *dbg,
                                                      void *base, SYMS_U64Range range);

#endif //SYMS_DWARF_TRANSPILER_H
