// Copyright Epic Games, Inc. All Rights Reserved.
/* date = May 11th 2021 4:54 pm */

#ifndef SYMS_ELF_H
#define SYMS_ELF_H

////////////////////////////////
//~ NOTE(nick): Generated

#include "syms/core/generated/syms_meta_elf.h"

////////////////////////////////
//~ rjf: Elf File Header Types

typedef enum SYMS_ElfIdentifier{
  SYMS_ElfIdentifier_MAG0       = 0,
  SYMS_ElfIdentifier_MAG1       = 1,
  SYMS_ElfIdentifier_MAG2       = 2,
  SYMS_ElfIdentifier_MAG3       = 3,
  SYMS_ElfIdentifier_CLASS      = 4,
  SYMS_ElfIdentifier_DATA       = 5,
  SYMS_ElfIdentifier_VERSION    = 6,
  SYMS_ElfIdentifier_OSABI      = 7,
  SYMS_ElfIdentfiier_ABIVERSION = 8,
  SYMS_ElfIdentifier_NIDENT     = 16,
}SYMS_ElfIdentifier;

typedef struct SYMS_ElfEhdr64 SYMS_ElfEhdr64;
struct SYMS_ElfEhdr64
{
  SYMS_U8 e_ident[SYMS_ElfIdentifier_NIDENT];
  SYMS_U16 e_type;
  SYMS_U16 e_machine;
  SYMS_U32 e_version;
  SYMS_U64 e_entry;
  SYMS_U64 e_phoff;
  SYMS_U64 e_shoff;
  SYMS_U32 e_flags;
  SYMS_U16 e_ehsize;
  SYMS_U16 e_phentsize;
  SYMS_U16 e_phnum;
  SYMS_U16 e_shentsize;
  SYMS_U16 e_shnum;
  SYMS_U16 e_shstrndx;
};

typedef struct SYMS_ElfEhdr32 SYMS_ElfEhdr32;
struct SYMS_ElfEhdr32
{
  SYMS_U8 e_ident[SYMS_ElfIdentifier_NIDENT];
  SYMS_U16 e_type;
  SYMS_U16 e_machine;
  SYMS_U32 e_version;
  SYMS_U32 e_entry;
  SYMS_U32 e_phoff;
  SYMS_U32 e_shoff;
  SYMS_U32 e_flags;
  SYMS_U16 e_ehsize;
  SYMS_U16 e_phentsize;
  SYMS_U16 e_phnum;
  SYMS_U16 e_shentsize;
  SYMS_U16 e_shnum;
  SYMS_U16 e_shstrndx;
};

typedef struct SYMS_ElfShdr64 SYMS_ElfShdr64;
struct SYMS_ElfShdr64
{
  SYMS_U32 sh_name;
  SYMS_U32 sh_type;
  SYMS_U64 sh_flags;
  SYMS_U64 sh_addr;
  SYMS_U64 sh_offset;
  SYMS_U64 sh_size;
  SYMS_U32 sh_link;
  SYMS_U32 sh_info;
  SYMS_U64 sh_addralign;
  SYMS_U64 sh_entsize;
};

typedef struct SYMS_ElfShdr32 SYMS_ElfShdr32;
struct SYMS_ElfShdr32
{
  SYMS_U32 sh_name;
  SYMS_U32 sh_type;
  SYMS_U32 sh_flags;
  SYMS_U32 sh_addr;
  SYMS_U32 sh_offset;
  SYMS_U32 sh_size;
  SYMS_U32 sh_link;
  SYMS_U32 sh_info;
  SYMS_U32 sh_addralign;
  SYMS_U32 sh_entsize;
};

typedef struct SYMS_ElfPhdr64 SYMS_ElfPhdr64;
struct SYMS_ElfPhdr64
{
  SYMS_U32 p_type;
  SYMS_U32 p_flags;
  SYMS_U64 p_offset;
  SYMS_U64 p_vaddr;
  SYMS_U64 p_paddr;
  SYMS_U64 p_filesz;
  SYMS_U64 p_memsz;
  SYMS_U64 p_align;
};

typedef struct SYMS_ElfPhdr32 SYMS_ElfPhdr32;
struct SYMS_ElfPhdr32
{
  SYMS_U32 p_type;
  SYMS_U32 p_offset;
  SYMS_U32 p_vaddr;
  SYMS_U32 p_paddr;
  SYMS_U32 p_filesz;
  SYMS_U32 p_memsz;
  SYMS_U32 p_flags;
  SYMS_U32 p_align;
};

////////////////////////////////
//~ NOTE(allen): Auxiliary Vectors

// these appear in /proc/<pid>/auxv of a process, they are not in elf files

typedef struct SYMS_ElfAuxv32{
  SYMS_U32 a_type;
  SYMS_U32 a_val;
} SYMS_ElfAuxv32;

typedef struct SYMS_ElfAuxv64{
  SYMS_U64 a_type;
  SYMS_U64 a_val;
} SYMS_ElfAuxv64;

////////////////////////////////
//~ NOTE(allen): Dynamic Structures

// these appear in the virtual address space of a process, they are not in elf files

typedef struct SYMS_ElfDyn32{
  SYMS_U32 tag;
  SYMS_U32 val;
} SYMS_ElfDyn32;

typedef struct SYMS_ElfDyn64{
  SYMS_U64 tag;
  SYMS_U64 val;
} SYMS_ElfDyn64;

typedef struct SYMS_ElfLinkMap32{
  SYMS_U32 base;
  SYMS_U32 name;
  SYMS_U32 ld;
  SYMS_U32 next;
} SYMS_ElfLinkMap32;

typedef struct SYMS_ElfLinkMap64{
  SYMS_U64 base;
  SYMS_U64 name;
  SYMS_U64 ld;
  SYMS_U64 next;
} SYMS_ElfLinkMap64;

////////////////////////////////
// Imports and Exports

typedef struct 
{
  SYMS_U32 st_name;  // Holds index into files string table.
  SYMS_U32 st_value; // Depending on the context, this may be address, size, etc.
  SYMS_U32 st_size;  // Data size in bytes. Zero when size is unknown.
  SYMS_U8  st_info;  // Contains symbols type and binding.
  SYMS_U8  st_other; // Reserved for future use, currenly zero.
  SYMS_U16 st_shndx; // Section index to which symbol is relevant.
} SYMS_ElfSym32;

typedef struct 
{
  SYMS_U32 st_name;
  SYMS_U8  st_info;
  SYMS_U8  st_other;
  SYMS_U16 st_shndx;
  SYMS_U64 st_value;
  SYMS_U64 st_size;
} SYMS_ElfSym64;

#define SYMS_ELF_ST_INFO(b,t)     (((b) << 4) + ((t) & 0xF))
#define SYMS_ELF_ST_BIND(x)       ((x) >> 4)
#define SYMS_ELF_ST_TYPE(x)       ((x) & 0xF)
#define SYMS_ELF_ST_VISIBILITY(v) ((v) & 0x3)

typedef struct
{
  SYMS_U32 r_offset;
  SYMS_U32 r_info;
} SYMS_ElfRel32;

typedef struct
{
  SYMS_U32 r_offset;
  SYMS_U32 r_info;
  SYMS_S32 r_addend;
} SYMS_ElfRela32;

typedef struct
{
  SYMS_U64 r_offset;
  SYMS_U64 r_info;
} SYMS_ElfRel64;

typedef struct
{
  SYMS_U64 r_offset;
  SYMS_U64 r_info;
  SYMS_S64 r_addend;
} SYMS_ElfRela64;

#define SYMS_ELF32_R_SYM(x)  ((x) >> 8)
#define SYMS_ELF32_R_TYPE(x) ((x) & 0xFF)

#define SYMS_ELF64_R_INFO(s,t) (((SYMS_U64)(s) << 32) | (SYMS_U64)t)
#define SYMS_ELF64_R_SYM(x)  ((x) >> 32)
#define SYMS_ELF64_R_TYPE(x) ((x) & 0xffffffff)

// This flag is set to indicate that symbol is not available outside the shared object
#define SYMS_ELF_EXTERNAL_VERSYM_HIDDEN 0x8000
#define SYMS_ELF_EXTERNAL_VERSYM_MASK   0x7FFF

// Appears in .gnu.verdef (SHT_GNU_verdef)
typedef struct
{
  SYMS_U16 vd_version;
  SYMS_U16 vd_flags;
  SYMS_U16 vd_ndx;
  SYMS_U16 vd_cnt;
  SYMS_U32 vd_hash;
  SYMS_U32 vd_aux;
  SYMS_U32 vd_next;
} SYMS_ElfExternalVerdef;

// Appears in .gnu.verdef (SHT_GNU_verdef)
typedef struct
{
  SYMS_U32 vda_name;
  SYMS_U32 vda_next;
} SYMS_ElfExternalVerdaux;

// Appears in .gnu.verneed (SHT_GNU_verneed)
typedef struct
{
  SYMS_U16 vn_version;
  SYMS_U16 vn_cnt;
  SYMS_U32 vn_file;
  SYMS_U32 vn_aux;
  SYMS_U32 vn_next;
} SYMS_ElfExternalVerneed;

// Appears in .gnu.verneed (SHT_GNU_verneed)
typedef struct
{
  SYMS_U32 vna_hash;
  SYMS_U16 vna_flags;
  SYMS_U16 vna_other;
  SYMS_U32 vna_name;
  SYMS_U32 vna_next;
} SYMS_ElfExternalVernaux;

// Appears in .gnu.version (SHT_GNU_versym)
typedef struct
{
  SYMS_U16 vs_vers;
} SYMS_ElfExternalVersym;

typedef struct
{
  SYMS_U32 name_size;
  SYMS_U32 desc_size;
  SYMS_U32 type;
  // name + desc
  // SYMS_U8  data[1];
} SYMS_ElfNote;

////////////////////////////////
// Extensions

typedef SYMS_U8 SYMS_ElfCompressType;
enum SYMS_ElfCompressTypeEnum
{
  SYMS_ElfCompressType_None = 0,
  SYMS_ElfCompressType_ZLib = 1,
  SYMS_ElfCompressType_ZStd = 2,
  
  SYMS_ElfCompressType_LoOs = 0x60000000,
  SYMS_ElfCompressType_HiOs = 0x6fffffff,
  
  SYMS_ElfCompressType_LoProc = 0x70000000,
  SYMS_ElfCompressType_HiProc = 0x7fffffff,
};

typedef struct SYMS_ElfChdr32
{
  SYMS_U32 ch_type;
  SYMS_U32 ch_size;
  SYMS_U32 ch_addr_align;
} SYMS_ElfChdr32;

typedef struct SYMS_ElfChdr64
{
  SYMS_U32 ch_type;
  SYMS_U32 ch_reserved;
  SYMS_U64 ch_size;
  SYMS_U64 ch_addr_align;
} SYMS_ElfChdr64;

SYMS_C_LINKAGE_BEGIN

////////////////////////////////
//~ rjf: 32 => 64 bit conversions

// TODO(allen): avoid extra copies (use pointers where convenient)
SYMS_API SYMS_ElfEhdr64 syms_elf_ehdr64_from_ehdr32(SYMS_ElfEhdr32 h32);
SYMS_API SYMS_ElfShdr64 syms_elf_shdr64_from_shdr32(SYMS_ElfShdr32 h32);
SYMS_API SYMS_ElfPhdr64 syms_elf_phdr64_from_phdr32(SYMS_ElfPhdr32 h32);
SYMS_API SYMS_ElfDyn64  syms_elf_dyn64_from_dyn32(SYMS_ElfDyn32 h32);
SYMS_API SYMS_ElfSym64  syms_elf_sym64_from_sym32(SYMS_ElfSym32 sym32);
SYMS_API SYMS_ElfRel64  syms_elf_rel64_from_rel32(SYMS_ElfRel32 rel32);
SYMS_API SYMS_ElfRela64 syms_elf_rela64_from_rela32(SYMS_ElfRela32 rela32);
// TODO(allen): auxv?
// TODO(allen): linkmap?

////////////////////////////////
//~ rjf: .gnu_debuglink section 32-bit CRC

SYMS_API SYMS_U32 syms_elf_gnu_debuglink_crc32(SYMS_U32 crc, SYMS_String8 data);

SYMS_C_LINKAGE_END

#endif // SYMS_ELF_H
