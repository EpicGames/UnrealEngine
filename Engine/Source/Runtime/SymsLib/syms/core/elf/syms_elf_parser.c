// Copyright Epic Games, Inc. All Rights Reserved.

#ifndef SYMS_ELF_PARSER_C
#define SYMS_ELF_PARSER_C

#include <zlib.h>

////////////////////////////////
//~ rjf: Low-Level Header/Section Parsing

SYMS_API SYMS_ElfImgHeader
syms_elf_img_header_from_file(SYMS_String8 file)
{
  void *file_base = file.str;
  SYMS_U64Range file_range = syms_make_u64_range(0, file.size);
  
  //- rjf: predetermined read offsets
  SYMS_U64 elf_header_off = 0;
  
  //- rjf: figure out if this file starts with a ELF header
  SYMS_U8 sig[SYMS_ElfIdentifier_NIDENT];
  SYMS_B32 sig_is_elf = syms_false;
  {
    syms_memset(sig, 0, sizeof(sig));
    syms_based_range_read(file_base, file_range, elf_header_off, sizeof(sig), sig);
    sig_is_elf = (sig[SYMS_ElfIdentifier_MAG0] == 0x7f &&
                  sig[SYMS_ElfIdentifier_MAG1] == 'E'  &&
                  sig[SYMS_ElfIdentifier_MAG2] == 'L'  &&
                  sig[SYMS_ElfIdentifier_MAG3] == 'F');
  }
  
  //- rjf: parse ELF header
  SYMS_ElfEhdr64 elf_header = {0};
  SYMS_B32 is_32bit = syms_false;
  SYMS_B32 good_elf_header = syms_false;
  if(sig_is_elf)
  {
    SYMS_U64 bytes_read = 0;
    switch(sig[SYMS_ElfIdentifier_CLASS])
    {
      //- rjf: parse 32-bit header
      case SYMS_ElfClass_32:
      {
        SYMS_ElfEhdr32 elf_header_32 = {0};
        bytes_read = syms_based_range_read_struct(file_base, file_range, elf_header_off, &elf_header_32);
        elf_header = syms_elf_ehdr64_from_ehdr32(elf_header_32);
        good_elf_header = (bytes_read == sizeof(elf_header_32));
        is_32bit = syms_true;
      }break;
      //- rjf: parse 64-bit header
      case SYMS_ElfClass_64:
      {
        bytes_read = syms_based_range_read_struct(file_base, file_range, elf_header_off, &elf_header);
        good_elf_header = (bytes_read == sizeof(elf_header));
      }break;
      default:break;
    }
  }
  
  //- allen: extract entry point
  SYMS_U64 entry_point = elf_header.e_entry;
  
  //- rjf: parse section header
  SYMS_U64 sh_name_low_offset  = SYMS_U64_MAX;
  SYMS_U64 sh_name_high_offset = SYMS_U64_MAX;
  SYMS_B32 good_section_header = syms_false;
  SYMS_ElfShdr64 section_header;
  if(good_elf_header)
  {
    syms_memzero_struct(&section_header);
    SYMS_U64 bytes_read = 0;
    SYMS_U64 shstr_off = elf_header.e_shoff + elf_header.e_shentsize*elf_header.e_shstrndx;
    switch (sig[SYMS_ElfIdentifier_CLASS]) {
      case SYMS_ElfClass_32: {
        SYMS_ElfShdr32 section_header_32;
        syms_memzero_struct(&section_header_32);
        bytes_read = syms_based_range_read_struct(file_base, file_range, shstr_off, &section_header_32);
        section_header = syms_elf_shdr64_from_shdr32(section_header_32);
        good_section_header = bytes_read == sizeof(section_header_32);
      } break;
      case SYMS_ElfClass_64: {
        bytes_read = syms_based_range_read_struct(file_base, file_range, shstr_off, &section_header);
        good_section_header = bytes_read == sizeof(section_header);
      } break;
    }
    
    //- rjf: save base address and high address of string header names
    sh_name_low_offset  = section_header.sh_offset;
    sh_name_high_offset = sh_name_low_offset + section_header.sh_size;
  }
  
  //- rjf: parse program headers
  SYMS_U64 base_address = 0;
  if(good_section_header)
  {
    SYMS_U64 program_header_off = elf_header.e_phoff;
    SYMS_U64 program_header_num = (elf_header.e_phnum != SYMS_U16_MAX) ? elf_header.e_phnum : section_header.sh_info;
    
    //- rjf: search for base address, by grabbing the first LOAD phdr
    for(SYMS_U64 i = 0; i < program_header_num; i += 1)
    {
      SYMS_ElfPhdr64 program_header = {0};
      switch(sig[SYMS_ElfIdentifier_CLASS])
      {
        case SYMS_ElfClass_32:
        {
          SYMS_ElfPhdr32 h32;
          syms_memzero_struct(&h32);
          syms_based_range_read_struct(file_base, file_range, program_header_off + i*sizeof(SYMS_ElfPhdr32), &h32);
          program_header = syms_elf_phdr64_from_phdr32(h32);
        }break;
        case SYMS_ElfClass_64:
        {
          syms_based_range_read_struct(file_base, file_range, program_header_off + i*sizeof(SYMS_ElfPhdr64), &program_header);
        }break;
      }
      if(program_header.p_type == SYMS_ElfPKind_Load)
      {
        base_address = program_header.p_vaddr;
        break;
      }
    }
  }
  
  //- rjf: determine SYMS_Arch from the ELF machine kind
  SYMS_Arch arch = SYMS_Arch_Null;
  switch(elf_header.e_machine)
  {
    default:break;
    case SYMS_ElfMachineKind_AARCH64: arch = SYMS_Arch_ARM;   break;
    case SYMS_ElfMachineKind_ARM:     arch = SYMS_Arch_ARM32; break;
    case SYMS_ElfMachineKind_386:     arch = SYMS_Arch_X86;   break;
    case SYMS_ElfMachineKind_X86_64:  arch = SYMS_Arch_X64;   break;
    case SYMS_ElfMachineKind_PPC:     arch = SYMS_Arch_PPC;   break;
    case SYMS_ElfMachineKind_PPC64:   arch = SYMS_Arch_PPC64; break;
    case SYMS_ElfMachineKind_IA_64:   arch = SYMS_Arch_IA64;  break;
  }
  
  //- rjf: fill img
  SYMS_ElfImgHeader img = {0};
  if(good_elf_header)
  {
    img.valid = 1;
    img.is_32bit = is_32bit;
    img.ehdr = elf_header;
    img.arch = arch;
    img.sh_name_low_offset = sh_name_low_offset;
    img.sh_name_high_offset = sh_name_high_offset;
    img.base_address = base_address;
    img.entry_point = entry_point;
  }
  
  return img;
}

SYMS_API SYMS_ElfSectionArray
syms_elf_section_array_from_img_header(SYMS_Arena *arena, SYMS_String8 file, SYMS_ElfImgHeader img)
{
  SYMS_ElfSectionArray result = {0};
  void *file_base = file.str;
  SYMS_U64Range file_range = syms_make_u64_range(0, file.size);
  
  //- rjf: figure out section count
  // NOTE(rjf): ELF files have a null/empty section at the first slot of the
  // section header table. We're explicitly skipping that, so we need to
  // account for (e_shnum-1) sections.
  SYMS_U64 section_count = img.ehdr.e_shnum ? (img.ehdr.e_shnum-1) : 0;
  
  //- rjf: figure out section range and section header size (32-bit or 64-bit)
  SYMS_U64Range section_range = syms_make_u64_range(SYMS_U64_MAX, SYMS_U64_MAX);
  SYMS_U64 section_header_size = img.ehdr.e_shentsize;
  {
    section_range.min = img.ehdr.e_shoff + 1*section_header_size;
    section_range.max = section_range.min + section_count*section_header_size;
    section_range.max = SYMS_ClampTop(file.size, section_range.max);
  }
  
  //- rjf: allocate sections
  SYMS_ElfSection *sections = syms_push_array_zero(arena, SYMS_ElfSection, section_count);
  
  //- rjf: parse section headers
  for(SYMS_U64 section_idx = 0; section_idx < section_count; section_idx += 1)
  {
    // rjf: parse section header
    SYMS_ElfShdr64 header;
    if(img.is_32bit)
    {
      // NOTE(rjf): In the case of 32-bit ELF files, we need to convert the 32-bit section
      // headers to the 64-bit format, which is what we'll be using everywhere else.
      SYMS_ElfShdr32 header32;
      syms_based_range_read_struct(file_base, section_range, section_idx*sizeof(header32), &header32);
      header = syms_elf_shdr64_from_shdr32(header32);
    }
    else
    {
      syms_based_range_read_struct(file_base, section_range, section_idx*sizeof(header), &header);
    }
    
    // rjf: parse section name
    SYMS_String8 name = syms_based_range_read_string(file_base, file_range, img.sh_name_low_offset + header.sh_name);
    SYMS_String8 name_stabilized = syms_push_string_copy(arena, name);
    
    // allen: determine virt size vs file size
    SYMS_U64 virt_size = header.sh_size;
    if ((header.sh_flags & SYMS_ElfSectionFlag_ALLOC) == 0){
      virt_size = 0;
    }
    SYMS_U64 file_size = header.sh_size;
    if (header.sh_type == SYMS_ElfSectionCode_NOBITS){
      file_size = 0;
    }
    
    // rjf: fill section data
    sections[section_idx].header = header;
    sections[section_idx].virtual_range = syms_make_u64_range(header.sh_addr, header.sh_addr + virt_size);
    sections[section_idx].file_range = syms_make_u64_range(header.sh_offset, header.sh_offset + file_size);
    sections[section_idx].name = name_stabilized;

    if (header.sh_flags & SYMS_ElfSectionFlag_COMPRESSED)
    {
      SYMS_U64Range chdr_range = sections[section_idx].file_range;
      chdr_range.min = SYMS_ClampTop(chdr_range.min, syms_u64_range_size(file_range));
      chdr_range.max = SYMS_ClampTop(chdr_range.max, syms_u64_range_size(file_range));
      
      SYMS_ElfChdr64 chdr = { 0 };
      SYMS_U64 chdr_size = 0;
      if (img.is_32bit)
      {
        SYMS_ElfChdr32 chdr32 = { 0 };
        chdr_size = syms_based_range_read_struct(file_base, chdr_range, 0, &chdr32);
        if (chdr_size == sizeof(chdr32))
        {
          chdr.ch_type = chdr32.ch_type;
          chdr.ch_size = chdr32.ch_size;
          chdr.ch_addr_align = chdr32.ch_addr_align;
        }
      }
      else
      {
        SYMS_ElfChdr64 chdr64 = { 0 };
        chdr_size = syms_based_range_read_struct(file_base, chdr_range, 0, &chdr64);
        if (chdr_size == sizeof(chdr64))
        {
          chdr.ch_type = chdr64.ch_type;
          chdr.ch_size = chdr64.ch_size;
          chdr.ch_addr_align = chdr64.ch_addr_align;
        }
      }

      if (chdr.ch_type == SYMS_ElfCompressType_None)
      {
        // ok, no compression
      }
      else if (chdr.ch_type == SYMS_ElfCompressType_ZLib)
      {
        SYMS_U64 uncompressed_size = chdr.ch_size;
        SYMS_U64 compressed_size = file_size;
        if (compressed_size > syms_u64_range_size(file_range) - header.sh_offset) compressed_size = 0;
        if (compressed_size >= chdr_size) compressed_size -= chdr_size;
        
        // chdr.ch_addr_align is not relevant here, because structures will be memcpy'ied out of memory
        SYMS_U8* uncompressed = syms_push_array(arena, SYMS_U8, uncompressed_size);
        SYMS_U8* compressed = (SYMS_U8*)file_base + header.sh_offset + chdr_size;

        SYMS_U64 output_size = 0;

        z_stream stream = { 0 };
        int err = inflateInit(&stream);

        if (err == Z_OK)
        {
            stream.next_in = (const Bytef*)compressed;
            stream.next_out = (Bytef*)uncompressed;

            const SYMS_U64 max_chunk = SYMS_U32_MAX; // max uInt value for zlib

            do
            {
                if (stream.avail_out == 0)
                {
                    stream.avail_out = (uInt)SYMS_ClampTop(uncompressed_size, max_chunk);
                    uncompressed_size -= stream.avail_out;
                }
                if (stream.avail_in == 0)
                {
                    stream.avail_in = (uInt)SYMS_ClampTop(compressed_size, max_chunk);
                    compressed_size -= stream.avail_in;
                }

                uInt avail_out = stream.avail_out;

                err = inflate(&stream, Z_NO_FLUSH);
                if (err == Z_OK || err == Z_STREAM_END)
                {
                    output_size += avail_out - stream.avail_out;
                }
            }
            while (err == Z_OK);

            if (err != Z_STREAM_END)
            {
                output_size = 0;
            }

            inflateEnd(&stream);
        }

        if (output_size == 0)
        {
          // failed to decompress with zlib, this is failure
          result.v = 0;
          result.count = 0;
          return result;
        }

        // adjust range so when original data pointer is added, we get offsets into newly allocated space
        sections[section_idx].file_range.min = uncompressed - (SYMS_U8*)file_base;
        sections[section_idx].file_range.max = uncompressed - (SYMS_U8*)file_base + output_size;
      }
      //else if (chdr.ch_type == ELF_CompressType_ZStd)
      //{
      //  TODO
      //}
      else
      {
        // unknown compression type, cannot continue parsing
        result.v = 0;
        result.count = 0;
        return result;
      }
    }
  }

  //- rjf: fill result
  result.v = sections;
  result.count = section_count;
  
  return result;
}

SYMS_API SYMS_ElfSegmentArray
syms_elf_segment_array_from_img_header(SYMS_Arena *arena, SYMS_String8 file, SYMS_ElfImgHeader img)
{
  void *base = file.str;
  SYMS_U64Range range = syms_make_u64_range(0, file.size);
  
  SYMS_U64 segment_count = img.ehdr.e_phnum;
  SYMS_ElfPhdr64 *segments = syms_push_array_zero(arena, SYMS_ElfPhdr64, segment_count);
  for(SYMS_U64 segment_idx = 0; segment_idx < segment_count; segment_idx += 1) 
  {
    if(img.is_32bit) 
    {
      SYMS_ElfPhdr32 phdr32;
      syms_based_range_read_struct(base, range, img.ehdr.e_phoff + segment_idx * sizeof(SYMS_ElfPhdr32), &phdr32);
      segments[segment_idx] = syms_elf_phdr64_from_phdr32(phdr32);
    }
    else
    {
      syms_based_range_read_struct(base, range, img.ehdr.e_phoff + segment_idx * sizeof(SYMS_ElfPhdr64), &segments[segment_idx]);
    }
  }
  
  SYMS_ElfSegmentArray result;
  result.count = segment_count;
  result.v = segments;
  
  return result;
}

SYMS_API SYMS_ElfExtDebugRef
syms_elf_ext_debug_ref_from_elf_section_array(SYMS_String8 file, SYMS_ElfSectionArray sections)
{
  void *file_base = file.str;
  SYMS_U64Range file_range = syms_make_u64_range(0, file.size);
  SYMS_ElfExtDebugRef result;
  syms_memzero_struct(&result);
  for(SYMS_U64 section_idx = 0; section_idx < sections.count; section_idx += 1)
  {
    if(syms_string_match(sections.v[section_idx].name, syms_str8_lit(".gnu_debuglink"), 0))
    {
      //- rjf: offsets
      SYMS_U64 path_off     = sections.v[section_idx].file_range.min;
      SYMS_U64 checksum_off = SYMS_U64_MAX;
      
      //- rjf: read external debug info path
      result.path = syms_based_range_read_string(file_base, file_range, path_off);
      SYMS_U64 path_bytes = result.path.size + 1;
      
      //- rjf: calculate checksum read offset; pad to the next 4-byte boundary
      checksum_off = path_off + path_bytes;
      checksum_off += (checksum_off % 4);
      
      //- rjf: read checksum
      syms_based_range_read_struct(file_base, file_range, checksum_off, &result.external_file_checksum);
      
      break;
    }
  }
  return result;
}

////////////////////////////////
//~ rjf: High-Level API Canonical Conversions

SYMS_API SYMS_SecInfo
syms_elf_section_info_from_elf_section(SYMS_ElfSection elf_section){
  SYMS_SecInfo result = {0};
  result.vrange = elf_section.virtual_range;
  result.frange = elf_section.file_range;
  result.name = elf_section.name;
  return(result);
}

SYMS_API SYMS_String8
syms_elf_sec_name_from_elf_section(SYMS_ElfSection elf_section){
  SYMS_String8 result = elf_section.name;
  return(result);
}

////////////////////////////////
//~ rjf: File Accelerator

SYMS_API SYMS_ElfFileAccel *
syms_elf_file_accel_from_data(SYMS_Arena *arena, SYMS_String8 string)
{
  SYMS_ElfFileAccel *file_accel = syms_push_array_zero(arena, SYMS_ElfFileAccel, 1);
  file_accel->header = syms_elf_img_header_from_file(string);
  if (file_accel->header.valid){
    file_accel->format = SYMS_FileFormat_ELF;
  }
  return file_accel;
}

////////////////////////////////
//~ rjf: Binary

SYMS_API SYMS_ElfBinAccel *
syms_elf_bin_accel_from_file(SYMS_Arena *arena, SYMS_String8 data, SYMS_ElfFileAccel *file_accel)
{
  SYMS_ElfBinAccel *bin_accel = syms_push_array_zero(arena, SYMS_ElfBinAccel, 1);
  syms_memmove(&bin_accel->header, &file_accel->header, sizeof(file_accel->header));
  bin_accel->format = file_accel->format;
  bin_accel->sections = syms_elf_section_array_from_img_header(arena, data, file_accel->header);
  bin_accel->segments = syms_elf_segment_array_from_img_header(arena, data, file_accel->header);
  return bin_accel;
}

SYMS_API SYMS_ExtFileList
syms_elf_ext_file_list_from_bin(SYMS_Arena *arena, SYMS_String8 file, SYMS_ElfBinAccel *bin_accel)
{
  SYMS_ExtFileList list = {0};
  SYMS_ElfExtDebugRef ext_debug_ref = syms_elf_ext_debug_ref_from_elf_section_array(file, bin_accel->sections);
  if(ext_debug_ref.path.size != 0)
  {
    SYMS_ExtFileNode *node = syms_push_array(arena, SYMS_ExtFileNode, 1);
    node->ext_file.file_name = ext_debug_ref.path;
    syms_memzero_struct(&node->ext_file.match_key);
    syms_memmove(node->ext_file.match_key.v, &ext_debug_ref.external_file_checksum, sizeof(SYMS_U32));
    SYMS_QueuePush(list.first, list.last, node);
    list.node_count += 1;
  }
  return list;
}

SYMS_API SYMS_SecInfoArray
syms_elf_sec_info_array_from_bin(SYMS_Arena *arena, SYMS_String8 data, SYMS_ElfBinAccel *bin)
{
  SYMS_SecInfoArray array = {0};
  array.count = bin->sections.count;
  array.sec_info = syms_push_array_zero(arena, SYMS_SecInfo, array.count);
  for(SYMS_U64 idx = 0; idx < array.count; idx += 1)
  {
    array.sec_info[idx] = syms_elf_section_info_from_elf_section(bin->sections.v[idx]);
  }
  return array;
}

SYMS_API SYMS_U64
syms_elf_default_vbase_from_bin(SYMS_ElfBinAccel *bin)
{
  return bin->header.base_address;
}

SYMS_API SYMS_U64
syms_elf_entry_point_voff_from_bin(SYMS_ElfBinAccel *bin)
{
  return bin->header.entry_point;
}

SYMS_API SYMS_Arch
syms_elf_arch_from_bin(SYMS_ElfBinAccel *bin){
  SYMS_Arch result = bin->header.arch;
  return(result);
}

////////////////////////////////
//~ NOTE(allen): ELF Specific Helpers

SYMS_API SYMS_ElfSection*
syms_elf_sec_from_bin_name__unstable(SYMS_ElfBinAccel *bin, SYMS_String8 name){
  SYMS_ElfSection *result = 0;
  for (SYMS_ElfSection *section = bin->sections.v, *opl = bin->sections.v + bin->sections.count;
       section < opl;
       section += 1){
    if (syms_string_match(name, section->name, 0)){
      result = section;
      break;
    }
  }
  return(result);
}

////////////////////////////////
//~ rjf: Imports/Exports

SYMS_API SYMS_ImportArray
syms_elf_imports_from_bin(SYMS_Arena *arena, SYMS_String8 data, SYMS_ElfBinAccel *bin)
{
  SYMS_ArenaTemp scratch = syms_get_scratch(&arena, 1);
  
  //- rjf: grab prerequisites
  void *base = (void *)data.str;
  SYMS_B32 bin_is_32bit = bin->header.is_32bit;
  SYMS_U64 sym_size = bin_is_32bit ? sizeof(SYMS_ElfSym32) : sizeof(SYMS_ElfSym64);
  
  //- rjf: find ranges
  SYMS_U64Range verneed_range        = syms_make_u64_range(0,0);
  SYMS_U64Range verneed_strtab_range = syms_make_u64_range(0,0);
  SYMS_U64Range versym_range         = syms_make_u64_range(0,0);
  SYMS_U64Range dynsym_range         = syms_make_u64_range(0,0);
  SYMS_U64Range dynsym_strtab_range  = syms_make_u64_range(0,0);
  {
    SYMS_ElfSection *first = &bin->sections.v[0];
    SYMS_ElfSection *opl = first+bin->sections.count;
    for(SYMS_ElfSection *sect = first; sect < opl; sect += 1) 
    {
      // NOTE(rjf): We subtract 1 from sh_link because we don't store the null
      // section in the bin section array.
      SYMS_U64 sh_link_idx = sect->header.sh_link-1;
      SYMS_B32 sh_link_in_bounds = (0 < sect->header.sh_link && sect->header.sh_link < bin->sections.count);
      (void)sh_link_in_bounds;
      if(syms_string_match(sect->name, syms_str8_lit(".gnu.version_r"), 0)) 
      {
        verneed_range = sect->file_range;
        verneed_strtab_range = bin->sections.v[sh_link_idx].file_range;
      }
      if(syms_string_match(sect->name, syms_str8_lit(".gnu.version"), 0)) 
      {
        versym_range = sect->file_range;
      }
      if(syms_string_match(sect->name, syms_str8_lit(".dynsym"), 0)) 
      {
        dynsym_range = sect->file_range;
        dynsym_strtab_range = bin->sections.v[sh_link_idx].file_range;
      }
    }
  }
  
  //- rjf: grab .dynsym symbol array
  SYMS_ElfSym64 *sym_arr = 0;
  SYMS_ElfSym64 *sym_arr_opl = 0;
  if (syms_u64_range_size(dynsym_range) > sym_size) 
  {
    SYMS_U64 sym_arr_count = (syms_u64_range_size(dynsym_range) / sym_size) - 1;
    sym_arr = syms_push_array_zero(scratch.arena, SYMS_ElfSym64, sym_arr_count);
    sym_arr_opl = sym_arr + sym_arr_count;
    // NOTE(rjf): We start the read offset at sym_size because the format
    // always has a meaningless null symbol at the beginning of the block.
    SYMS_U64 roff__sym_array = sym_size;
    if(bin_is_32bit)
    {
      for(SYMS_U64 idx = 0; idx < sym_arr_count; idx += 1)
      {
        SYMS_ElfSym32 *sym32 = (SYMS_ElfSym32 *)syms_based_range_ptr(base, dynsym_range, roff__sym_array + idx*sym_size);
        sym_arr[idx] = syms_elf_sym64_from_sym32(*sym32);
      }
    }
    else
    {
      syms_memmove(sym_arr, syms_based_range_ptr(base, dynsym_range, roff__sym_array), sym_arr_count*sym_size);
    }
  }
  
  //- rjf: parse all import info
  SYMS_ImportNode *first_import = 0;
  SYMS_ImportNode *last_import = 0;
  SYMS_U64 import_count = 0;
  {
    SYMS_U64 sym_idx = 1;
    for(SYMS_ElfSym64 *sym = sym_arr; sym < sym_arr_opl; sym += 1, sym_idx += 1)
    {
      // rjf: skip any symbol that does not have the right section index
      if(sym->st_shndx != SYMS_ElfSectionIndex_UNDEF)
      {
        continue;
      }
      
      // rjf: read versym
      SYMS_U64 roff__versym = sym_idx * sizeof(SYMS_ElfExternalVersym);
      SYMS_ElfExternalVersym vs = {0};
      syms_based_range_read_struct(base, versym_range, roff__versym, &vs);
      
      // rjf: parse vn_file / vna_name
      SYMS_String8 vn_file = syms_str8_lit("");
      SYMS_String8 vna_name = syms_str8_lit("");
      {
        // rjf: parse all verneeds
        SYMS_U64 verneed_read_offset = 0;
        SYMS_U16 version = vs.vs_vers & SYMS_ELF_EXTERNAL_VERSYM_MASK;
        SYMS_ElfExternalVerneed vn = {0};
        for(;;)
        {
          syms_based_range_read_struct(base, verneed_range, verneed_read_offset, &vn);
          
          // rjf: find vernaux with a matching version
          SYMS_U64 aux_read_offset = verneed_read_offset + vn.vn_aux;
          for(SYMS_U32 aux_idx = 0; aux_idx < vn.vn_cnt; aux_idx += 1) 
          {
            // rjf: read
            SYMS_ElfExternalVernaux vna = {0};
            syms_based_range_read_struct(base, verneed_range, aux_read_offset, &vna);
            
            // rjf: if the version matches, we've found the right strings
            if(vna.vna_other == version) 
            {
              vn_file  = syms_based_range_read_string(base, verneed_strtab_range, vn.vn_file);
              vna_name = syms_based_range_read_string(base, verneed_strtab_range, vna.vna_name);
              goto end__vn_file__vna_name__parse;
            }
            
            // rjf: advance
            if(vna.vna_next == 0) 
            {
              break;
            }
            aux_read_offset += vna.vna_next;
          }
          
          // rjf: advance
          if(vn.vn_next == 0) 
          {
            break;
          }
          verneed_read_offset += vn.vn_next;
        }
      }
      end__vn_file__vna_name__parse:;
      
      // rjf: build node
      SYMS_ImportNode *node = syms_push_array_zero(scratch.arena, SYMS_ImportNode, 1);
      SYMS_QueuePush(first_import, last_import, node);
      import_count += 1;
      {
        SYMS_Import *imp  = &node->data;
        
        // rjf: fill out symbol name
        {
          SYMS_ArenaTemp temp = syms_arena_temp_begin(scratch.arena);
          SYMS_String8List list = {0};
          SYMS_String8 symbol_name = syms_based_range_read_string(base, dynsym_strtab_range, sym->st_name);
          syms_string_list_push(temp.arena, &list, symbol_name);
          if(vna_name.size > 0) 
          {
            SYMS_B32 is_hidden = !!(vs.vs_vers & SYMS_ELF_EXTERNAL_VERSYM_HIDDEN);
            syms_string_list_push(temp.arena, &list, is_hidden ? syms_str8_lit("@@") : syms_str8_lit("@"));
            syms_string_list_push(temp.arena, &list, vna_name);
          }
          imp->name = syms_string_list_join(arena, &list, 0);
          syms_arena_temp_end(temp);
        }
        
        // rjf: fill library name
        {
          imp->library_name = syms_push_string_copy(arena, vn_file);
        }
      }
    }
  }
  
  //- rjf: build/fill result
  SYMS_ImportArray result = {0};
  {
    result.count = import_count;
    result.imports = syms_push_array_zero(arena, SYMS_Import, result.count);
    SYMS_U64 idx = 0;
    for(SYMS_ImportNode *in = first_import;
        in != 0;
        in = in->next, idx += 1)
    {
      result.imports[idx] = in->data;
    }
  }
  
  //- rjf: return
  syms_release_scratch(scratch);
  return result;
}

SYMS_API SYMS_ExportArray
syms_elf_exports_from_bin(SYMS_Arena *arena, SYMS_String8 data, SYMS_ElfBinAccel *bin)
{
  SYMS_ArenaTemp scratch = syms_get_scratch(&arena, 1);
  
  SYMS_U64Range verdef_range         = syms_make_u64_range(0,0);
  SYMS_U64Range verdef_strtab_range  = syms_make_u64_range(0,0);
  SYMS_U64Range versym_range         = syms_make_u64_range(0,0);
  SYMS_U64Range dynsym_range         = syms_make_u64_range(0,0);
  SYMS_U64Range dynsym_strtab_range  = syms_make_u64_range(0,0);
  for (SYMS_U32 sec_idx = 0; sec_idx < bin->sections.count; sec_idx += 1) 
  {
    SYMS_ElfSection *sect = &bin->sections.v[sec_idx];
    if (syms_string_match(sect->name, syms_str8_lit(".gnu.version_d"), 0)) 
    {
      SYMS_ASSERT(sect->header.sh_link > 0 && sect->header.sh_link < bin->sections.count);
      verdef_range = sect->file_range;
      verdef_strtab_range = bin->sections.v[sect->header.sh_link-1].file_range;
    }
    if (syms_string_match(sect->name, syms_str8_lit(".gnu.version"), 0)) 
    {
      versym_range = sect->file_range;
    }
    if (syms_string_match(sect->name, syms_str8_lit(".dynsym"), 0)) 
    {
      SYMS_ASSERT(sect->header.sh_link > 0 && sect->header.sh_link < bin->sections.count);
      dynsym_range = sect->file_range;
      dynsym_strtab_range = bin->sections.v[sect->header.sh_link-1].file_range;
    }
  }
  
  SYMS_ExportNode *first_export = 0;
  SYMS_ExportNode *last_export = 0;
  SYMS_U32 export_count = 0;
  void *base = (void*)data.str;
  SYMS_U64 read_offset = 0;
  for (; read_offset < syms_u64_range_size(dynsym_range);)
  {
    SYMS_U32 sym_idx = read_offset / (bin->header.is_32bit ? sizeof(SYMS_ElfSym32) : sizeof(SYMS_ElfSym64));
    SYMS_ElfSym64 sym;
    if (bin->header.is_32bit) 
    {
      SYMS_ElfSym32 sym32; syms_memzero_struct(&sym32);
      read_offset += syms_based_range_read_struct(base, dynsym_range, read_offset, &sym32);
      sym = syms_elf_sym64_from_sym32(sym32);
    }
    else
    {
      syms_memzero_struct(&sym);
      read_offset += syms_based_range_read_struct(base, dynsym_range, read_offset, &sym);
    }
    SYMS_U32 st_bind = SYMS_ELF_ST_BIND(sym.st_info);
    SYMS_B32 is_symbol_visible_to_other_objects = st_bind == SYMS_ElfSymBind_GLOBAL || st_bind == SYMS_ElfSymBind_WEAK;
    SYMS_B32 is_symbol_defined = sym.st_shndx != SYMS_ElfSectionIndex_UNDEF;
    if (is_symbol_visible_to_other_objects && is_symbol_defined) 
    {
      SYMS_StringJoin join; syms_memzero_struct(&join);
      SYMS_String8List name_list; syms_memzero_struct(&name_list);
      
      SYMS_String8 st_name = syms_based_range_read_string(base, dynsym_strtab_range, sym.st_name);
      syms_string_list_push(scratch.arena, &name_list, st_name);
      
      {
        // find symbol version token
        SYMS_ElfExternalVersym vs;
        syms_based_range_read_struct(base, versym_range, sym_idx * sizeof(vs), &vs);
        SYMS_U32 symbol_version = vs.vs_vers & SYMS_ELF_EXTERNAL_VERSYM_MASK;
        // iterate defined version to find version string
        SYMS_U64 verdef_read_offset = 0;
        for (;verdef_read_offset < syms_u64_range_size(verdef_range);) 
        {
          SYMS_ElfExternalVerdef vd; syms_memzero_struct(&vd);
          syms_based_range_read_struct(base, verdef_range, verdef_read_offset, &vd);
          SYMS_U32 version = vd.vd_ndx & SYMS_ELF_EXTERNAL_VERSYM_MASK;
          if (version == symbol_version) 
          {
            SYMS_B32 has_name = vd.vd_ndx > 1 && (~vd.vd_flags & SYMS_ElfExternalVerFlag_BASE);
            if (has_name) 
            {
              SYMS_U64 aux_read_offset = verdef_read_offset + vd.vd_aux;
              SYMS_ElfExternalVerdaux vda; syms_memzero_struct(&vda);
              syms_based_range_read_struct(base, verdef_range, aux_read_offset, &vda);
              SYMS_String8 verdef_name = syms_based_range_read_string(base, verdef_strtab_range, vda.vda_name);
              SYMS_B32 is_hidden = !!(vs.vs_vers & SYMS_ELF_EXTERNAL_VERSYM_HIDDEN);
              syms_string_list_push(scratch.arena, &name_list, is_hidden ? syms_str8_lit("@@") : syms_str8_lit("@"));
              syms_string_list_push(scratch.arena, &name_list, verdef_name);
            } 
            break;
          }
          if (vd.vd_next == 0) 
          {
            break;
          }
          verdef_read_offset += vd.vd_next;
        }
      }
      
      SYMS_U64 symbol_base_address = 0;
      if (sym.st_shndx < SYMS_ElfSectionIndex_LO_RESERVE) 
      {
        SYMS_ASSERT(sym.st_shndx > 0 && sym.st_shndx <= bin->sections.count);
        SYMS_ElfSection *sect = &bin->sections.v[sym.st_shndx-1];
        symbol_base_address = sect->virtual_range.min;
      }
      
      // add to list
      SYMS_ExportNode *node = syms_push_array_zero(scratch.arena, SYMS_ExportNode, 1);
      SYMS_QueuePush(first_export, last_export, node);
      export_count += 1;
      
      // fill out data
      SYMS_Export *exp = &node->data;
      exp->name = syms_string_list_join(arena, &name_list, &join);
      exp->address = symbol_base_address + sym.st_value;
      exp->ordinal = 0;
      exp->forwarder_library_name = syms_str8(0,0);
      exp->forwarder_import_name = syms_str8(0,0);
    }
  }
  
  SYMS_ExportArray export_array;
  export_array.count = 0;
  export_array.exports = syms_push_array(arena, SYMS_Export, export_count);
  for (SYMS_ExportNode *en = first_export; en != 0; en = en->next) 
  {
    export_array.exports[export_array.count++] = en->data;
  }
  
  syms_release_scratch(scratch);
  
  return export_array;
}

#endif // SYMS_ELF_PARSER_C
