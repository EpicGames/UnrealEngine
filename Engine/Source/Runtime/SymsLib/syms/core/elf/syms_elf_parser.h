// Copyright Epic Games, Inc. All Rights Reserved.
/* date = May 11th 2021 4:55 pm */

#ifndef SYMS_ELF_PARSER_H
#define SYMS_ELF_PARSER_H

////////////////////////////////
//~ NOTE(rjf): Low-Level Parsing Types

typedef struct SYMS_ElfImgHeader SYMS_ElfImgHeader;
struct SYMS_ElfImgHeader
{
  SYMS_B32 valid;
  SYMS_B32 is_32bit;
  SYMS_ElfEhdr64 ehdr;
  SYMS_Arch arch;
  SYMS_U64 sh_name_low_offset;
  SYMS_U64 sh_name_high_offset;
  SYMS_U64 base_address;
  SYMS_U64 entry_point;
};

typedef struct SYMS_ElfSection SYMS_ElfSection;
struct SYMS_ElfSection
{
  SYMS_ElfShdr64 header;
  SYMS_U64Range virtual_range;
  SYMS_U64Range file_range;
  SYMS_String8 name;
};

typedef struct SYMS_ElfSectionArray SYMS_ElfSectionArray;
struct SYMS_ElfSectionArray
{
  SYMS_U64 count;
  SYMS_ElfSection *v;
};

typedef struct SYMS_ElfSegmentArray SYMS_ElfSegmentArray;
struct SYMS_ElfSegmentArray
{
  SYMS_U64 count;
  SYMS_ElfPhdr64 *v;
};

typedef struct SYMS_ElfExtDebugRef SYMS_ElfExtDebugRef;
struct SYMS_ElfExtDebugRef
{
  SYMS_String8 path;
  SYMS_U32 external_file_checksum;
};

////////////////////////////////
//~ NOTE(rjf): File Accelerator

typedef struct SYMS_ElfFileAccel SYMS_ElfFileAccel;
struct SYMS_ElfFileAccel
{
  SYMS_FileFormat format;
  SYMS_ElfImgHeader header;
};

////////////////////////////////
//~ NOTE(rjf): Binary Accelerator

typedef struct SYMS_ElfBinAccel SYMS_ElfBinAccel;
struct SYMS_ElfBinAccel
{
  SYMS_FileFormat format;
  SYMS_ElfImgHeader header;
  SYMS_ElfSectionArray sections;
  SYMS_ElfSegmentArray segments;
};

SYMS_C_LINKAGE_BEGIN

////////////////////////////////
//~ rjf: Low-Level Header/Section Parsing

SYMS_API SYMS_ElfImgHeader    syms_elf_img_header_from_file(SYMS_String8 file);
SYMS_API SYMS_ElfExtDebugRef  syms_elf_ext_debug_ref_from_elf_section_array(SYMS_String8 file, SYMS_ElfSectionArray sections);
SYMS_API SYMS_ElfSectionArray syms_elf_section_array_from_img_header(SYMS_Arena *arena, SYMS_String8 file, SYMS_ElfImgHeader img);
SYMS_API SYMS_ElfSegmentArray syms_elf_segment_array_from_img_header(SYMS_Arena *arena, SYMS_String8 file, SYMS_ElfImgHeader img);

////////////////////////////////
//~ rjf: High-Level API Canonical Conversions

SYMS_API SYMS_SecInfo syms_elf_section_info_from_elf_section(SYMS_ElfSection elf_section);

////////////////////////////////
//~ rjf: File Accelerator

SYMS_API SYMS_ElfFileAccel *syms_elf_file_accel_from_data(SYMS_Arena *arena, SYMS_String8 string);

////////////////////////////////
//~ rjf: Binary

SYMS_API SYMS_ElfBinAccel *syms_elf_bin_accel_from_file(SYMS_Arena *arena, SYMS_String8 data, SYMS_ElfFileAccel *file_accel);
SYMS_API SYMS_ExtFileList  syms_elf_ext_file_list_from_bin(SYMS_Arena *arena, SYMS_String8 file, SYMS_ElfBinAccel *bin_accel);
SYMS_API SYMS_SecInfoArray syms_elf_sec_info_array_from_bin(SYMS_Arena *arena, SYMS_String8 data, SYMS_ElfBinAccel *bin);
SYMS_API SYMS_U64          syms_elf_default_vbase_from_bin(SYMS_ElfBinAccel *bin);
SYMS_API SYMS_U64          syms_elf_entry_point_voff_from_bin(SYMS_ElfBinAccel *bin);

SYMS_API SYMS_Arch         syms_elf_arch_from_bin(SYMS_ElfBinAccel *bin);

////////////////////////////////
//~ NOTE(allen): ELF Specific Helpers

SYMS_API SYMS_ElfSection*   syms_elf_sec_from_bin_name__unstable(SYMS_ElfBinAccel *bin, SYMS_String8 name);

////////////////////////////////
//~ rjf: Imports/Exports

SYMS_API SYMS_ImportArray syms_elf_imports_from_bin(SYMS_Arena *arena, SYMS_String8 data, SYMS_ElfBinAccel *bin);
SYMS_API SYMS_ExportArray syms_elf_exports_from_bin(SYMS_Arena *arena, SYMS_String8 data, SYMS_ElfBinAccel *bin);

SYMS_C_LINKAGE_END

#endif // SYMS_ELF_PARSER_H
