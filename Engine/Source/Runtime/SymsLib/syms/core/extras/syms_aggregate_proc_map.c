// Copyright Epic Games, Inc. All Rights Reserved.

////////////////////////////////
// NOTE(allen): Syms Procedure Aggregator Function

SYMS_API SYMS_AggregatedProcedureMap
syms_aggregated_procedure_map_from_group(SYMS_Arena *arena, SYMS_Group *group){
  SYMS_ProfBegin("syms_group_aggregated_procedure_map");
  SYMS_ArenaTemp scratch = syms_get_scratch(&arena, 1);
  
  SYMS_U64 unit_count = group->unit_count;
  
  //- gather all procs
  SYMS_U64ArrayNode *ua_first = 0;
  SYMS_U64ArrayNode *ua_last = 0;
  SYMS_U64 total_count = 0;
  for (SYMS_UnitID uid = 1;
       uid <= unit_count;
       uid += 1){
    SYMS_SymbolIDArray *procs = syms_group_proc_sid_array_from_uid(group, uid);
    SYMS_U64 *proc_ids = procs->ids;
    SYMS_U64 proc_count = procs->count;
    
    SYMS_U64ArrayNode *ua_node = syms_push_array_zero(scratch.arena, SYMS_U64ArrayNode, 1);
    SYMS_QueuePush(ua_first, ua_last, ua_node);
    ua_node->u64 = proc_ids;
    ua_node->count = proc_count;
    total_count += proc_count;
  }
  
  //- setup spatial map
  SYMS_SpatialMap1DLoose sm_loose = {0};
  {
    SYMS_String8 data = group->dbg_data;
    SYMS_DbgAccel *dbg = group->dbg;
    SYMS_U64ArrayNode *ua_node = ua_first;
    SYMS_U64 proc_index = 0;
    for (SYMS_UnitID uid = 1;
         uid <= unit_count;
         uid += 1, ua_node = ua_node->next){
      SYMS_UnitAccel *unit = syms_group_unit_from_uid(group, uid);
      SYMS_U64 *sid_ptr = ua_node->u64;
      SYMS_U64 count = ua_node->count;
      for (SYMS_U64 i = 0; i < count; i += 1, sid_ptr += 1){
        SYMS_U64RangeArray vranges = syms_scope_vranges_from_sid(scratch.arena, data, dbg, unit, *sid_ptr);
        syms_spatial_map_1d_loose_push(scratch.arena, &sm_loose, proc_index, vranges);
        proc_index += 1;
      }
    }
  }
  
  SYMS_SpatialMultiMap1D map = syms_spatial_multi_map_1d_bake(arena, &sm_loose);
  
  //- flatten procs into a usid array
  SYMS_USIDArray procs = {0};
  procs.usid = syms_push_array(arena, SYMS_USID, total_count);
  procs.count = total_count;
  {
    SYMS_USID *usid_ptr = procs.usid;
    SYMS_U64ArrayNode *ua_node = ua_first;
    for (SYMS_UnitID uid = 1;
         uid <= unit_count;
         uid += 1, ua_node = ua_node->next){
      SYMS_U64 *u64_ptr = ua_node->u64;
      SYMS_U64 count = ua_node->count;
      for (SYMS_U64 i = 0; i < count; i += 1){
        usid_ptr->uid = uid;
        usid_ptr->sid = *u64_ptr;
        usid_ptr += 1;
        u64_ptr += 1;
      }
    }
  }
  
  //- assemble result
  SYMS_AggregatedProcedureMap result = {0};
  result.procs = procs;
  result.map = map;
  
  syms_release_scratch(scratch);
  SYMS_ProfEnd();
  return(result);
}