// Copyright Epic Games, Inc. All Rights Reserved.
/* date = January 18th 2022 1:40 pm */

#ifndef SYMS_AGGREGATE_PROC_MAP_H
#define SYMS_AGGREGATE_PROC_MAP_H

// This extra relies on the "syms group" layer

typedef struct SYMS_AggregatedProcedureMap{
  SYMS_USIDArray procs;
  SYMS_SpatialMultiMap1D map;
} SYMS_AggregatedProcedureMap;

SYMS_API SYMS_AggregatedProcedureMap syms_aggregated_procedure_map_from_group(SYMS_Arena *arena,
                                                                              SYMS_Group *group);

#endif //SYMS_AGGREGATE_PROC_MAP_H
