// Copyright Epic Games, Inc. All Rights Reserved.

#ifndef SYMS_FILE_INF_C
#define SYMS_FILE_INF_C

////////////////////////////////
// NOTE(allen): File Inference Functions

SYMS_API SYMS_FileInfResult
syms_file_inf_infer_from_file_list(SYMS_Arena *arena, SYMS_FileLoadCtx ctx, SYMS_String8List file_name_list,
                                   SYMS_FileInfOptions *opts_ptr){
  SYMS_ProfBegin("syms_file_inf_infer_from_file_list");
  
  SYMS_FileInfOptions opts = {0};
  if (opts_ptr != 0){
    syms_memmove(&opts, opts_ptr, sizeof(opts));
  }
  
  SYMS_FileInfState state = {0};
  SYMS_ArenaTemp scratch = syms_get_scratch(&arena, 1);
  
  //- dump input file names into task queue
  for (SYMS_String8Node *node = file_name_list.first;
       node != 0;
       node = node->next){
    SYMS_FileInfTask *task = syms_file_inf_push_task(scratch.arena, &state);
    task->file_name = node->string;
  }
  
  //- while there are tasks in the task queue, analyze each task
  for (;;){
    SYMS_FileInfTask *task = state.first_task;
    if (task == 0){
      break;
    }
    
    //- file load setup
    SYMS_String8 file_name = task->file_name;
    SYMS_String8List new_names = {0};
    SYMS_FileInfNode *existing_node = 0;
    SYMS_String8 data = {0};
    
    //- try the path exactly as stated
    existing_node = syms_file_inf_node_from_name(&state, file_name);
    if (existing_node == 0){
      syms_string_list_push(scratch.arena, &new_names, file_name);
      SYMS_ProfBegin("syms_file_inf_infer_from_file_list.file_load_func");
      data = ctx.file_load_func(ctx.file_load_user, arena, file_name);
      SYMS_ProfEnd();
    }
    else{
      data = existing_node->data;
    }
    
    //- try the fallback path
    if ((existing_node == 0 && data.size == 0) &&
        !opts.disable_fallback && opts.fallback_path.size != 0){
      // skip last slash
      SYMS_U8 *ptr = file_name.str + file_name.size - 1;
      for (;ptr >= file_name.str; ptr -= 1){
        if (*ptr == '/' || *ptr == '\\'){
          break;
        }
      }
      ptr += 1;
      SYMS_U8 *opl = file_name.str + file_name.size;
      SYMS_String8 file_name_without_path = syms_str8_range(ptr, opl);
      
      // concat strings
      SYMS_String8 joined_path = {0};
      {
        joined_path.size = opts.fallback_path.size + file_name_without_path.size + 1;
        joined_path.str = syms_push_array(arena, SYMS_U8, joined_path.size + 1);
        syms_memmove(joined_path.str, opts.fallback_path.str, opts.fallback_path.size);
        syms_memmove(joined_path.str + opts.fallback_path.size, "/", 1);
        syms_memmove(joined_path.str + opts.fallback_path.size + 1,
                     file_name_without_path.str, file_name_without_path.size);
        joined_path.str[joined_path.size] = 0;
      }
      
      existing_node = syms_file_inf_node_from_name(&state, joined_path);
      if (existing_node == 0){
        syms_string_list_push(scratch.arena, &new_names, joined_path);
        data = ctx.file_load_func(ctx.file_load_user, arena, joined_path);
      }
      else{
        data = existing_node->data;
      }
    }
    
    //- PE/PDB specific fallback
    if ((existing_node == 0 && data.size == 0) &&
        !opts.disable_fallback && task->inferred_from_node != 0 &&
        task->inferred_from_node->bin->format == SYMS_FileFormat_PE){
      SYMS_String8 inferred_from_file_name = task->inferred_from_node->file_name;
      if (inferred_from_file_name.size >= 4){
        SYMS_String8 file_name_head = {inferred_from_file_name.str, inferred_from_file_name.size - 4};
        SYMS_String8 file_name_tail = {inferred_from_file_name.str + file_name_head.size, 4};
        if (syms_string_match(file_name_tail, syms_str8_lit(".exe"), 0)){
          SYMS_String8 file_name_pdb = {0};
          {
            file_name_pdb.size = inferred_from_file_name.size;
            file_name_pdb.str = syms_push_array(arena, SYMS_U8, file_name_pdb.size + 1);
            syms_memmove(file_name_pdb.str, file_name_head.str, file_name_head.size);
            syms_memmove(file_name_pdb.str + file_name_head.size, ".pdb", 4);
            file_name_pdb.str[file_name_pdb.size] = 0;
          }
          
          existing_node = syms_file_inf_node_from_name(&state, file_name_pdb);
          if (existing_node == 0){
            syms_string_list_push(scratch.arena, &new_names, file_name_pdb);
            data = ctx.file_load_func(ctx.file_load_user, arena, file_name_pdb);
          }
          else{
            data = existing_node->data;
          }
        }
      }
    }
    
    //- analyze
    SYMS_FileAccel *file = (SYMS_FileAccel*)&syms_format_nil;
    SYMS_BinAccel *bin = (SYMS_BinAccel*)&syms_format_nil;
    SYMS_DbgAccel *dbg = (SYMS_DbgAccel*)&syms_format_nil;
    if (existing_node != 0){
      file = existing_node->file;
      bin = existing_node->bin;
      dbg = existing_node->dbg;
    }
    else if (data.size != 0){
      file = syms_file_accel_from_data(arena, data);
      
      //- set bin
      // try bin from bin list
      if (syms_file_is_bin_list(file)){
        SYMS_ArenaTemp scratch = syms_get_scratch(&arena, 1);
        SYMS_BinListAccel *bin_list = syms_bin_list_from_file(scratch.arena, data, file);
        
        SYMS_U64 number = 1;
        SYMS_Arch arch_hint = opts_ptr->preferred_arch;
        if (arch_hint != SYMS_Arch_Null){
          SYMS_BinInfoArray bin_info_array = syms_bin_info_array_from_bin_list(scratch.arena, bin_list);
          for (SYMS_U64 i = 0; i < bin_info_array.count; i += 1){
            if (bin_info_array.bin_info[i].arch == arch_hint){
              number = i + 1;
              break;
            }
          }
        }
        bin = syms_bin_accel_from_bin_list_number(arena, data, bin_list, number);
        
        syms_release_scratch(scratch);
      }
      
      // try bin from file
      if (bin->format == SYMS_FileFormat_Null){
        bin = syms_bin_accel_from_file(arena, data, file);
      }
      
      //- set dbg
      // try dbg from file
      dbg = syms_dbg_accel_from_file(arena, data, file);
      
      // try dbg from bin
      if (bin->format != SYMS_FileFormat_Null &&
          dbg->format == SYMS_FileFormat_Null){
        dbg = syms_dbg_accel_from_bin(arena, data, bin);
      }
    }
    
    //- save nodes
    SYMS_B32 inferred = (task->inferred_from_node != 0);
    SYMS_FileInfNode *first_new_node = 0;
    SYMS_FileInfNode *last_new_node = 0;
    for (SYMS_String8Node *new_name_node = new_names.first;
         new_name_node != 0;
         new_name_node = new_name_node->next){
      SYMS_FileInfNode *new_node = syms_file_inf_insert_node(arena, &state);
      new_node->file_name = new_name_node->string;
      new_node->data = data;
      new_node->file = file;
      new_node->bin = bin;
      new_node->dbg = dbg;
      new_node->inferred = inferred;
      
      if (new_name_node == new_names.first){
        first_new_node = new_node;
      }
      if (new_name_node == new_names.last){
        last_new_node = new_node;
      }
    }
    
    //- fill in fallback pointers on new nodes
    for (SYMS_FileInfNode *new_node = first_new_node;
         new_node != 0;
         new_node = new_node->next){
      new_node->fallback_to = new_node->next;
    }
    
    //- run file inference
    if (existing_node == 0 && !opts.disable_inference){
      SYMS_ExtFileList ext[2];
      ext[0] = syms_ext_file_list_from_bin(arena, data, bin);
      ext[1] = syms_ext_file_list_from_dbg(arena, data, dbg);
      for (SYMS_U64 i = 0; i < SYMS_ARRAY_SIZE(ext); i += 1){
        for (SYMS_ExtFileNode *node = ext[i].first;
             node != 0;
             node = node->next){
          SYMS_String8 ext_file_name = node->ext_file.file_name;
          SYMS_FileInfTask *task = syms_file_inf_push_task(scratch.arena, &state);
          task->file_name = ext_file_name;
          task->inferred_from_node = last_new_node;
        }
      }
    }
    
    syms_file_inf_pop_task(&state);
  }
  
  //- select bin_node & dbg_node
  SYMS_FileInfNode *bin_node = 0;
  SYMS_FileInfNode *dbg_node = 0;
  
  //- is there a working bin in the list?
  for (SYMS_FileInfNode *node = state.first_loaded;
       node != 0;
       node = node->next){
    if (node->bin->format != SYMS_FileFormat_Null){
      bin_node = node;
      break;
    }
  }
  
  //- first compatible dbg
  if (bin_node != 0){
    for (SYMS_FileInfNode *node = state.first_loaded;
         node != 0;
         node = node->next){
      
      // TODO(allen): check for compatibility for real
      SYMS_B32 compatible = syms_false;
      if (node->dbg->format != SYMS_FileFormat_Null){
        SYMS_FileFormat bin_format = bin_node->bin->format;
        SYMS_FileFormat dbg_format = node->dbg->format;
        
        SYMS_FileFormat expected_dbg_format = SYMS_FileFormat_Null;
        switch (bin_format){
          case SYMS_FileFormat_PE:     expected_dbg_format = SYMS_FileFormat_PDB;    break;
          case SYMS_FileFormat_ELF:
          case SYMS_FileFormat_MACH:   expected_dbg_format = SYMS_FileFormat_DWARF;  break;
        }
        if (dbg_format == expected_dbg_format){
          compatible = syms_true;
        }
      }
      
      if (compatible){
        dbg_node = node;
        break;
      }
    }
  }
  
  //- first dbg
  else{
    for (SYMS_FileInfNode *node = state.first_loaded;
         node != 0;
         node = node->next){
      if (node->dbg->format != SYMS_FileFormat_Null){
        dbg_node = node;
        break;
      }
    }
  }
  
  //- advance selected nodes through fallback chains
  if (bin_node != 0){
    for (;bin_node->fallback_to != 0; bin_node = bin_node->fallback_to);
  }
  if (dbg_node != 0){
    for (;dbg_node->fallback_to != 0; dbg_node = dbg_node->fallback_to);
  }
  
  syms_release_scratch(scratch);
  
  //- fill data_parsed
  SYMS_ParseBundle data_parsed = {0};
  data_parsed.bin = (SYMS_BinAccel*)&syms_format_nil;
  data_parsed.dbg = (SYMS_DbgAccel*)&syms_format_nil;
  if (bin_node != 0){
    data_parsed.bin_data = bin_node->data;
    data_parsed.bin = bin_node->bin;
  }
  if (dbg_node != 0){
    data_parsed.dbg_data = dbg_node->data;
    data_parsed.dbg = dbg_node->dbg;
  }
  
  //- mark the selected nodes
  if (bin_node != 0){
    bin_node->is_selected_bin = syms_true;
  }
  if (dbg_node != 0){
    dbg_node->is_selected_dbg = syms_true;
  }
  
  //- set result
  SYMS_FileInfResult result = {0};
  result.first_inf_node = state.first_loaded;
  result.last_inf_node = state.last_loaded;
  result.selected_bin = bin_node;
  result.selected_dbg = dbg_node;
  result.data_parsed = data_parsed;
  
  SYMS_ProfEnd();
  
  return(result);
}

SYMS_API SYMS_FileInfResult
syms_file_inf_infer_from_file(SYMS_Arena *arena, SYMS_FileLoadCtx ctx,
                              SYMS_String8 file_name, SYMS_FileInfOptions *opts){
  SYMS_String8Node node;
  SYMS_String8List list = {0};
  syms_string_list_push_node(&node, &list, file_name);
  SYMS_FileInfResult result = syms_file_inf_infer_from_file_list(arena, ctx, list, opts);
  return(result);
}

SYMS_API SYMS_FileInfState
syms_file_inf_begin(void){
  SYMS_FileInfState result = {0};
  return(result);
}

SYMS_API SYMS_FileInfNode*
syms_file_inf_node_from_name(SYMS_FileInfState *state, SYMS_String8 file_name){
  SYMS_ProfBegin("syms_file_inf_node_from_name");
  SYMS_FileInfNode *result = 0;
  for (SYMS_FileInfNode *node = state->first_loaded;
       node != 0;
       node = node->next){
    if (syms_string_match(file_name, node->file_name, 0)){
      result = node;
      break;
    }
  }
  SYMS_ProfEnd();
  return(result);
}

SYMS_API SYMS_FileInfNode*
syms_file_inf_insert_node(SYMS_Arena *arena, SYMS_FileInfState *state){
  SYMS_FileInfNode *result = syms_push_array_zero(arena, SYMS_FileInfNode, 1);
  SYMS_QueuePush(state->first_loaded, state->last_loaded, result);
  return(result);
}

SYMS_API SYMS_FileInfTask*
syms_file_inf_push_task(SYMS_Arena *arena, SYMS_FileInfState *state){
  SYMS_FileInfTask *result = state->free_task;
  if (result != 0){
    syms_memzero_struct(result);
  }
  else{
    result = syms_push_array_zero(arena, SYMS_FileInfTask, 1);
  }
  SYMS_QueuePush(state->first_task, state->last_task, result);
  return(result);
}

SYMS_API void
syms_file_inf_pop_task(SYMS_FileInfState *state){
  SYMS_FileInfTask *task = state->first_task;
  if (task != 0){
    SYMS_QueuePop(state->first_task, state->last_task);
    SYMS_StackPush(state->free_task, task);
  }
}

#endif //SYMS_FILE_INF_C