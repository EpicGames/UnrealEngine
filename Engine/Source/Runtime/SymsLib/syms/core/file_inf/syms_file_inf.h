// Copyright Epic Games, Inc. All Rights Reserved.
/* date = September 27th 2021 0:57 pm */

#ifndef SYMS_FILE_INF_H
#define SYMS_FILE_INF_H

////////////////////////////////
// NOTE(allen): File Inference Types

typedef SYMS_String8 SYMS_FileLoadFunc(void *user, SYMS_Arena *arena, SYMS_String8 file_name);

typedef struct SYMS_FileLoadCtx{
  SYMS_FileLoadFunc *file_load_func;
  void *file_load_user;
} SYMS_FileLoadCtx;

typedef struct SYMS_FileInfNode{
  struct SYMS_FileInfNode *next;
  SYMS_String8 file_name;
  SYMS_String8 data;
  
  SYMS_FileAccel *file;
  SYMS_BinAccel *bin;
  SYMS_DbgAccel *dbg;
  
  SYMS_B32 is_selected_bin;
  SYMS_B32 is_selected_dbg;
  SYMS_B32 inferred;
  
  struct SYMS_FileInfNode *fallback_to;
} SYMS_FileInfNode;

typedef struct SYMS_FileInfTask{
  struct SYMS_FileInfTask *next;
  SYMS_String8 file_name;
  SYMS_FileInfNode *inferred_from_node;
} SYMS_FileInfTask;

typedef struct SYMS_FileInfState{
  SYMS_FileInfNode *first_loaded;
  SYMS_FileInfNode *last_loaded;
  SYMS_FileInfTask *first_task;
  SYMS_FileInfTask *last_task;
  SYMS_FileInfTask *free_task;
} SYMS_FileInfState;

typedef struct SYMS_FileInfOptions{
  SYMS_B32 disable_inference;
  SYMS_B32 disable_fallback;
  SYMS_String8 fallback_path;
  SYMS_Arch preferred_arch;
} SYMS_FileInfOptions;

typedef struct SYMS_FileInfResult{
  SYMS_FileInfNode *first_inf_node;
  SYMS_FileInfNode *last_inf_node;
  SYMS_FileInfNode *selected_bin;
  SYMS_FileInfNode *selected_dbg;
  SYMS_ParseBundle data_parsed;
} SYMS_FileInfResult;

////////////////////////////////
// NOTE(allen): File Inference Functions

SYMS_API SYMS_FileInfResult syms_file_inf_infer_from_file_list(SYMS_Arena *arena, SYMS_FileLoadCtx ctx,
                                                               SYMS_String8List file_name_list,
                                                               SYMS_FileInfOptions *opts);
SYMS_API SYMS_FileInfResult syms_file_inf_infer_from_file(SYMS_Arena *arena, SYMS_FileLoadCtx ctx,
                                                          SYMS_String8 file_name, SYMS_FileInfOptions *opts);

SYMS_API SYMS_FileInfState syms_file_inf_begin(void);
SYMS_API SYMS_FileInfNode* syms_file_inf_node_from_name(SYMS_FileInfState *state, SYMS_String8 file_name);
SYMS_API SYMS_FileInfNode* syms_file_inf_insert_node(SYMS_Arena *arena, SYMS_FileInfState *state);

SYMS_API SYMS_FileInfTask* syms_file_inf_push_task(SYMS_Arena *arena, SYMS_FileInfState *state);
SYMS_API void              syms_file_inf_pop_task(SYMS_FileInfState *state);

SYMS_API SYMS_B32          syms_file_inf_state_step(SYMS_Arena *arena, SYMS_FileInfState *state);

#endif //SYMS_FILE_INF_H
