// Copyright Epic Games, Inc. All Rights Reserved.

////////////////////////////////
//~ allen: Spatial Map Constructors 

SYMS_API SYMS_SpatialMap1D
syms_spatial_map_from_line_table(SYMS_Arena *arena, SYMS_LineTable *line_table){
  SYMS_ProfBegin("syms_spatial_map_from_line_table");
  
  SYMS_U64 count = line_table->sequence_count;
  SYMS_SpatialMap1DRange *ranges = syms_push_array(arena, SYMS_SpatialMap1DRange, count);
  SYMS_SpatialMap1DRange *range_ptr = ranges;
  SYMS_U64 *seq_index_ptr = line_table->sequence_index_array;
  SYMS_Line *line_array = line_table->line_array;
  for (SYMS_U64 n = 1; n <= count; n += 1, range_ptr += 1){
    // get sequence range & inc
    SYMS_U64 first = *seq_index_ptr;
    seq_index_ptr += 1;
    SYMS_U64 opl = *seq_index_ptr;
    
    // extract sequence range
    SYMS_U64 last = opl - 1;
    SYMS_U64 first_voff = line_array[first].voff;
    SYMS_U64 opl_voff = line_array[last].voff;
    
    // fill range ptr
    range_ptr->range.min = first_voff;
    range_ptr->range.max = opl_voff;
    range_ptr->val = n;
  }
  
  //- sort
  if (!syms_spatial_map_1d_array_check_sorted(ranges, count)){
    syms_spatial_map_1d_array_sort(ranges, count);
  }
  
  //- assemble map
  SYMS_SpatialMap1D result = {ranges, count};
  
  SYMS_ProfEnd();
  return(result);
}

// creates a map of (voff -> procedure symbol id)
SYMS_API SYMS_SpatialMap1D
syms_spatial_map_for_procs_from_sid_array(SYMS_Arena *arena, SYMS_String8 data, SYMS_DbgAccel *dbg,
                                          SYMS_UnitAccel *unit, SYMS_SymbolIDArray *sid_array){
  //- build loose map
  SYMS_ArenaTemp scratch = syms_get_scratch(&arena, 1);
  SYMS_SpatialMap1DLoose loose = {0};
  
  SYMS_SymbolID *sid_ptr = sid_array->ids;
  SYMS_U64 count = sid_array->count;
  for (SYMS_U64 i = 0; i < count; i += 1, sid_ptr += 1){
    SYMS_SymbolID sid = *sid_ptr;
    SYMS_U64RangeArray ranges = syms_scope_vranges_from_sid(scratch.arena, data, dbg, unit, sid);
    syms_spatial_map_1d_loose_push(scratch.arena, &loose, sid, ranges);
  }
  
  //- bake tight map
  SYMS_SpatialMap1D result = syms_spatial_map_1d_bake(arena, &loose);
  
  syms_release_scratch(scratch);
  
  return(result);
}

////////////////////////////////
//~ allen: Name Map Constructors 

SYMS_API SYMS_SymbolNameMap
syms_name_map_from_sid_array(SYMS_Arena *arena, SYMS_String8 data, SYMS_DbgAccel *dbg,
                             SYMS_UnitAccel *unit, SYMS_SymbolID *sids, SYMS_U64 sid_count){
  
  SYMS_ArenaTemp scratch = syms_get_scratch(&arena, 1);
  
  SYMS_SymbolNameMapLoose loose = {0};
  SYMS_SymbolID *sid = sids;
  for (SYMS_U64 i = 0; i < sid_count; i += 1, sid += 1){
    SYMS_String8 name = syms_symbol_name_from_sid(scratch.arena, data, dbg, unit, *sid);
    syms_symbol_name_map_push(scratch.arena, &loose, name, *sid);
  }
  
  SYMS_SymbolNameMap result = syms_symbol_name_map_bake(arena, &loose);
  
  syms_release_scratch(scratch);
  
  return(result);
}
