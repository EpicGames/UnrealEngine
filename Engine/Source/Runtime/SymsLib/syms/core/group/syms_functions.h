// Copyright Epic Games, Inc. All Rights Reserved.
/* date = August 15th 2022 10:09 am */

#ifndef SYMS_FUNCTIONS_H
#define SYMS_FUNCTIONS_H

////////////////////////////////
//~ allen: Spatial Map Constructors 

// creates a map of (voff -> line_sequence_index)
SYMS_API SYMS_SpatialMap1D syms_spatial_map_from_line_table(SYMS_Arena *arena, SYMS_LineTable *table);

// creates a map of (voff -> procedure symbol id)
SYMS_API SYMS_SpatialMap1D
syms_spatial_map_for_procs_from_sid_array(SYMS_Arena *arena, SYMS_String8 data, SYMS_DbgAccel *dbg,
                                          SYMS_UnitAccel *unit, SYMS_SymbolIDArray *sid_array);

////////////////////////////////
//~ allen: Name Map Constructors 

SYMS_API SYMS_SymbolNameMap
syms_name_map_from_sid_array(SYMS_Arena *arena, SYMS_String8 data, SYMS_DbgAccel *dbg,
                             SYMS_UnitAccel *unit, SYMS_SymbolID *sids, SYMS_U64 sid_count);

#endif //SYMS_FUNCTIONS_H
