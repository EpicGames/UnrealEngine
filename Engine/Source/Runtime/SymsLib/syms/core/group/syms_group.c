// Copyright Epic Games, Inc. All Rights Reserved.

#ifndef SYMS_GROUP_C
#define SYMS_GROUP_C

////////////////////////////////
// allen: Syms Group Setup Functions

SYMS_API SYMS_Group*
syms_group_alloc(void){
  SYMS_Arena *arena = syms_arena_alloc();
  SYMS_Group *result = syms_push_array_zero(arena, SYMS_Group, 1);
  result->arena = arena;
  return(result);
}

SYMS_API void
syms_group_release(SYMS_Group *group){
  syms_arena_release(group->arena);
}

SYMS_API void
syms_group_init(SYMS_Group *group, SYMS_ParseBundle *params){
  SYMS_ASSERT_PARANOID(syms_thread_lane == 0);
  SYMS_ProfBegin("syms_group_init");
  
  //- fill in bin
  SYMS_String8 bin_data = params->bin_data;
  SYMS_BinAccel *bin = params->bin;
  if (bin_data.size > 0){
    if (bin == 0){
      SYMS_FileAccel *bin_file = syms_file_accel_from_data(group->arena, params->bin_data);
      bin = syms_bin_accel_from_file(group->arena, params->bin_data, bin_file);
    }
  }
  else{
    bin = (SYMS_BinAccel*)&syms_format_nil;
  }
  
  //- fill in dbg
  SYMS_String8 dbg_data = params->dbg_data;
  SYMS_DbgAccel *dbg = params->dbg;
  if (dbg_data.size > 0){
    if (dbg == 0){
      SYMS_FileAccel *dbg_file = syms_file_accel_from_data(group->arena, params->dbg_data);
      dbg = syms_dbg_accel_from_file(group->arena, params->dbg_data, dbg_file);
    }
  }
  else{
    dbg = (SYMS_DbgAccel*)&syms_format_nil;
  }
  
  //- arch & address size
  SYMS_Arch arch = syms_arch_from_bin(bin);
  if (arch == SYMS_Arch_Null){
    arch = syms_arch_from_dbg(dbg);
  }
  SYMS_U64 address_size = (syms_address_size_from_arch(arch) >> 3);
  
  //- default vbase
  SYMS_U64 default_vbase = 0;
  if (bin->format != SYMS_FileFormat_Null){
    default_vbase = syms_default_vbase_from_bin(bin);
  }
  else{
    default_vbase = syms_default_vbase_from_dbg(dbg);
  }
  
  //- sections
  SYMS_SecInfoArray sec_info_array = {0};
  if (bin->format != SYMS_FileFormat_Null){
    sec_info_array = syms_sec_info_array_from_bin(group->arena, bin_data, bin);
  }
  else{
    sec_info_array = syms_sec_info_array_from_dbg(group->arena, dbg_data, dbg);
  }
  
  //- top-level accelerators
  SYMS_UnitSetAccel *unit_set = syms_unit_set_accel_from_dbg(group->arena, dbg_data, dbg);
  SYMS_U64 unit_count = syms_unit_count_from_set(unit_set);
  SYMS_MapAccel *type_map = syms_type_map_from_dbg(group->arena, dbg_data, dbg);
  SYMS_MapAccel *symbol_map = syms_unmangled_symbol_map_from_dbg(group->arena, dbg_data, dbg);
  
  //- init lanes to single-threaded setup
  group->lane_arenas = &group->arena;
  group->lane_count = 1;
  
  //- fill the group
  group->bin_data = bin_data;
  group->bin = bin;
  group->dbg_data = dbg_data;
  group->dbg = dbg;
  
  group->arch = arch;
  group->address_size = address_size;
  group->default_vbase = default_vbase;
  group->sec_info_array = sec_info_array;
  group->unit_set = unit_set;
  group->unit_count = unit_count;
  group->type_mau.map = type_map;
  group->symbol_mau.map = symbol_map;
  
  //- setup basic caches
  group->unit_cache_flags = syms_push_array_zero(group->arena, SYMS_GroupUnitCacheFlags, unit_count);
  group->units = syms_push_array_zero(group->arena, SYMS_UnitAccel*, unit_count);
  group->proc_sid_arrays = syms_push_array_zero(group->arena, SYMS_SymbolIDArray, unit_count);
  group->var_sid_arrays = syms_push_array_zero(group->arena, SYMS_SymbolIDArray, unit_count);
  group->thread_sid_arrays = syms_push_array_zero(group->arena, SYMS_SymbolIDArray, unit_count);
  group->type_sid_arrays = syms_push_array_zero(group->arena, SYMS_SymbolIDArray, unit_count);
  group->file_tables = syms_push_array_zero(group->arena, SYMS_String8Array, unit_count);
  group->inferred_file_tables = syms_push_array_zero(group->arena, SYMS_String8Array, unit_count);
  group->line_tables = syms_push_array_zero(group->arena, SYMS_LineParseOut, unit_count);
  group->unit_proc_maps = syms_push_array_zero(group->arena, SYMS_SpatialMap1D, unit_count);
  group->unit_var_maps = syms_push_array_zero(group->arena, SYMS_SpatialMap1D, unit_count);
  group->line_sequence_maps = syms_push_array_zero(group->arena, SYMS_SpatialMap1D, unit_count);
  group->file_to_line_to_addr_maps = syms_push_array_zero(group->arena, SYMS_FileToLineToAddrMap,
                                                          unit_count);
  group->unit_type_maps = syms_push_array_zero(group->arena, SYMS_SymbolNameMap, unit_count);
  
  //- setup type graph
  syms_type_graph_init(&group->type_graph,
                       group->arena, &group->string_cons,
                       group->address_size);
  
  SYMS_ProfEnd();
}

SYMS_API void
syms_group_parse_all_units__single_thread(SYMS_Group *group){
  SYMS_ProfBegin("syms_group_parse_all_units__single_thread");
  
  SYMS_U64 count = group->unit_count;
  for (SYMS_U64 n = 1; n <= count; n += 1){
    SYMS_UnitID uid = n;
    syms_group_unit_from_uid(group, uid);
    syms_group_proc_sid_array_from_uid(group, uid);
    syms_group_var_sid_array_from_uid(group, uid);
    syms_group_type_sid_array_from_uid(group, uid);
    syms_group_file_table_from_uid(group, uid);
    syms_group_line_table_from_uid(group, uid);
    syms_group_proc_map_from_uid(group, uid);
    syms_group_line_sequence_map_from_uid(group, uid);
    syms_group_fetch_line_to_addr_maps_from_uid(group, uid);
  }
  
  SYMS_ProfEnd();
}

SYMS_API void
syms_group_parse_all_top_level(SYMS_Group *group){
  SYMS_ProfBegin("syms_group_parse_all_top_level");
  
  syms_group_unit_ranges(group);
  syms_group_type_map(group);
  syms_group_unmangled_symbol_map(group);
  
  syms_group_sec_map_v(group);
  syms_group_sec_map_f(group);
  syms_group_unit_map(group);
  syms_group_file_map(group);
  
  SYMS_ProfEnd();
}

SYMS_API void
syms_group_begin_multilane(SYMS_Group *group, SYMS_U64 lane_count){
  SYMS_ASSERT_PARANOID(syms_thread_lane == 0);
  SYMS_ASSERT(group->lane_count == 1);
  if (lane_count > 1){
    // allocate the group's lane arenas array
    if (lane_count > group->lane_max){
      group->lane_arenas = syms_push_array(group->arena, SYMS_Arena*, lane_count);
      group->lane_max = lane_count;
    }
    
    // fill the group's lane arenas array
    SYMS_Arena **lane_arena_ptr = group->lane_arenas;
    *lane_arena_ptr = group->arena;
    lane_arena_ptr += 1;
    for (SYMS_U64 i = 1; i < lane_count; i += 1, lane_arena_ptr += 1){
      *lane_arena_ptr = syms_arena_alloc();
    }
    group->lane_count = lane_count;
  }
}

SYMS_API void
syms_group_end_multilane(SYMS_Group *group){
  SYMS_U64 lane_count = group->lane_count;
  if (lane_count > 1){
    // absorb the lane arenas
    SYMS_Arena *arena = group->arena;
    SYMS_Arena **lane_arena_ptr = group->lane_arenas + 1;
    for (SYMS_U64 i = 1; i < lane_count; i += 1, lane_arena_ptr += 1){
      syms_arena_absorb(arena, *lane_arena_ptr);
      *lane_arena_ptr = 0;
    }
    
    // drop the lane count
    group->lane_count = 1;
  }
}

SYMS_API SYMS_Arena*
syms_group_get_lane_arena(SYMS_Group *group){
  SYMS_U64 thread_lane = syms_get_lane();
  SYMS_Arena *result = 0;
  if (thread_lane < group->lane_count){
    result = group->lane_arenas[thread_lane];
  }
  return(result);
}

////////////////////////////////
// allen: Syms Group Getters

SYMS_API SYMS_String8
syms_group_bin_data(SYMS_Group *group){
  return(group->bin_data);
}

SYMS_API SYMS_BinAccel*
syms_group_bin(SYMS_Group *group){
  return(group->bin);
}

SYMS_API SYMS_String8
syms_group_dbg_data(SYMS_Group *group){
  return(group->dbg_data);
}

SYMS_API SYMS_DbgAccel*
syms_group_dbg(SYMS_Group *group){
  return(group->dbg);
}

SYMS_API SYMS_Arch
syms_group_arch(SYMS_Group *group){
  return(group->arch);
}

SYMS_API SYMS_UnitSetAccel*
syms_group_unit_set(SYMS_Group *group){
  return(group->unit_set);
}

SYMS_API SYMS_SecInfoArray
syms_group_sec_info_array(SYMS_Group *group){
  SYMS_SecInfoArray result = group->sec_info_array;
  return(result);
}

SYMS_API SYMS_SecInfo*
syms_group_sec_info_from_number(SYMS_Group *group, SYMS_U64 n){
  SYMS_ProfBegin("syms_group_sec_info_from_number");
  SYMS_SecInfoArray array = group->sec_info_array;
  SYMS_SecInfo *result = 0;
  if (1 <= n && n <= array.count){
    result = &array.sec_info[n - 1];
  }
  SYMS_ProfEnd();
  return(result);
}

SYMS_API SYMS_U64
syms_group_unit_count(SYMS_Group *group){
  return(group->unit_count);
}

SYMS_API SYMS_UnitInfo
syms_group_unit_info_from_uid(SYMS_Group *group, SYMS_UnitID uid){
  SYMS_UnitInfo result = syms_unit_info_from_uid(group->unit_set, uid);
  return(result);
}

SYMS_API SYMS_UnitNames
syms_group_unit_names_from_uid(SYMS_Arena *arena, SYMS_Group *group, SYMS_UnitID uid){
  SYMS_UnitNames result = syms_unit_names_from_uid(arena, group->unit_set, uid);
  return(result);
}

SYMS_API SYMS_UnitAccel*
syms_group_unit_from_uid(SYMS_Group *group, SYMS_UnitID uid){
  SYMS_ProfBegin("syms_group_unit_from_uid");
  SYMS_Arena *arena = syms_group_get_lane_arena(group);
  SYMS_UnitAccel *result = (SYMS_UnitAccel*)&syms_format_nil;
  if (1 <= uid && uid <= group->unit_count){
    SYMS_U64 index = uid - 1;
    result = group->units[index];
    if (result == 0){
      SYMS_UnitAccel *unit = syms_unit_accel_from_uid(arena, group->dbg_data, group->dbg, group->unit_set, uid);
      group->units[index] = unit;
      result = unit;
    }
  }
  SYMS_ProfEnd();
  return(result);
}

SYMS_API SYMS_SymbolIDArray*
syms_group_proc_sid_array_from_uid(SYMS_Group *group, SYMS_UnitID uid){
  SYMS_ProfBegin("syms_group_proc_sid_array_from_uid");
  SYMS_SymbolIDArray *result = &syms_sid_array_nil;
  if (1 <= uid && uid <= group->unit_count){
    SYMS_U64 index = uid - 1;
    if ((group->unit_cache_flags[index] & SYMS_GroupUnitCacheFlag_HasProcSidArray) == 0){
      SYMS_UnitAccel *unit = syms_group_unit_from_uid(group, uid);
      SYMS_Arena *arena = syms_group_get_lane_arena(group);
      group->proc_sid_arrays[index] = syms_proc_sid_array_from_unit(arena, unit);
      group->unit_cache_flags[index] |= SYMS_GroupUnitCacheFlag_HasProcSidArray;
    }
    result = &group->proc_sid_arrays[index];
  }
  SYMS_ProfEnd();
  return(result);
}

SYMS_API SYMS_SymbolIDArray*
syms_group_var_sid_array_from_uid(SYMS_Group *group, SYMS_UnitID uid){
  SYMS_ProfBegin("syms_group_var_sid_array_from_uid");
  SYMS_SymbolIDArray *result = &syms_sid_array_nil;
  if (1 <= uid && uid <= group->unit_count){
    SYMS_U64 index = uid - 1;
    if ((group->unit_cache_flags[index] & SYMS_GroupUnitCacheFlag_HasVarSidArray) == 0){
      SYMS_UnitAccel *unit = syms_group_unit_from_uid(group, uid);
      SYMS_Arena *arena = syms_group_get_lane_arena(group);
      group->var_sid_arrays[index] = syms_var_sid_array_from_unit(arena, unit);
      group->unit_cache_flags[index] |= SYMS_GroupUnitCacheFlag_HasVarSidArray;
    }
    result = &group->var_sid_arrays[index];
  }
  SYMS_ProfEnd();
  return(result);
}

SYMS_API SYMS_SymbolIDArray*
syms_group_tls_var_sid_array_from_uid(SYMS_Group *group, SYMS_UnitID uid){
  SYMS_ProfBegin("syms_group_tls_var_sid_array_from_uid");
  SYMS_SymbolIDArray *result = &syms_sid_array_nil;
  if (1 <= uid && uid < group->unit_count){
    SYMS_U64 index = uid - 1;
    if ((group->unit_cache_flags[index] & SYMS_GroupUnitCacheFlag_HasTlsVarSidArray) == 0){
      SYMS_UnitAccel *unit = syms_group_unit_from_uid(group, uid);
      SYMS_Arena *arena = syms_group_get_lane_arena(group);
      group->thread_sid_arrays[index] = syms_tls_var_sid_array_from_unit(arena, unit);
      group->unit_cache_flags[index] |= SYMS_GroupUnitCacheFlag_HasTlsVarSidArray;
    }
    result = &group->thread_sid_arrays[index];
  }
  SYMS_ProfEnd();
  return(result);
}

SYMS_API SYMS_SymbolIDArray*
syms_group_type_sid_array_from_uid(SYMS_Group *group, SYMS_UnitID uid){
  SYMS_ProfBegin("syms_group_type_sid_array_from_uid");
  SYMS_SymbolIDArray *result = &syms_sid_array_nil;
  if (1 <= uid && uid <= group->unit_count){
    SYMS_U64 index = uid - 1;
    if ((group->unit_cache_flags[index] & SYMS_GroupUnitCacheFlag_HasTypeSidArray) == 0){
      SYMS_UnitAccel *unit = syms_group_unit_from_uid(group, uid);
      SYMS_Arena *arena = syms_group_get_lane_arena(group);
      group->type_sid_arrays[index] = syms_type_sid_array_from_unit(arena, unit);
      group->unit_cache_flags[index] |= SYMS_GroupUnitCacheFlag_HasTypeSidArray;
    }
    result = &group->type_sid_arrays[index];
  }
  SYMS_ProfEnd();
  return(result);
}

SYMS_API SYMS_String8Array*
syms_group_file_table_from_uid(SYMS_Group *group, SYMS_UnitID uid){
  SYMS_ProfBegin("syms_group_file_table_from_uid");
  SYMS_String8Array *result = &syms_string_array_nil;
  if (1 <= uid && uid <= group->unit_count){
    SYMS_U64 index = uid - 1;
    if ((group->unit_cache_flags[index] & SYMS_GroupUnitCacheFlag_HasFileTable) == 0){
      SYMS_Arena *arena = syms_group_get_lane_arena(group);
      group->file_tables[index] = syms_file_table_from_uid(arena, group->dbg_data, group->dbg,
                                                           group->unit_set, uid);
      group->unit_cache_flags[index] |= SYMS_GroupUnitCacheFlag_HasFileTable;
    }
    result = &group->file_tables[index];
  }
  SYMS_ProfEnd();
  return(result);
}

SYMS_API SYMS_LineParseOut*
syms_group_line_parse_from_uid(SYMS_Group *group, SYMS_UnitID uid){
  SYMS_ProfBegin("syms_group_line_parse_from_uid");
  SYMS_LineParseOut* result = &syms_line_parse_nil;
  if (1 <= uid && uid <= group->unit_count){
    SYMS_U64 index = uid - 1;
    if ((group->unit_cache_flags[index] & SYMS_GroupUnitCacheFlag_HasLineTable) == 0){
      SYMS_Arena *arena = syms_group_get_lane_arena(group);
      group->line_tables[index] = syms_line_parse_from_uid(arena, group->dbg_data, group->dbg, group->unit_set, uid);
      group->unit_cache_flags[index] |= SYMS_GroupUnitCacheFlag_HasLineTable;
    }
    result = &group->line_tables[index];
  }
  SYMS_ProfEnd();
  return(result);
}

SYMS_API SYMS_LineTable*
syms_group_line_table_from_uid(SYMS_Group *group, SYMS_UnitID uid){
  SYMS_ProfBegin("syms_group_line_table_from_uid");
  SYMS_LineParseOut *parse = syms_group_line_parse_from_uid(group, uid);
  SYMS_LineTable *result = &parse->line_table;
  SYMS_ProfEnd();
  return(result);
}

SYMS_API SYMS_String8Array*
syms_group_inferred_file_table_from_uid(SYMS_Group *group, SYMS_UnitID uid){
  SYMS_ProfBegin("syms_group_inferred_file_table_from_uid");
  SYMS_String8Array *result = &syms_string_array_nil;
  if (1 <= uid && uid <= group->unit_count){
    SYMS_U32 index = uid - 1;
    if ((group->unit_cache_flags[index] & SYMS_GroupUnitCacheFlag_HasInfFileTable) == 0){
      // get the line parse
      SYMS_LineParseOut *line_parse = syms_group_line_parse_from_uid(group, uid);
      
      if (line_parse->file_table.count != 0){
        result = &line_parse->file_table;
      }
      else{
        // setup
        SYMS_Arena *arena = syms_group_get_lane_arena(group);
        SYMS_String8 dbg_data = group->dbg_data;
        SYMS_DbgAccel *dbg = group->dbg;
        SYMS_UnitSetAccel *unit_set = group->unit_set;
        
        // infer names
        SYMS_U64 count = line_parse->file_id_array.count;
        SYMS_String8 *names = syms_push_array(arena, SYMS_String8, count);
        {
          SYMS_FileID *file_id_ptr = line_parse->file_id_array.ids;
          SYMS_String8 *name_ptr = names;
          for (SYMS_FileID *opl = file_id_ptr + count;
               file_id_ptr < opl;
               file_id_ptr += 1, name_ptr += 1){
            *name_ptr = syms_file_name_from_id(arena, dbg_data, dbg, unit_set, uid, *file_id_ptr);
          }
        }
        
        // assemble file table
        SYMS_String8Array file_table = {0};
        file_table.strings = names;
        file_table.count = count;
        
        // fill cache slot
        group->inferred_file_tables[index] = file_table;
        group->unit_cache_flags[index] |= SYMS_GroupUnitCacheFlag_HasInfFileTable;
      }
    }
    result = &group->inferred_file_tables[index];
  }
  SYMS_ProfEnd();
  return(result);
}

SYMS_API SYMS_String8Array*
syms_group_file_table_from_uid_with_fallbacks(SYMS_Group *group, SYMS_UnitID uid){
  SYMS_ProfBegin("syms_group_file_table_from_uid_with_fallbacks");
  SYMS_String8Array *result = syms_group_file_table_from_uid(group, uid);
  if (result != 0 && result->count == 0){
    result = syms_group_inferred_file_table_from_uid(group, uid);
  }
  SYMS_ProfEnd();
  return(result);
}

SYMS_API SYMS_UnitRangeArray
syms_group_unit_ranges(SYMS_Group *group){
  if (!group->unit_ranges_is_filled){
    SYMS_ASSERT_PARANOID(syms_thread_lane == 0);
    group->unit_ranges = syms_unit_ranges_from_set(group->arena, group->dbg_data, group->dbg, group->unit_set);
    group->unit_ranges_is_filled = syms_true;
  }
  SYMS_UnitRangeArray result = group->unit_ranges;
  return(result);
}

SYMS_API SYMS_SymbolKind
syms_group_symbol_kind_from_sid(SYMS_Group *group, SYMS_UnitAccel *unit, SYMS_SymbolID sid){
  SYMS_SymbolKind result = syms_symbol_kind_from_sid(group->dbg_data, group->dbg, unit, sid);
  return(result);
}

SYMS_API SYMS_String8
syms_group_symbol_name_from_sid(SYMS_Arena *arena, SYMS_Group *group, SYMS_UnitAccel *unit, SYMS_SymbolID sid){
  SYMS_String8 result = syms_symbol_name_from_sid(arena, group->dbg_data, group->dbg, unit, sid);
  return(result);
}

SYMS_API SYMS_U64
syms_group_address_size(SYMS_Group *group){
  SYMS_U64 result = group->address_size;
  return(result);
}

SYMS_API SYMS_U64
syms_group_default_vbase(SYMS_Group *group){
  SYMS_U64 result = group->default_vbase;
  return(result);
}

SYMS_API SYMS_String8
syms_group_file_name_from_id(SYMS_Group *group, SYMS_UnitID uid, SYMS_FileID file_id){
  SYMS_ASSERT_PARANOID(syms_thread_lane == 0);
  SYMS_ProfBegin("syms_group_file_name_from_id");
  SYMS_String8 from_map = syms_file_id_2_name_map_name_from_id(&group->file_id_2_name_map, uid, file_id);
  SYMS_String8 result = {0};
  if (from_map.size != 0){
    if (from_map.str != 0){
      result = from_map;
    }
  }
  else{
    SYMS_ArenaTemp scratch = syms_get_scratch(0, 0);
    SYMS_String8 file_name = syms_file_name_from_id(scratch.arena, group->dbg_data, group->dbg,
                                                    group->unit_set, uid, file_id);
    result = syms_string_cons(group->arena, &group->string_cons, file_name);
    syms_release_scratch(scratch);
    syms_file_id_2_name_map_insert(group->arena, &group->file_id_2_name_map, uid, file_id, result);
  }
  SYMS_ProfEnd();
  return(result);
}

SYMS_API SYMS_MapAndUnit*
syms_group_type_map(SYMS_Group *group){
  if (!group->type_map_unit_is_filled){
    SYMS_UnitID type_map_uid = syms_partner_uid_from_map(group->type_mau.map);
    SYMS_UnitAccel *type_map_unit = syms_group_unit_from_uid(group, type_map_uid);
    group->type_mau.unit = type_map_unit;
    group->type_map_unit_is_filled = syms_true;
  }
  SYMS_MapAndUnit *result = &group->type_mau;
  return(result);
}

SYMS_API SYMS_MapAndUnit*
syms_group_unmangled_symbol_map(SYMS_Group *group){
  if (!group->symbol_map_unit_is_filled){
    SYMS_UnitID symbol_map_uid = syms_partner_uid_from_map(group->symbol_mau.map);
    SYMS_UnitAccel *symbol_map_unit = syms_group_unit_from_uid(group, symbol_map_uid);
    group->symbol_mau.unit = symbol_map_unit;
    group->symbol_map_unit_is_filled = syms_true;
  }
  SYMS_MapAndUnit *result = &group->symbol_mau;
  return(result);
}

SYMS_API SYMS_LinkNameRecArray
syms_group_link_name_records(SYMS_Group *group){
  if (!group->link_name_record_array_is_filled){
    SYMS_UnitID link_uid = syms_link_names_uid(group->dbg);
    SYMS_UnitAccel *unit = syms_group_unit_from_uid(group, link_uid);
    SYMS_LinkNameRecArray array = syms_link_name_array_from_unit(group->arena, group->dbg_data, group->dbg, unit);
    group->link_name_record_array = array;
    
    // sort
    syms_group__link_names_sort_in_place(array.recs, array.count);
    
    group->link_name_record_array_is_filled = syms_true;
  }
  SYMS_LinkNameRecArray result = group->link_name_record_array;
  return(result);
}

SYMS_API SYMS_LinkMapAccel*
syms_group_link_name_map(SYMS_Group *group){
  if (!group->link_map_is_built){
    group->link_map = syms_link_map_from_dbg(group->arena, group->dbg_data, group->dbg);
    group->link_map_is_built = !group->link_map_is_built;
  }
  SYMS_LinkMapAccel *result = group->link_map;
  return(result);
}

SYMS_API SYMS_UnitAccel*
syms_group_link_name_unit(SYMS_Group *group){
  if (!group->link_name_unit_is_filled){
    SYMS_UnitID uid = syms_link_names_uid(group->dbg);
    group->link_name_unit = syms_group_unit_from_uid(group, uid);
    group->link_name_unit_is_filled = syms_true;
  }
  SYMS_UnitAccel *result = group->link_name_unit;
  return(result);
}

SYMS_API void
syms_group__link_names_sort_in_place(SYMS_LinkNameRec *recs, SYMS_U64 count){
  SYMS_ArenaTemp scratch = syms_get_scratch(0, 0);
  
  // setup stack with full range
  SYMS_SortNode *stack = 0;
  SYMS_SortNode *free_stack = 0;
  syms_sort_node_push(scratch.arena, &stack, &free_stack, 0, count);
  
  // sort loop
  for (; stack != 0;){
    SYMS_SortNode *node = stack;
    SYMS_StackPop(stack);
    
    SYMS_U64 first = node->first;
    SYMS_U64 opl = node->opl;
    
    SYMS_StackPush(free_stack, node);
    
    SYMS_U64 node_count = (opl - first);
    if (node_count > 2){
      SYMS_U64 last = opl - 1;
      
      // swap key to last
      SYMS_U64 mid = (first + opl)/2;
      SYMS_Swap(SYMS_LinkNameRec, recs[mid], recs[last]);
      
      // partition
      SYMS_B32 equal_send_left = syms_false;
      SYMS_U64 key = recs[last].voff;
      SYMS_U64 j = first;
      for (SYMS_U64 i = first; i < last; i += 1){
        SYMS_B32 send_left = (recs[i].voff < key);
        if (!send_left && recs[i].voff == key){
          send_left = equal_send_left;
          equal_send_left = !equal_send_left;
        }
        if (send_left){
          if (j != i){
            SYMS_Swap(SYMS_LinkNameRec, recs[i], recs[j]);
          }
          j += 1;
        }
      }
      
      // swap last to pivot
      SYMS_Swap(SYMS_LinkNameRec, recs[j], recs[last]);
      
      // recurse
      SYMS_U64 pivot = j;
      if (pivot - first > 1){
        syms_sort_node_push(scratch.arena, &stack, &free_stack, first, pivot);
      }
      if (opl - (pivot + 1) > 1){
        syms_sort_node_push(scratch.arena, &stack, &free_stack, pivot + 1, opl);
      }
    }
    else if (node_count == 2){
      if (recs[first].voff > recs[first + 1].voff){
        SYMS_Swap(SYMS_LinkNameRec, recs[first], recs[first + 1]);
      }
    }
  }
  
  syms_release_scratch(scratch);
}


////////////////////////////////
// allen: Syms Group Address Mapping Functions

SYMS_API SYMS_U64
syms_group_sec_number_from_voff__linear_scan(SYMS_Group *group, SYMS_U64 voff){
  SYMS_ProfBegin("syms_group_sec_number_from_voff__linear_scan");
  SYMS_U64 result = 0;
  SYMS_SecInfoArray sec_info_array = syms_group_sec_info_array(group);
  SYMS_SecInfo *sec_info = sec_info_array.sec_info;
  for (SYMS_U64 i = 0; i < sec_info_array.count; i += 1, sec_info += 1){
    if (sec_info->vrange.min <= voff && voff < sec_info->vrange.max){
      result = i + 1;
      break;
    }
  }
  SYMS_ProfEnd();
  return(result);
}

SYMS_API SYMS_U64
syms_group_sec_number_from_foff__linear_scan(SYMS_Group *group, SYMS_U64 foff){
  SYMS_ProfBegin("syms_group_sec_number_from_foff__linear_scan");
  SYMS_U64 result = 0;
  SYMS_SecInfoArray sec_info_array = syms_group_sec_info_array(group);
  SYMS_SecInfo *sec_info = sec_info_array.sec_info;
  for (SYMS_U64 i = 0; i < sec_info_array.count; i += 1, sec_info += 1){
    if (sec_info->frange.min <= foff && foff < sec_info->frange.max){
      result = i + 1;
      break;
    }
  }
  SYMS_ProfEnd();
  return(result);
}

SYMS_API SYMS_U64Maybe
syms_group_voff_from_foff__linear_scan(SYMS_Group *group, SYMS_U64 foff){
  SYMS_ProfBegin("syms_group_voff_from_foff__linear_scan");
  SYMS_U64Maybe result = {0};
  SYMS_U64 sec_number = syms_group_sec_number_from_foff__linear_scan(group, foff);
  SYMS_SecInfo *info = syms_group_sec_info_from_number(group, sec_number);
  if (info != 0){
    result.valid = syms_true;
    result.u64 = foff - info->frange.min + info->vrange.min;
  }
  SYMS_ProfEnd();
  return(result);
}

SYMS_API SYMS_U64Maybe
syms_group_foff_from_voff__linear_scan(SYMS_Group *group, SYMS_U64 voff){
  SYMS_ProfBegin("syms_group_foff_from_voff__linear_scan");
  SYMS_U64Maybe result = {0};
  SYMS_U64 sec_number = syms_group_sec_number_from_voff__linear_scan(group, voff);
  SYMS_SecInfo *info = syms_group_sec_info_from_number(group, sec_number);
  if (info != 0){
    result.valid = syms_true;
    result.u64 = voff - info->vrange.min + info->frange.min;
  }
  SYMS_ProfEnd();
  return(result);
}

SYMS_API SYMS_UnitID
syms_group_uid_from_voff__linear_scan(SYMS_Group *group, SYMS_U64 virt_off){
  SYMS_ProfBegin("syms_group_uid_from_voff__linear_scan");
  SYMS_UnitID result = 0;
  SYMS_UnitRangeArray unit_ranges = syms_group_unit_ranges(group);
  SYMS_U64 range_count = unit_ranges.count;
  SYMS_UnitRange *unit_range = unit_ranges.ranges;
  for (SYMS_U64 i = 0; i < range_count; i += 1, unit_range += 1){
    if (unit_range->vrange.min <= virt_off && virt_off < unit_range->vrange.max){
      result = unit_range->uid;
      break;
    }
  }
  SYMS_ProfEnd();
  return(result);
}

SYMS_API SYMS_SymbolID
syms_group_proc_sid_from_uid_voff__linear_scan(SYMS_Group *group, SYMS_UnitID uid, SYMS_U64 voff){
  SYMS_ProfBegin("syms_group_proc_sid_from_uid_voff__linear_scan");
  SYMS_ArenaTemp scratch = syms_get_scratch(0, 0);
  
  SYMS_String8 dbg_data = group->dbg_data;
  SYMS_DbgAccel *dbg = group->dbg;
  SYMS_UnitAccel *unit = syms_group_unit_from_uid(group, uid);
  SYMS_SymbolIDArray *sid_array = syms_group_proc_sid_array_from_uid(group, uid);
  SYMS_U64 count = sid_array->count;
  
  SYMS_SymbolID result = {0};
  SYMS_SymbolID *sid_ptr = sid_array->ids;
  for (SYMS_U64 i = 0; i < count; i += 1, sid_ptr += 1){
    SYMS_SymbolID sid = *sid_ptr;
    SYMS_SymbolKind kind = syms_group_symbol_kind_from_sid(group, unit, sid);
    SYMS_B32 hit = syms_false;
    if (kind == SYMS_SymbolKind_Procedure){
      SYMS_U64RangeArray ranges = syms_scope_vranges_from_sid(scratch.arena, dbg_data, dbg, unit, sid);
      SYMS_U64Range *range = ranges.ranges;
      for (SYMS_U64 i = 0; i < ranges.count; i += 1, range += 1){
        if (range->min <= voff && voff < range->max){
          hit = syms_true;
          break;
        }
      }
      syms_arena_temp_end(scratch);
    }
    if (hit){
      result = sid;
      break;
    }
  }
  
  syms_release_scratch(scratch);
  SYMS_ProfEnd();
  return(result);
}

SYMS_API SYMS_Line
syms_group_line_from_uid_voff__linear_scan(SYMS_Group *group, SYMS_UnitID uid, SYMS_U64 voff){
  SYMS_ProfBegin("syms_group_line_from_uid_voff__linear_scan");
  SYMS_Line result = {0};
  SYMS_LineTable *line_table = syms_group_line_table_from_uid(group, uid);
  SYMS_U64 *seq_index_ptr = line_table->sequence_index_array;
  SYMS_Line *line_array = line_table->line_array;
  SYMS_U64 seq_count = line_table->sequence_count;
  for (SYMS_U64 i = 0; i < seq_count; i += 1){
    // get sequence range & inc
    SYMS_U64 first = *seq_index_ptr;
    seq_index_ptr += 1;
    SYMS_U64 opl = *seq_index_ptr;
    
    // check sequence range
    SYMS_U64 last = opl - 1;
    SYMS_U64 first_voff = line_array[first].voff;
    SYMS_U64 opl_voff = line_array[last].voff;
    if (first_voff <= voff && voff < opl_voff){
      
      // get the line
      result = syms_line_from_sequence_voff(line_table, i + 1, voff);
    }
  }
  SYMS_ProfEnd();
  return(result);
}

SYMS_API SYMS_U64RangeList
syms_group_vranges_from_uid_line__linear_scan(SYMS_Arena *arena, SYMS_Group *group, SYMS_UnitID uid,
                                              SYMS_FileID file_id, SYMS_U32 line){
  SYMS_ProfBegin("syms_group_vranges_from_uid_line__linear_scan");
  
  SYMS_ASSERT(arena != group->arena);
  SYMS_U64RangeList result = {0};
  SYMS_LineTable *line_table = syms_group_line_table_from_uid(group, uid);
  SYMS_U64 *seq_index_ptr = line_table->sequence_index_array;
  SYMS_Line *line_array = line_table->line_array;
  SYMS_U64 seq_count = line_table->sequence_count;
  for (SYMS_U64 i = 0; i < seq_count; i += 1){
    // get sequence range & inc
    SYMS_U64 first = *seq_index_ptr;
    seq_index_ptr += 1;
    SYMS_U64 opl = *seq_index_ptr;
    
    // iterate lines
    SYMS_U64 last = opl - 1;
    SYMS_Line *line_ptr = line_array + first;
    for (SYMS_U64 j = first; j < last; j += 1, line_ptr += 1){
      if (line_ptr->src_coord.file_id == file_id &&
          line_ptr->src_coord.line == line){
        SYMS_U64Range range = syms_make_u64_range(line_ptr->voff, (line_ptr + 1)->voff);
        syms_u64_range_list_push(arena, &result, range);
      }
    }
  }
  
  SYMS_ProfEnd();
  return(result);
}

//- map getters
SYMS_API SYMS_SpatialMap1D*
syms_group_sec_map_v(SYMS_Group *group){
  SYMS_ProfBegin("syms_group_sec_map_v");
  if (!group->sec_map_v_is_built){
    SYMS_ASSERT_PARANOID(syms_thread_lane == 0);
    
    //- fill spatial map array
    SYMS_SecInfoArray sec_array = syms_group_sec_info_array(group);
    SYMS_SecInfo *sec_info = sec_array.sec_info;
    SYMS_SpatialMap1DRange *ranges = syms_push_array_zero(group->arena, SYMS_SpatialMap1DRange, sec_array.count);
    SYMS_SpatialMap1DRange *range_ptr = ranges;
    for (SYMS_U64 i = 0; i < sec_array.count; i += 1, sec_info += 1){
      if (sec_info->vrange.min < sec_info->vrange.max){
        range_ptr->range = sec_info->vrange;
        range_ptr->val = i + 1;
        range_ptr += 1;
      }
    }
    SYMS_U64 final_count = (SYMS_U64)(range_ptr - ranges);
    syms_arena_put_back(group->arena, sizeof(SYMS_SpatialMap1DRange)*(sec_array.count - final_count));
    
    //- sort
    if (!syms_spatial_map_1d_array_check_sorted(ranges, final_count)){
      syms_spatial_map_1d_array_sort(ranges, final_count);
    }
    
    //- assemble map
    SYMS_SpatialMap1D map = {ranges, final_count};
    
    //- save to group
    group->sec_map_v_is_built = syms_true;
    group->sec_map_v = map;
    
    SYMS_ASSERT_PARANOID(syms_spatial_map_1d_invariants(&map));
  }
  
  SYMS_SpatialMap1D *result = &group->sec_map_v;
  
  SYMS_ProfEnd();
  return(result);
}

SYMS_API SYMS_SpatialMap1D*
syms_group_sec_map_f(SYMS_Group *group){
  SYMS_ProfBegin("syms_group_sec_map_f");
  if (!group->sec_map_f_is_built){
    SYMS_ASSERT_PARANOID(syms_thread_lane == 0);
    
    //- fill spatial map array
    SYMS_SecInfoArray sec_array = syms_group_sec_info_array(group);
    SYMS_SecInfo *sec_info = sec_array.sec_info;
    SYMS_SpatialMap1DRange *ranges = syms_push_array_zero(group->arena, SYMS_SpatialMap1DRange, sec_array.count);
    SYMS_SpatialMap1DRange *range_ptr = ranges;
    for (SYMS_U64 i = 0; i < sec_array.count; i += 1, sec_info += 1){
      if (sec_info->frange.min < sec_info->frange.max){
        range_ptr->range = sec_info->frange;
        range_ptr->val = i + 1;
        range_ptr += 1;
      }
    }
    SYMS_U64 final_count = (SYMS_U64)(range_ptr - ranges);
    syms_arena_put_back(group->arena, sizeof(SYMS_SpatialMap1DRange)*(sec_array.count - final_count));
    
    //- sort
    if (!syms_spatial_map_1d_array_check_sorted(ranges, final_count)){
      syms_spatial_map_1d_array_sort(ranges, final_count);
    }
    
    //- assemble map
    SYMS_SpatialMap1D map = {ranges, final_count};
    
    //- save to group
    group->sec_map_f_is_built = syms_true;
    group->sec_map_f = map;
    
    SYMS_ASSERT_PARANOID(syms_spatial_map_1d_invariants(&map));
  }
  
  SYMS_SpatialMap1D *result = &group->sec_map_v;
  
  SYMS_ProfEnd();
  return(result);
}

SYMS_API SYMS_SpatialMap1D*
syms_group_unit_map(SYMS_Group *group){
  SYMS_ProfBegin("syms_group_unit_map");
  if (!group->unit_map_is_built){
    SYMS_ASSERT_PARANOID(syms_thread_lane == 0);
    
    //- fill spatial map array
    SYMS_UnitRangeArray unit_ranges = syms_group_unit_ranges(group);
    SYMS_U64 count = unit_ranges.count;
    SYMS_UnitRange *unit_range = unit_ranges.ranges;
    SYMS_SpatialMap1DRange *ranges = syms_push_array_zero(group->arena, SYMS_SpatialMap1DRange, count);
    SYMS_SpatialMap1DRange *range_ptr = ranges;
    for (SYMS_U64 i = 0; i < count; i += 1, unit_range += 1){
      if (unit_range->vrange.min < unit_range->vrange.max){
        range_ptr->range = unit_range->vrange;
        range_ptr->val = unit_range->uid;
        range_ptr += 1;
      }
    }
    SYMS_U64 final_count = (SYMS_U64)(range_ptr - ranges);
    syms_arena_put_back(group->arena, sizeof(SYMS_SpatialMap1DRange)*(count - final_count));
    
    //- sort
    if (!syms_spatial_map_1d_array_check_sorted(ranges, final_count)){
      syms_spatial_map_1d_array_sort(ranges, final_count);
    }
    
    //- assemble map
    SYMS_SpatialMap1D map = {ranges, final_count};
    
    //- save to group
    group->unit_map_is_built = syms_true;
    group->unit_map = map;
    
    SYMS_ASSERT_PARANOID(syms_spatial_map_1d_invariants(&map));
  }
  
  SYMS_SpatialMap1D *result = &group->unit_map;
  
  SYMS_ProfEnd();
  return(result);
}

SYMS_API SYMS_SpatialMap1D*
syms_group_proc_map_from_uid(SYMS_Group *group, SYMS_UnitID uid){
  SYMS_ProfBegin("syms_group_proc_map_from_uid");
  SYMS_U64 index = uid - 1;
  if (index < group->unit_count &&
      (group->unit_cache_flags[index] & SYMS_GroupUnitCacheFlag_HasProcMap) == 0){
    SYMS_Arena *arena = syms_group_get_lane_arena(group);
    SYMS_String8 dbg_data = group->dbg_data;
    SYMS_DbgAccel *dbg = group->dbg;
    SYMS_UnitAccel *unit = syms_group_unit_from_uid(group, uid);
    SYMS_SymbolIDArray *sid_array = syms_group_proc_sid_array_from_uid(group, uid);
    
    SYMS_SpatialMap1D map = syms_spatial_map_for_procs_from_sid_array(arena, dbg_data, dbg,
                                                                      unit, sid_array);
    
    SYMS_ASSERT_PARANOID(syms_spatial_map_1d_invariants(&map));
    
    //- save to group
    group->unit_cache_flags[index] |= SYMS_GroupUnitCacheFlag_HasProcMap;
    group->unit_proc_maps[index] = map;
  }
  
  SYMS_SpatialMap1D *result = &syms_spatial_map_1d_nil;
  if (index < group->unit_count){
    result = &group->unit_proc_maps[index];
  }
  
  SYMS_ProfEnd();
  return(result);
}

SYMS_API SYMS_SpatialMap1D*
syms_group_line_sequence_map_from_uid(SYMS_Group *group, SYMS_UnitID uid){
  SYMS_ProfBegin("syms_group_line_sequence_map_from_uid");
  SYMS_U64 index = uid - 1;
  if (index < group->unit_count &&
      (group->unit_cache_flags[index] & SYMS_GroupUnitCacheFlag_HasLineSeqMap) == 0){
    //- fill spatial map array
    SYMS_Arena *arena = syms_group_get_lane_arena(group);
    SYMS_LineTable *line_table = syms_group_line_table_from_uid(group, uid);
    SYMS_SpatialMap1D map = syms_spatial_map_from_line_table(arena, line_table);
    
    //- save to group
    group->unit_cache_flags[index] |= SYMS_GroupUnitCacheFlag_HasLineSeqMap;
    group->line_sequence_maps[index] = map;
    
    SYMS_ASSERT_PARANOID(syms_spatial_map_1d_invariants(&map));
  }
  
  SYMS_SpatialMap1D *result = &syms_spatial_map_1d_nil;
  if (index < group->unit_count){
    result = &group->line_sequence_maps[index];
  }
  
  SYMS_ProfEnd();
  return(result);
}

SYMS_API void
syms_group_fetch_line_to_addr_maps_from_uid(SYMS_Group *group, SYMS_UnitID uid){
  SYMS_ProfBegin("syms_group_fetch_line_to_addr_maps_from_uid");
  
  SYMS_U64 index = uid - 1;
  if (index < group->unit_count &&
      (group->unit_cache_flags[index] & SYMS_GroupUnitCacheFlag_HasLineToAddrMap) == 0){
    //- get buckets
    SYMS_LineTable *line_table = syms_group_line_table_from_uid(group, uid);
    SYMS_Arena *arena = syms_group_get_lane_arena(group);
    SYMS_FileToLineToAddrMap map = syms_line_to_addr_map_from_line_table(arena, line_table);
    
    //- save to group
    group->unit_cache_flags[index] |= SYMS_GroupUnitCacheFlag_HasLineToAddrMap;
    group->file_to_line_to_addr_maps[index] = map;
  }
  
  SYMS_ProfEnd();
}

SYMS_API SYMS_LineToAddrMap*
syms_group_line_to_addr_map_from_uid_file_id(SYMS_Group *group, SYMS_UnitID uid, SYMS_FileID file_id){
  SYMS_ProfBegin("syms_group_line_to_addr_map_from_uid_file_id");
  syms_group_fetch_line_to_addr_maps_from_uid(group, uid);
  
  //- get line-to-addr map from file_id
  SYMS_LineToAddrMap *result = &syms_line_to_addr_map_nil;
  SYMS_U64 index = uid - 1;
  if (index < group->unit_count){
    SYMS_FileToLineToAddrMap *buckets = &group->file_to_line_to_addr_maps[index];
    if (buckets->bucket_count > 0){
      result = syms_line_to_addr_map_lookup_file_id(buckets, file_id);
    }
  }
  
  SYMS_ProfEnd();
  return(result);
}

SYMS_API SYMS_SymbolNameMap*
syms_group_type_map_from_uid(SYMS_Group *group, SYMS_UnitID uid){
  SYMS_ProfBegin("syms_group_type_map_from_uid");
  
  SYMS_SymbolNameMap *result = 0;
  
  SYMS_U64 index = uid - 1;
  if (index < group->unit_count){
    if ((group->unit_cache_flags[index] & SYMS_GroupUnitCacheFlag_HasTypeNameMap) == 0){
      SYMS_ArenaTemp scratch = syms_get_scratch(0, 0);
      
      // grab group members
      SYMS_String8 dbg_data = group->dbg_data;
      SYMS_DbgAccel *dbg = group->dbg;
      
      // grab unit
      SYMS_UnitAccel *unit = syms_group_unit_from_uid(group, uid);
      
      // assemble loose map
      SYMS_SymbolNameMapLoose loose = {0};
      SYMS_SymbolIDArray *type_array = syms_group_type_sid_array_from_uid(group, uid);
      SYMS_SymbolID *sid_opl = type_array->ids + type_array->count;
      SYMS_SymbolID *sid_ptr = type_array->ids;
      for (;sid_ptr < sid_opl; sid_ptr += 1){
        SYMS_String8 name = syms_symbol_name_from_sid(scratch.arena, dbg_data, dbg, unit, *sid_ptr);
        syms_symbol_name_map_push(scratch.arena, &loose, name, *sid_ptr);
      }
      
      //- bake the map
      group->unit_type_maps[index] = syms_symbol_name_map_bake(group->arena, &loose);
      group->unit_cache_flags[index] |= SYMS_GroupUnitCacheFlag_HasTypeNameMap;
      
      syms_release_scratch(scratch);
    }
    
    result = &group->unit_type_maps[index];
  }
  
  SYMS_ProfEnd();
  return(result);
}


SYMS_API SYMS_SpatialMap1D*
syms_group_link_name_spatial_map(SYMS_Group *group){
  SYMS_ASSERT_PARANOID(syms_thread_lane == 0);
  SYMS_ProfBegin("syms_group_link_name_spatial_map");
  if (!group->link_name_spatial_map_is_built){
    SYMS_Arena *arena = group->arena;
    
    //- fill spatial map array
    SYMS_LinkNameRecArray link_name_rec_array = syms_group_link_name_records(group);
    SYMS_U64 guess_count = link_name_rec_array.count;
    SYMS_SpatialMap1DRange *ranges = syms_push_array(arena, SYMS_SpatialMap1DRange, guess_count);
    SYMS_SpatialMap1DRange *range_ptr = ranges;
    SYMS_LinkNameRec *rec_ptr = link_name_rec_array.recs;
    for (SYMS_U64 i = 0; i < guess_count; i += 1, rec_ptr += 1){
      SYMS_U64 min = rec_ptr->voff;
      SYMS_U64 max = SYMS_U64_MAX;
      if (i + 1 < guess_count){
        max = (rec_ptr + 1)->voff;
      }
      SYMS_ASSERT(min <= max);
      if (min < max){
        range_ptr->range.min = min;
        range_ptr->range.max = max;
        range_ptr->val = i + 1;
        range_ptr += 1;
      }
    }
    
    //- shrink the allocated array
    SYMS_U64 actual_count = (SYMS_U64)(range_ptr - ranges);
    syms_arena_put_back(arena, sizeof(SYMS_SpatialMap1D)*(guess_count - actual_count));
    
    //- assemble map
    SYMS_SpatialMap1D map = {ranges, actual_count};
    
    //- save to group
    group->link_name_spatial_map_is_built = syms_true;
    group->link_name_spatial_map = map;
    
    SYMS_ASSERT_PARANOID(syms_spatial_map_1d_invariants(&map));
  }
  
  SYMS_SpatialMap1D *result = &group->link_name_spatial_map;
  SYMS_ProfEnd();
  return(result);
}

//- accelerated versions
SYMS_API SYMS_U64
syms_group_sec_number_from_voff__accelerated(SYMS_Group *group, SYMS_U64 voff){
  SYMS_ProfBegin("syms_group_sec_number_from_voff__accelerated");
  SYMS_SpatialMap1D *map = syms_group_sec_map_v(group);
  SYMS_U64 result = syms_spatial_map_1d_value_from_point(map, voff);
  SYMS_ProfEnd();
  return(result);
}

SYMS_API SYMS_U64
syms_group_sec_number_from_foff__accelerated(SYMS_Group *group, SYMS_U64 foff){
  SYMS_ProfBegin("syms_group_sec_number_from_foff__accelerated");
  SYMS_SpatialMap1D *map = syms_group_sec_map_f(group);
  SYMS_U64 result = syms_spatial_map_1d_value_from_point(map, foff);
  SYMS_ProfEnd();
  return(result);
}

SYMS_API SYMS_U64Maybe
syms_group_sec_voff_from_foff__accelerated(SYMS_Group *group, SYMS_U64 foff){
  SYMS_ProfBegin("syms_group_sec_voff_from_foff__accelerated");
  SYMS_U64Maybe result = {0};
  SYMS_U64 sec_number = syms_group_sec_number_from_foff__accelerated(group, foff);
  SYMS_SecInfo *info = syms_group_sec_info_from_number(group, sec_number);
  if (info != 0){
    result.valid = syms_true;
    result.u64 = foff - info->frange.min + info->vrange.min;
  }
  SYMS_ProfEnd();
  return(result);
}

SYMS_API SYMS_U64Maybe
syms_group_sec_foff_from_voff__accelerated(SYMS_Group *group, SYMS_U64 voff){
  SYMS_ProfBegin("syms_group_sec_foff_from_voff__accelerated");
  SYMS_U64Maybe result = {0};
  SYMS_U64 sec_number = syms_group_sec_number_from_voff__accelerated(group, voff);
  SYMS_SecInfo *info = syms_group_sec_info_from_number(group, sec_number);
  if (info != 0){
    result.valid = syms_true;
    result.u64 = voff - info->vrange.min + info->frange.min;
  }
  SYMS_ProfEnd();
  return(result);
}

SYMS_API SYMS_UnitID
syms_group_uid_from_voff__accelerated(SYMS_Group *group, SYMS_U64 voff){
  SYMS_ProfBegin("syms_group_uid_from_voff__accelerated");
  SYMS_SpatialMap1D *map = syms_group_unit_map(group);
  SYMS_UnitID result = syms_spatial_map_1d_value_from_point(map, voff);
  SYMS_ProfEnd();
  return(result);
}

SYMS_API SYMS_SymbolID
syms_group_proc_sid_from_uid_voff__accelerated(SYMS_Group *group, SYMS_UnitID uid, SYMS_U64 voff){
  SYMS_ProfBegin("syms_group_proc_sid_from_uid_voff__accelerated");
  SYMS_SpatialMap1D *map = syms_group_proc_map_from_uid(group, uid);
  SYMS_SymbolID result = syms_spatial_map_1d_value_from_point(map, voff);
  SYMS_ProfEnd();
  return(result);
}

SYMS_API SYMS_Line
syms_group_line_from_uid_voff__accelerated(SYMS_Group *group, SYMS_UnitID uid, SYMS_U64 voff){
  SYMS_ProfBegin("syms_group_line_from_uid_voff__accelerated");
  SYMS_SpatialMap1D *map = syms_group_line_sequence_map_from_uid(group, uid);
  SYMS_U64 sequence_number = syms_spatial_map_1d_value_from_point(map, voff);
  
  SYMS_LineTable *line_table = syms_group_line_table_from_uid(group, uid);
  SYMS_Line result = syms_line_from_sequence_voff(line_table, sequence_number, voff);
  
  SYMS_ProfEnd();
  return(result);
}

SYMS_API SYMS_U64RangeArray
syms_group_vranges_from_uid_line__accelerated(SYMS_Group *group, SYMS_UnitID uid,
                                              SYMS_FileID file_id, SYMS_U32 line){
  SYMS_ProfBegin("syms_group_vranges_from_uid_line__accelerated");
  SYMS_LineToAddrMap *line_map = syms_group_line_to_addr_map_from_uid_file_id(group, uid, file_id);
  
  SYMS_U32 actual_line = 0;
  SYMS_U64RangeArray ranges = syms_line_to_addr_map_lookup_nearest_line_number(line_map, line, &actual_line);
  
  SYMS_U64RangeArray result = {0};
  if (actual_line == line){
    result = ranges;
  }
  
  SYMS_ProfEnd();
  return(result);
}

//- higher level mapping functions

SYMS_API SYMS_ResolvedLine
syms_group_resolved_location_from_proc_sid(SYMS_Group *group, SYMS_UnitAccel *unit, SYMS_SymbolID sid){
  SYMS_ResolvedLine result = {0};
  
  SYMS_ArenaTemp scratch = syms_get_scratch(0, 0);
  SYMS_U64RangeArray ranges = syms_scope_vranges_from_sid(scratch.arena, group->dbg_data, group->dbg, unit, sid);
  if (ranges.count > 0){
    SYMS_UnitID uid = syms_uid_from_unit(unit);
    SYMS_U64 voff = ranges.ranges[0].min;
    SYMS_Line line = syms_group_line_from_uid_voff__accelerated(group, uid, voff);
    if (line.src_coord.file_id != 0){
      SYMS_SrcCoord *src_coord = &line.src_coord;
      SYMS_String8 file_name = syms_group_file_name_from_id(group, uid, src_coord->file_id);
      
      result.file_name = file_name;
      result.line = src_coord->line;
      result.col = src_coord->col;
      result.voff = voff;
    }
  }
  syms_release_scratch(scratch);
  
  return(result);
}


////////////////////////////////
// allen: Syms Group Name Mapping Functions

SYMS_API SYMS_USID
syms_group_usid_from_unmangled_name(SYMS_Group *group, SYMS_String8 name){
  SYMS_USID result = {0};
  SYMS_ArenaTemp scratch = syms_get_scratch(0, 0);
  SYMS_MapAndUnit *map = syms_group_unmangled_symbol_map(group);
  if (syms_accel_is_good(map->map)){
    SYMS_USIDList list = syms_usid_list_from_string(scratch.arena, group->dbg_data, group->dbg, map, name);
    if (list.first != 0){
      result = list.first->usid;
    }
  }
  syms_release_scratch(scratch);
  return(result);
}

SYMS_API SYMS_USIDList
syms_group_all_usid_from_unmangled_name(SYMS_Arena *arena, SYMS_Group *group, SYMS_String8 name){
  SYMS_USIDList result = {0};
  SYMS_MapAndUnit *map = syms_group_unmangled_symbol_map(group);
  if (syms_accel_is_good(map->map)){
    result = syms_usid_list_from_string(arena, group->dbg_data, group->dbg, map, name);
  }
  return(result);
}

SYMS_API SYMS_U64
syms_group_voff_from_link_name(SYMS_Group *group, SYMS_String8 name){
  SYMS_LinkMapAccel *map = syms_group_link_name_map(group);
  SYMS_UnitAccel *unit = syms_group_link_name_unit(group);
  SYMS_U64 result = syms_voff_from_link_name(group->dbg_data, group->dbg, map, unit, name);
  return(result);
}

SYMS_API SYMS_ResolvedLine
syms_group_resolved_location_from_link_name(SYMS_Group *group, SYMS_String8 name){
  SYMS_U64 voff = syms_group_voff_from_link_name(group, name);
  SYMS_UnitID uid = syms_group_uid_from_voff__accelerated(group, voff);
  SYMS_Line line = syms_group_line_from_uid_voff__accelerated(group, uid, voff);
  SYMS_SrcCoord *src_coord = &line.src_coord;
  SYMS_String8 file_name = syms_group_file_name_from_id(group, uid, src_coord->file_id);
  
  SYMS_ResolvedLine result = {0};
  result.file_name = file_name;
  result.line = src_coord->line;
  result.col = src_coord->col;
  result.voff = voff;
  return(result);
}


////////////////////////////////
// allen: Syms Group Type Graph

SYMS_API SYMS_TypeGraph*
syms_group_type_graph(SYMS_Group *group){
  SYMS_TypeGraph *result = &group->type_graph;
  return(result);
}


////////////////////////////////
// allen: Syms Group Varaible Address Mapping Functions

SYMS_API SYMS_SymbolID
syms_group_var_sid_from_uid_voff__linear_scan(SYMS_Group *group, SYMS_UnitID uid, SYMS_U64 voff){
  SYMS_ASSERT_PARANOID(syms_thread_lane == 0);
  SYMS_ProfBegin("syms_group_var_sid_from_uid_voff__linear_scan");
  
  SYMS_String8 dbg_data = group->dbg_data;
  SYMS_DbgAccel *dbg = group->dbg;
  SYMS_UnitAccel *unit = syms_group_unit_from_uid(group, uid);
  SYMS_SymbolIDArray *sid_array = syms_group_var_sid_array_from_uid(group, uid);
  SYMS_U64 count = sid_array->count;
  
  SYMS_SymbolID result = {0};
  SYMS_SymbolID *sid_ptr = sid_array->ids;
  for (SYMS_U64 i = 0; i < count; i += 1, sid_ptr += 1){
    SYMS_SymbolID sid = *sid_ptr;
    SYMS_SymbolKind kind = syms_group_symbol_kind_from_sid(group, unit, sid);
    SYMS_B32 hit = syms_false;
    if (kind == SYMS_SymbolKind_ImageRelativeVariable){
      SYMS_U64 var_voff = syms_voff_from_var_sid(dbg_data, dbg, unit, sid);
      if (var_voff == voff){
        hit = syms_true;
      }
      else if (var_voff < voff){
        SYMS_USID var_type_usid = syms_type_from_var_sid(dbg_data, dbg, unit, sid);
        SYMS_U64 var_size = syms_group_type_size_from_usid(group, var_type_usid);
        if (voff < var_voff + var_size){
          hit = syms_true;
        }
      }
    }
    if (hit){
      result = sid;
      break;
    }
  }
  
  SYMS_ProfEnd();
  return(result);
}

SYMS_API SYMS_SpatialMap1D*
syms_group_var_map_from_uid(SYMS_Group *group, SYMS_UnitID uid){
  SYMS_ASSERT_PARANOID(syms_thread_lane == 0);
  SYMS_ProfBegin("syms_group_var_map_from_uid");
  SYMS_U64 index = uid - 1;
  if (index < group->unit_count &&
      (group->unit_cache_flags[index] & SYMS_GroupUnitCacheFlag_HasVarMap) == 0){
    //- build loose map
    SYMS_ArenaTemp scratch = syms_get_scratch(0, 0);
    SYMS_SpatialMap1DLoose loose = {0};
    
    SYMS_String8 dbg_data = group->dbg_data;
    SYMS_DbgAccel *dbg = group->dbg;
    SYMS_UnitAccel *unit = syms_group_unit_from_uid(group, uid);
    SYMS_SymbolIDArray *sid_array = syms_group_var_sid_array_from_uid(group, uid);
    SYMS_U64 count = sid_array->count;
    
    SYMS_SymbolID *sid_ptr = sid_array->ids;
    for (SYMS_U64 n = 1; n <= count; n += 1, sid_ptr += 1){
      SYMS_SymbolID sid = *sid_ptr;
      SYMS_SymbolKind kind = syms_group_symbol_kind_from_sid(group, unit, sid);
      if (kind == SYMS_SymbolKind_ImageRelativeVariable){
        SYMS_U64 var_virt_off = syms_voff_from_var_sid(dbg_data, dbg, unit, sid);
        SYMS_USID var_type_usid = syms_type_from_var_sid(dbg_data, dbg, unit, sid);
        SYMS_U64 var_size = syms_group_type_size_from_usid(group, var_type_usid);
        SYMS_U64Range range = {var_virt_off, var_virt_off + var_size};
        syms_spatial_map_1d_loose_push_single(scratch.arena, &loose, sid, range);
      }
    }
    
    //- bake tight map
    SYMS_Arena *arena = syms_group_get_lane_arena(group);
    SYMS_SpatialMap1D map = syms_spatial_map_1d_bake(arena, &loose);
    
    //- save to group
    group->unit_cache_flags[index] |= SYMS_GroupUnitCacheFlag_HasVarMap;
    group->unit_var_maps[index] = map;
    
    syms_release_scratch(scratch);
    
    SYMS_ASSERT_PARANOID(syms_spatial_map_1d_invariants(&map));
  }
  
  SYMS_SpatialMap1D *result = &syms_spatial_map_1d_nil;
  if (index < group->unit_count){
    result = &group->unit_var_maps[index];
  }
  
  SYMS_ProfEnd();
  return(result);
}

SYMS_API SYMS_SymbolID
syms_group_var_sid_from_uid_voff__accelerated(SYMS_Group *group, SYMS_UnitID uid, SYMS_U64 voff){
  SYMS_ASSERT_PARANOID(syms_thread_lane == 0);
  SYMS_ProfBegin("syms_group_var_sid_from_uid_voff__accelerated");
  SYMS_SpatialMap1D *map = syms_group_var_map_from_uid(group, uid);
  SYMS_SymbolID result = syms_spatial_map_1d_value_from_point(map, voff);
  SYMS_ProfEnd();
  return(result);
}


////////////////////////////////
// allen: Syms Group Type Graph Functions

SYMS_API SYMS_TypeNode*
syms_group_type_from_usid(SYMS_Group *group, SYMS_USID usid){
  SYMS_ASSERT_PARANOID(syms_thread_lane == 0);
  SYMS_ProfBegin("syms_group_type_from_usid");
  
  SYMS_TypeGraph *graph = &group->type_graph;
  
  SYMS_TypeParseParams params = {0};
  params.data     = group->dbg_data;
  params.dbg      = group->dbg;
  params.unit_set = group->unit_set;
  params.unit     = syms_group_unit_from_uid(group, usid.uid);
  params.uid      = usid.uid;
  params.type_map = syms_group_type_map(group);
  
  SYMS_TypeNode *result = syms_type_from_dbg_sid(graph, &params, usid.sid);
  SYMS_ProfEnd();
  return(result);
}

SYMS_API SYMS_U64
syms_group_type_size_from_usid(SYMS_Group *group, SYMS_USID usid){
  SYMS_ASSERT_PARANOID(syms_thread_lane == 0);
  SYMS_ProfBegin("syms_group_type_size_from_usid");
  SYMS_UnitAccel *unit = syms_group_unit_from_uid(group, usid.uid);
  SYMS_TypeInfo type_info = syms_type_info_from_sid(group->dbg_data, group->dbg, unit, usid.sid);
  SYMS_U64 result = 0;
  switch (type_info.reported_size_interp){
    case SYMS_SizeInterpretation_ByteCount:
    {
      result = type_info.reported_size;
    }break;
    case SYMS_SizeInterpretation_Multiplier:
    {
      SYMS_U64 element_size = syms_group_type_size_from_usid(group, type_info.direct_type);
      result = element_size*type_info.reported_size;
    }break;
    default:
    {
      SYMS_TypeNode *type_node = syms_group_type_from_usid(group, usid);
      result = type_node->byte_size;
    }break;
  }
  SYMS_ProfEnd();
  return(result);
}

SYMS_API SYMS_USIDList
syms_group_type_list_from_name_accelerated(SYMS_Arena *arena, SYMS_Group *group, SYMS_String8 name){
  SYMS_USIDList result = {0};
  SYMS_MapAndUnit *map_unit = syms_group_type_map(group);
  if (syms_accel_is_good(map_unit->map)){
    result = syms_usid_list_from_string(arena, group->dbg_data, group->dbg, map_unit, name);
  }
  return(result);
}

SYMS_API SYMS_SymbolIDArray
syms_group_types_from_unit_name(SYMS_Group *group, SYMS_UnitID uid, SYMS_String8 name){
  SYMS_SymbolNameMap *map = syms_group_type_map_from_uid(group, uid);
  SYMS_SymbolIDArray result = {0};
  if (map != 0){
    result = syms_symbol_name_map_array_from_string(map, name);
  }
  return(result);
}

SYMS_API SYMS_TypeMemberArray
syms_group_type_members_from_type(SYMS_Group *group, SYMS_TypeNode *type){
  SYMS_ProfBegin("syms_group_type_members_from_type");
  
  SYMS_B32 has_members = syms_type_kind_is_record(type->kind);
  
  // ensure members equipped
  if (has_members){
    SYMS_UnitID uid = 0;
    if (type->unique != 0){
      uid = type->unique->usid.uid;
    }
    
    if (uid != 0){
      SYMS_TypeParseParams params = {0};
      params.data = group->dbg_data;
      params.dbg = group->dbg;
      params.unit_set = group->unit_set;
      params.unit = syms_group_unit_from_uid(group, uid);
      params.uid = uid;
      
      syms_type_equip_members_from_dbg(&group->type_graph, &params, type);
    }
  }
  
  // fill result
  SYMS_TypeMemberArray result = {0};
  if (has_members && type->lazy_ptr != 0){
    result = *(SYMS_TypeMemberArray*)type->lazy_ptr;
  }
  
  SYMS_ProfEnd();
  return(result);
}

SYMS_API SYMS_EnumMemberArray
syms_group_type_enum_members_from_type(SYMS_Group *group, SYMS_TypeNode *type){
  SYMS_ProfBegin("syms_group_type_enum_members_from_type");
  
  SYMS_B32 has_members = syms_type_kind_is_enum(type->kind);
  
  // ensure members equipped
  if (has_members){
    SYMS_UnitID uid = 0;
    if (type->unique != 0){
      uid = type->unique->usid.uid;
    }
    
    if (uid != 0){
      SYMS_TypeParseParams params = {0};
      params.data = group->dbg_data;
      params.dbg = group->dbg;
      params.unit_set = group->unit_set;
      params.unit = syms_group_unit_from_uid(group, uid);
      params.uid = uid;
      
      syms_type_equip_members_from_dbg(&group->type_graph, &params, type);
    }
  }
  
  // fill result
  SYMS_EnumMemberArray result = {0};
  if (has_members && type->lazy_ptr != 0){
    result = *(SYMS_EnumMemberArray*)type->lazy_ptr;
  }
  
  SYMS_ProfEnd();
  return(result);
}

////////////////////////////////
//~ allen: Syms File Map

SYMS_API SYMS_Name2FileIDMap*
syms_group_file_map(SYMS_Group *group){
  SYMS_ProfBegin("syms_group_file_map");
  if (!group->name_2_file_id_map_is_built){
    SYMS_ASSERT_PARANOID(syms_thread_lane == 0);
    SYMS_ArenaTemp scratch = syms_get_scratch(0, 0);
    
    //- build loose file map
    SYMS_U64 unit_count = group->unit_count;
    SYMS_Name2FileIDMapLoose loose = {0};
    for (SYMS_UnitID uid = 1; uid <= unit_count; uid += 1){
      SYMS_LineParseOut *line_parse = syms_group_line_parse_from_uid(group, uid);
      
      // build from file_id_array
      if (line_parse->file_id_array.count != 0){
        SYMS_String8Array *inferred = syms_group_inferred_file_table_from_uid(group, uid);
        SYMS_U64 count = SYMS_MIN(line_parse->file_id_array.count, inferred->count);
        SYMS_FileID *file_id_ptr = line_parse->file_id_array.ids;
        SYMS_FileID *opl = file_id_ptr + count;
        SYMS_String8 *name_ptr = inferred->strings;
        for (;file_id_ptr < opl; file_id_ptr += 1, name_ptr += 1){
          SYMS_String8 name_cons = syms_string_cons(group->arena, &group->string_cons, *name_ptr);
          syms_name_2_file_id_map_loose_push(scratch.arena, &loose, name_cons, uid, *file_id_ptr);
        }
      }
      
      // build from file_table
      else{
        SYMS_String8Array *file_table = syms_group_file_table_from_uid_with_fallbacks(group, uid);
        SYMS_FileID file_id = 1;
        for (SYMS_String8 *name_ptr = file_table->strings, *opl = file_table->strings + file_table->count;
             name_ptr < opl;
             name_ptr += 1, file_id += 1){
          SYMS_String8 name_cons = syms_string_cons(group->arena, &group->string_cons, *name_ptr);
          syms_name_2_file_id_map_loose_push(scratch.arena, &loose, name_cons, uid, file_id);
        }
      }
    }
    
    //- bake
    SYMS_Name2FileIDMap name_2_file_id_map = syms_name_2_file_id_map_bake(group->arena, &loose);
    
    //- save to group
    group->name_2_file_id_map_is_built = syms_true;
    group->name_2_file_id_map = name_2_file_id_map;
    
    syms_release_scratch(scratch);
  }
  
  SYMS_Name2FileIDMap *result = &group->name_2_file_id_map;
  
  SYMS_ProfEnd();
  return(result);
}

#endif //SYMS_GROUP_C
