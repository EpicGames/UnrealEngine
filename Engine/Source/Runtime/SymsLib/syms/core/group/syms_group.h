// Copyright Epic Games, Inc. All Rights Reserved.
/* date = August 2nd 2021 0:48 pm */

#ifndef SYMS_GROUP_H
#define SYMS_GROUP_H

////////////////////////////////
//~ allen: Syms Group Types

typedef SYMS_U32 SYMS_GroupUnitCacheFlags;
enum{
  SYMS_GroupUnitCacheFlag_HasProcSidArray   = (1 <<  0),
  SYMS_GroupUnitCacheFlag_HasVarSidArray    = (1 <<  1),
  SYMS_GroupUnitCacheFlag_HasTlsVarSidArray = (1 <<  2),
  SYMS_GroupUnitCacheFlag_HasTypeSidArray   = (1 <<  3),
  SYMS_GroupUnitCacheFlag_HasFileTable      = (1 <<  4),
  SYMS_GroupUnitCacheFlag_HasInfFileTable   = (1 <<  5),
  SYMS_GroupUnitCacheFlag_HasLineTable      = (1 <<  6),
  SYMS_GroupUnitCacheFlag_HasProcMap        = (1 <<  7),
  SYMS_GroupUnitCacheFlag_HasVarMap         = (1 <<  8),
  SYMS_GroupUnitCacheFlag_HasLineSeqMap     = (1 <<  9),
  SYMS_GroupUnitCacheFlag_HasLineToAddrMap  = (1 << 10),
  SYMS_GroupUnitCacheFlag_HasTypeNameMap    = (1 << 11),
};

typedef struct SYMS_Group{
  SYMS_Arena *arena;
  
  //- thread lanes
  SYMS_Arena **lane_arenas;
  SYMS_U64 lane_count;
  SYMS_U64 lane_max;
  
  //- data for binary and debug files
  SYMS_String8 bin_data;
  SYMS_String8 dbg_data;
  SYMS_BinAccel *bin;
  SYMS_DbgAccel *dbg;
  
  //- top-level accelerators and info
  SYMS_Arch arch;
  SYMS_U64 address_size;
  SYMS_U64 default_vbase;
  SYMS_SecInfoArray sec_info_array;
  SYMS_UnitSetAccel *unit_set;
  SYMS_U64 unit_count;
  
  //- basic section caches
  SYMS_String8 *sec_names;
  
  //- basic unit caches
  SYMS_GroupUnitCacheFlags *unit_cache_flags;
  SYMS_UnitAccel **units;
  SYMS_SymbolIDArray *proc_sid_arrays;
  SYMS_SymbolIDArray *var_sid_arrays;
  SYMS_SymbolIDArray *thread_sid_arrays;
  SYMS_SymbolIDArray *type_sid_arrays;
  SYMS_String8Array *file_tables;
  SYMS_String8Array *inferred_file_tables;
  SYMS_LineParseOut *line_tables;
  SYMS_SpatialMap1D *unit_proc_maps;
  SYMS_SpatialMap1D *unit_var_maps;
  SYMS_SpatialMap1D *line_sequence_maps;
  SYMS_FileToLineToAddrMap *file_to_line_to_addr_maps;
  
  // TODO(allen): we probably don't need to do this as the type graph does it better
  SYMS_SymbolNameMap *unit_type_maps;
  
  //- hash tables caches
  SYMS_StringCons string_cons;
  SYMS_FileID2NameMap file_id_2_name_map;
  
  //- type graph
  SYMS_TypeGraph type_graph;
  
  //- one-time fills/builds
  SYMS_B8 unit_ranges_is_filled;
  SYMS_B8 type_map_unit_is_filled;
  SYMS_B8 symbol_map_unit_is_filled;
  SYMS_B8 mangled_symbol_map_unit_is_filled;
  SYMS_B8 sec_map_v_is_built;
  SYMS_B8 sec_map_f_is_built;
  SYMS_B8 unit_map_is_built;
  SYMS_B8 name_2_file_id_map_is_built;
  SYMS_B8 link_name_record_array_is_filled;
  SYMS_B8 link_name_spatial_map_is_built;
  SYMS_B8 link_map_is_built;
  SYMS_B8 link_name_unit_is_filled;
  
  SYMS_UnitRangeArray unit_ranges;
  SYMS_SpatialMap1D sec_map_v;
  SYMS_SpatialMap1D sec_map_f;
  SYMS_SpatialMap1D unit_map;
  SYMS_Name2FileIDMap name_2_file_id_map;
  SYMS_LinkNameRecArray link_name_record_array;
  SYMS_SpatialMap1D link_name_spatial_map;
  SYMS_LinkMapAccel *link_map;
  SYMS_UnitAccel *link_name_unit;
  
  //- map & units
  SYMS_MapAndUnit type_mau;
  SYMS_MapAndUnit symbol_mau;
} SYMS_Group;

////////////////////////////////
//~ allen: Data Structure Nils

SYMS_READ_ONLY SYMS_GLOBAL SYMS_SymbolIDArray syms_sid_array_nil = {0};
SYMS_READ_ONLY SYMS_GLOBAL SYMS_String8Array syms_string_array_nil = {0};
SYMS_READ_ONLY SYMS_GLOBAL SYMS_LineParseOut syms_line_parse_nil = {{0}};
SYMS_READ_ONLY SYMS_GLOBAL SYMS_SpatialMap1D syms_spatial_map_1d_nil = {0};
SYMS_READ_ONLY SYMS_GLOBAL SYMS_LineToAddrMap syms_line_to_addr_map_nil = {0};

////////////////////////////////
//~ allen: Syms Group Setup Functions

SYMS_API SYMS_Group* syms_group_alloc(void);
SYMS_API void        syms_group_release(SYMS_Group *group);

SYMS_API void        syms_group_init(SYMS_Group *group, SYMS_ParseBundle *params);

SYMS_API void        syms_group_parse_all_units__single_thread(SYMS_Group *group);
SYMS_API void        syms_group_parse_all_top_level(SYMS_Group *group);

SYMS_API void        syms_group_begin_multilane(SYMS_Group *group, SYMS_U64 lane_count);
SYMS_API void        syms_group_end_multilane(SYMS_Group *group);

SYMS_API SYMS_Arena* syms_group_get_lane_arena(SYMS_Group *group);

////////////////////////////////
//~ allen: Syms Group Getters & Cache Accessors

// TODO(allen): better sorting plan here.

SYMS_API SYMS_String8       syms_group_bin_data(SYMS_Group *group);
SYMS_API SYMS_BinAccel*     syms_group_bin(SYMS_Group *group);
SYMS_API SYMS_String8       syms_group_dbg_data(SYMS_Group *group);
SYMS_API SYMS_DbgAccel*     syms_group_dbg(SYMS_Group *group);
SYMS_API SYMS_Arch          syms_group_arch(SYMS_Group *group);
SYMS_API SYMS_UnitSetAccel* syms_group_unit_set(SYMS_Group *group);

SYMS_API SYMS_U64           syms_group_address_size(SYMS_Group *group);
SYMS_API SYMS_U64           syms_group_default_vbase(SYMS_Group *group);

SYMS_API SYMS_SecInfoArray  syms_group_sec_info_array(SYMS_Group *group);
SYMS_API SYMS_SecInfo*      syms_group_sec_info_from_number(SYMS_Group *group, SYMS_U64 n);

SYMS_API SYMS_U64           syms_group_unit_count(SYMS_Group *group);
SYMS_API SYMS_UnitInfo      syms_group_unit_info_from_uid(SYMS_Group *group, SYMS_UnitID uid);
SYMS_API SYMS_UnitNames     syms_group_unit_names_from_uid(SYMS_Arena *arena, SYMS_Group *group, SYMS_UnitID uid);

// thread safe (with lanes equipped to group)
SYMS_API SYMS_UnitAccel*    syms_group_unit_from_uid(SYMS_Group *group, SYMS_UnitID uid);
SYMS_API SYMS_SymbolIDArray*syms_group_proc_sid_array_from_uid(SYMS_Group *group, SYMS_UnitID uid);
SYMS_API SYMS_SymbolIDArray*syms_group_var_sid_array_from_uid(SYMS_Group *group, SYMS_UnitID uid);
SYMS_API SYMS_SymbolIDArray*syms_group_tls_var_sid_array_from_uid(SYMS_Group *group, SYMS_UnitID uid);
SYMS_API SYMS_SymbolIDArray*syms_group_type_sid_array_from_uid(SYMS_Group *group, SYMS_UnitID uid);
SYMS_API SYMS_String8Array* syms_group_file_table_from_uid(SYMS_Group *group, SYMS_UnitID uid);
SYMS_API SYMS_LineParseOut* syms_group_line_parse_from_uid(SYMS_Group *group, SYMS_UnitID uid);
SYMS_API SYMS_LineTable*    syms_group_line_table_from_uid(SYMS_Group *group, SYMS_UnitID uid);
SYMS_API SYMS_String8Array* syms_group_inferred_file_table_from_uid(SYMS_Group *group, SYMS_UnitID uid);

SYMS_API SYMS_String8Array* syms_group_file_table_from_uid_with_fallbacks(SYMS_Group *group, SYMS_UnitID uid);

SYMS_API SYMS_UnitRangeArray syms_group_unit_ranges(SYMS_Group *group);

SYMS_API SYMS_SymbolKind    syms_group_symbol_kind_from_sid(SYMS_Group *group,
                                                            SYMS_UnitAccel *unit, SYMS_SymbolID sid);
SYMS_API SYMS_String8       syms_group_symbol_name_from_sid(SYMS_Arena *arena, SYMS_Group *group,
                                                            SYMS_UnitAccel *unit, SYMS_SymbolID sid);

SYMS_API SYMS_String8       syms_group_file_name_from_id(SYMS_Group *group, SYMS_UnitID uid, SYMS_FileID file_id);

SYMS_API SYMS_MapAndUnit*   syms_group_type_map(SYMS_Group *group);
SYMS_API SYMS_MapAndUnit*   syms_group_unmangled_symbol_map(SYMS_Group *group);

SYMS_API SYMS_LinkNameRecArray syms_group_link_name_records(SYMS_Group *group);
SYMS_API SYMS_LinkMapAccel*    syms_group_link_name_map(SYMS_Group *group);
SYMS_API SYMS_UnitAccel*       syms_group_link_name_unit(SYMS_Group *group);


////////////////////////////////
//~ allen: Syms Group Address Mapping Functions

//- linear scan versions
SYMS_API SYMS_U64      syms_group_sec_number_from_voff__linear_scan(SYMS_Group *group, SYMS_U64 voff);
SYMS_API SYMS_U64      syms_group_sec_number_from_foff__linear_scan(SYMS_Group *group, SYMS_U64 foff);

SYMS_API SYMS_U64Maybe syms_group_voff_from_foff__linear_scan(SYMS_Group *group, SYMS_U64 foff);
SYMS_API SYMS_U64Maybe syms_group_foff_from_voff__linear_scan(SYMS_Group *group, SYMS_U64 voff);

SYMS_API SYMS_UnitID   syms_group_uid_from_voff__linear_scan(SYMS_Group *group, SYMS_U64 voff);
SYMS_API SYMS_SymbolID syms_group_proc_sid_from_uid_voff__linear_scan(SYMS_Group *group, SYMS_UnitID uid,
                                                                      SYMS_U64 voff);

SYMS_API SYMS_Line     syms_group_line_from_uid_voff__linear_scan(SYMS_Group *group, SYMS_UnitID uid,
                                                                  SYMS_U64 voff);
SYMS_API SYMS_U64RangeList syms_group_vranges_from_uid_line__linear_scan(SYMS_Arena *arena, SYMS_Group *group,
                                                                         SYMS_UnitID uid,
                                                                         SYMS_FileID file_id, SYMS_U32 line);

//- map getters
SYMS_API SYMS_SpatialMap1D* syms_group_sec_map_v(SYMS_Group *group);
SYMS_API SYMS_SpatialMap1D* syms_group_sec_map_f(SYMS_Group *group);

SYMS_API SYMS_SpatialMap1D* syms_group_unit_map(SYMS_Group *group);

// thread safe (with lanes equipped to group)
SYMS_API SYMS_SpatialMap1D* syms_group_proc_map_from_uid(SYMS_Group *group, SYMS_UnitID uid);
SYMS_API SYMS_SpatialMap1D* syms_group_line_sequence_map_from_uid(SYMS_Group *group, SYMS_UnitID uid);

SYMS_API void                syms_group_fetch_line_to_addr_maps_from_uid(SYMS_Group *group, SYMS_UnitID uid);
SYMS_API SYMS_LineToAddrMap* syms_group_line_to_addr_map_from_uid_file_id(SYMS_Group *group, SYMS_UnitID uid,
                                                                          SYMS_FileID file_id);

SYMS_API SYMS_SymbolNameMap* syms_group_type_map_from_uid(SYMS_Group *group, SYMS_UnitID uid);

SYMS_API SYMS_SpatialMap1D*  syms_group_link_name_spatial_map(SYMS_Group *group);
SYMS_API void                syms_group__link_names_sort_in_place(SYMS_LinkNameRec *recs, SYMS_U64 count);

//- accelerated versions
SYMS_API SYMS_U64      syms_group_sec_number_from_voff__accelerated(SYMS_Group *group, SYMS_U64 voff);
SYMS_API SYMS_U64      syms_group_sec_number_from_foff__accelerated(SYMS_Group *group, SYMS_U64 foff);

SYMS_API SYMS_U64Maybe syms_group_sec_voff_from_foff__accelerated(SYMS_Group *group, SYMS_U64 voff);
SYMS_API SYMS_U64Maybe syms_group_sec_foff_from_voff__accelerated(SYMS_Group *group, SYMS_U64 foff);

SYMS_API SYMS_UnitID   syms_group_uid_from_voff__accelerated(SYMS_Group *group, SYMS_U64 voff);
SYMS_API SYMS_SymbolID syms_group_proc_sid_from_uid_voff__accelerated(SYMS_Group *group, SYMS_UnitID uid,
                                                                      SYMS_U64 voff);

SYMS_API SYMS_Line     syms_group_line_from_uid_voff__accelerated(SYMS_Group *group, SYMS_UnitID uid,
                                                                  SYMS_U64 voff);
SYMS_API SYMS_U64RangeArray syms_group_vranges_from_uid_line__accelerated(SYMS_Group *group, SYMS_UnitID uid,
                                                                          SYMS_FileID file_id, SYMS_U32 line);

//- higher level mapping functions
SYMS_API SYMS_ResolvedLine syms_group_resolved_location_from_proc_sid(SYMS_Group *group, SYMS_UnitAccel *unit,
                                                                      SYMS_SymbolID sid);


////////////////////////////////
//~ allen: Syms Group Name Mapping Functions

SYMS_API SYMS_USID     syms_group_usid_from_unmangled_name(SYMS_Group *group, SYMS_String8 name);
SYMS_API SYMS_USIDList syms_group_all_usid_from_unmangled_name(SYMS_Arena *arena, SYMS_Group *group,
                                                               SYMS_String8 name);

SYMS_API SYMS_U64      syms_group_voff_from_link_name(SYMS_Group *group, SYMS_String8 name);
SYMS_API SYMS_ResolvedLine syms_group_resolved_location_from_link_name(SYMS_Group *group, SYMS_String8 name);

////////////////////////////////
//~ allen: Syms Group Type Graph

SYMS_API SYMS_TypeGraph* syms_group_type_graph(SYMS_Group *group);

////////////////////////////////
//~ allen: Syms Group Varaible Address Mapping Functions

// these are seperated because they require type info which
// cannot be parallelized like the others


SYMS_API SYMS_SymbolID syms_group_var_sid_from_uid_voff__linear_scan(SYMS_Group *group,
                                                                     SYMS_UnitID uid, SYMS_U64 voff);

SYMS_API SYMS_SpatialMap1D* syms_group_var_map_from_uid(SYMS_Group *group, SYMS_UnitID uid);

SYMS_API SYMS_SymbolID syms_group_var_sid_from_uid_voff__accelerated(SYMS_Group *group,
                                                                     SYMS_UnitID uid, SYMS_U64 voff);


////////////////////////////////
//~ allen: Syms Group Type Info Functions

SYMS_API SYMS_TypeNode* syms_group_type_from_usid(SYMS_Group *group, SYMS_USID usid);

// TODO(allen): sort with name maps?
SYMS_API SYMS_USIDList syms_group_type_list_from_name_accelerated(SYMS_Arena *arena, SYMS_Group *group,
                                                                  SYMS_String8 name);
SYMS_API SYMS_SymbolIDArray syms_group_types_from_unit_name(SYMS_Group *group, SYMS_UnitID uid, SYMS_String8 name);

SYMS_API SYMS_U64       syms_group_type_size_from_usid(SYMS_Group *group, SYMS_USID usid);

SYMS_API SYMS_TypeMemberArray syms_group_type_members_from_type(SYMS_Group *group, SYMS_TypeNode *type);
SYMS_API SYMS_EnumMemberArray syms_group_type_enum_members_from_type(SYMS_Group *group, SYMS_TypeNode *type);

////////////////////////////////
//~ allen: Syms File Map

SYMS_API SYMS_Name2FileIDMap* syms_group_file_map(SYMS_Group *group);


#endif //SYMS_GROUP_H
