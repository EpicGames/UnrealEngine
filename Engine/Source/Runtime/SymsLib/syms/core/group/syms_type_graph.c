// Copyright Epic Games, Inc. All Rights Reserved.

#ifndef SYMS_TYPE_GRAPH_C
#define SYMS_TYPE_GRAPH_C

////////////////////////////////
//~ allen: Type Graph Setup Functions

SYMS_API void
syms_type_graph_init(SYMS_TypeGraph *graph,
                     SYMS_Arena *graph_arena, SYMS_StringCons *graph_string_cons,
                     SYMS_U64 address_size){
  SYMS_ProfBegin("syms_type_graph_init");
  
  graph->arena = graph_arena;
  graph->string_cons = graph_string_cons;
  graph->address_size = address_size;
  
  graph->content_buckets.buckets =
    syms_push_array_zero(graph->arena, SYMS_TypeContentNode*, SYMS_TYPE_GRAPH_TABLE_BUCKET_COUNT);
  
  graph->usid_buckets.buckets =
    syms_push_array_zero(graph->arena, SYMS_TypeUSIDNode*, SYMS_TYPE_GRAPH_TABLE_BUCKET_COUNT);
  
  graph->usid_buckets.bucket_counts =
    syms_push_array_zero(graph->arena, SYMS_U64, SYMS_TYPE_GRAPH_TABLE_BUCKET_COUNT);
  
  graph->name_buckets.buckets =
    syms_push_array_zero(graph->arena, SYMS_TypeNameNode*, SYMS_TYPE_GRAPH_TABLE_BUCKET_COUNT);
  
  SYMS_ProfEnd();
}

////////////////////////////////
//~ allen: Type Name Lookup Functions

SYMS_API SYMS_String8
syms_type_string_cons(SYMS_TypeGraph *graph, SYMS_String8 name){
  SYMS_String8 result = syms_string_cons(graph->arena, graph->string_cons, name);
  return(result);
}

SYMS_API SYMS_TypeNode*
syms_type_from_usid(SYMS_TypeGraph *graph, SYMS_USID usid){
  SYMS_TypeNode *result = syms_type_from_usid_buckets(&graph->usid_buckets, usid);
  return(result);
}

SYMS_API SYMS_TypeChain
syms_type_from_name(SYMS_TypeGraph *graph, SYMS_String8 name){
  SYMS_TypeChain result = {0};
  SYMS_String8 name_cons = syms_type_string_cons(graph, name);
  SYMS_TypeChain *chain = syms_type_chain_from_name_buckets(&graph->name_buckets, name_cons.str);
  if (chain != 0){
    syms_memmove(&result, chain, sizeof(*chain));
  }
  return(result);
}

////////////////////////////////
//~ allen: Type Node Info Getters

SYMS_API SYMS_TypeMemberArray
syms_type_members_from_type(SYMS_TypeGraph *graph, SYMS_TypeNode *node){
  SYMS_TypeMemberArray result = {0};
  if (node->lazy_ptr != 0 && syms_type_kind_is_record(node->kind)){
    result = *(SYMS_TypeMemberArray*)node->lazy_ptr;
  }
  return(result);
}

SYMS_API SYMS_EnumMemberArray
syms_type_enum_members_from_type(SYMS_TypeGraph *graph, SYMS_TypeNode *node){
  SYMS_EnumMemberArray result = {0};
  if (node->lazy_ptr != 0 && syms_type_kind_is_enum(node->kind)){
    result = *(SYMS_EnumMemberArray*)node->lazy_ptr;
  }
  return(result);
}

SYMS_API SYMS_B32
syms_type_members_are_equipped(SYMS_TypeGraph *graph, SYMS_TypeNode *node){
  SYMS_B32 result = 0;
  if (node->kind == SYMS_TypeKind_Struct ||
      node->kind == SYMS_TypeKind_Union ||
      node->kind == SYMS_TypeKind_Class ||
      node->kind == SYMS_TypeKind_Enum){
    result = (node->lazy_ptr != 0);
  }
  return(result);
}

////////////////////////////////
//~ allen: Type Node Basic Type Getters

SYMS_API SYMS_TypeNode*
syms_type_void(SYMS_TypeGraph *graph){
  SYMS_TypeNode *result = graph->type_void;
  if (result == 0){
    result = syms_type_cons_basic(graph, SYMS_TypeKind_Void, 0, syms_str8_lit("void"));
    graph->type_void = result;
  }
  return(result);
}

SYMS_API SYMS_TypeNode*
syms_type_bool(SYMS_TypeGraph *graph){
  SYMS_TypeNode *result = graph->type_bool;
  if (result == 0){
    result = syms_type_cons_basic(graph, SYMS_TypeKind_Bool, 1, syms_str8_lit("bool"));
    graph->type_bool = result;
  }
  return(result);
}

SYMS_API SYMS_TypeNode*
syms_type_u8(SYMS_TypeGraph *graph){
  SYMS_TypeNode *result = graph->type_u8;
  if (result == 0){
    result = syms_type_cons_basic(graph, SYMS_TypeKind_UInt8, 1, syms_str8_lit("uint8"));
    graph->type_u8 = result;
  }
  return(result);
}

SYMS_API SYMS_TypeNode*
syms_type_u16(SYMS_TypeGraph *graph){
  SYMS_TypeNode *result = graph->type_u16;
  if (result == 0){
    result = syms_type_cons_basic(graph, SYMS_TypeKind_UInt16, 2, syms_str8_lit("uint16"));
    graph->type_u16 = result;
  }
  return(result);
}

SYMS_API SYMS_TypeNode*
syms_type_u32(SYMS_TypeGraph *graph){
  SYMS_TypeNode *result = graph->type_u32;
  if (result == 0){
    result = syms_type_cons_basic(graph, SYMS_TypeKind_UInt32, 4, syms_str8_lit("uint32"));
    graph->type_u32 = result;
  }
  return(result);
}

SYMS_API SYMS_TypeNode*
syms_type_u64(SYMS_TypeGraph *graph){
  SYMS_TypeNode *result = graph->type_u64;
  if (result == 0){
    result = syms_type_cons_basic(graph, SYMS_TypeKind_UInt64, 8, syms_str8_lit("uint64"));
    graph->type_u64 = result;
  }
  return(result);
}

SYMS_API SYMS_TypeNode*
syms_type_u128(SYMS_TypeGraph *graph){
  SYMS_TypeNode *result = graph->type_u128;
  if (result == 0){
    result = syms_type_cons_basic(graph, SYMS_TypeKind_UInt128, 16, syms_str8_lit("uint128"));
    graph->type_u128 = result;
  }
  return(result);
}

SYMS_API SYMS_TypeNode*
syms_type_s8(SYMS_TypeGraph *graph){
  SYMS_TypeNode *result = graph->type_s8;
  if (result == 0){
    result = syms_type_cons_basic(graph, SYMS_TypeKind_Int8, 1, syms_str8_lit("int8"));
    graph->type_s8 = result;
  }
  return(result);
}

SYMS_API SYMS_TypeNode*
syms_type_s16(SYMS_TypeGraph *graph){
  SYMS_TypeNode *result = graph->type_s16;
  if (result == 0){
    result = syms_type_cons_basic(graph, SYMS_TypeKind_Int16, 2, syms_str8_lit("int16"));
    graph->type_s16 = result;
  }
  return(result);
}

SYMS_API SYMS_TypeNode*
syms_type_s32(SYMS_TypeGraph *graph){
  SYMS_TypeNode *result = graph->type_s32;
  if (result == 0){
    result = syms_type_cons_basic(graph, SYMS_TypeKind_Int32, 4, syms_str8_lit("int32"));
    graph->type_s32 = result;
  }
  return(result);
}

SYMS_API SYMS_TypeNode*
syms_type_s64(SYMS_TypeGraph *graph){
  SYMS_TypeNode *result = graph->type_s64;
  if (result == 0){
    result = syms_type_cons_basic(graph, SYMS_TypeKind_Int64, 8, syms_str8_lit("int64"));
    graph->type_s64 = result;
  }
  return(result);
}

SYMS_API SYMS_TypeNode*
syms_type_s128(SYMS_TypeGraph *graph){
  SYMS_TypeNode *result = graph->type_s64;
  if (result == 0){
    result = syms_type_cons_basic(graph, SYMS_TypeKind_Int128, 16, syms_str8_lit("int128"));
    graph->type_s128 = result;
  }
  return(result);
}

SYMS_API SYMS_TypeNode*
syms_type_f32(SYMS_TypeGraph *graph){
  SYMS_TypeNode *result = graph->type_f32;
  if (result == 0){
    result = syms_type_cons_basic(graph, SYMS_TypeKind_Float32, 4, syms_str8_lit("float32"));
    graph->type_f32 = result;
  }
  return(result);
}

SYMS_API SYMS_TypeNode*
syms_type_f64(SYMS_TypeGraph *graph){
  SYMS_TypeNode *result = graph->type_f64;
  if (result == 0){
    result = syms_type_cons_basic(graph, SYMS_TypeKind_Float64, 8, syms_str8_lit("float64"));
    graph->type_f64 = result;
  }
  return(result);
}

////////////////////////////////
//~ allen: Type Node Constructors

SYMS_API SYMS_TypeNode*
syms_type_cons_basic(SYMS_TypeGraph *graph, SYMS_TypeKind basic_kind, SYMS_U64 size, SYMS_String8 name){
  SYMS_ProfBegin("syms_type_cons_basic");
  
  SYMS_String8 name_cons = syms_type_string_cons(graph, name);
  
  SYMS_U64 content[3];
  content[0] = basic_kind;
  content[1] = size;
  content[2] = (SYMS_U64)name_cons.str;
  SYMS_String8 data = {(SYMS_U8*)content, sizeof(content)};
  SYMS_TypeNode *result = syms_type_from_content_buckets(&graph->content_buckets, data);
  
  if (result == 0){
    result = syms_push_array_zero(graph->arena, SYMS_TypeNode, 1);
    syms_type_content_insert(graph->arena, &graph->content_buckets, data, result);
    syms_type_name_insert(graph->arena, &graph->name_buckets, name_cons.str, result);
    
    result->kind = basic_kind;
    result->name = name_cons;
    result->byte_size = size;
    result->direct_type = &syms_type_node_nil;
    result->this_type = &syms_type_node_nil;
  }
  
  SYMS_ProfEnd();
  return(result);
}

SYMS_API SYMS_TypeNode*
syms_type_cons_mod(SYMS_TypeGraph *graph, SYMS_TypeNode *type, SYMS_TypeModifiers mods){
  SYMS_ProfBegin("syms_type_cons_mod");
  
  SYMS_U64 content[3];
  content[0] = SYMS_TypeKind_Modifier;
  content[1] = mods;
  content[2] = (SYMS_U64)type;
  SYMS_String8 data = {(SYMS_U8*)content, sizeof(content)};
  SYMS_TypeNode *result = syms_type_from_content_buckets(&graph->content_buckets, data);
  
  if (result == 0){
    result = syms_push_array_zero(graph->arena, SYMS_TypeNode, 1);
    syms_type_content_insert(graph->arena, &graph->content_buckets, data, result);
    
    result->kind = SYMS_TypeKind_Modifier;
    result->byte_size = type->byte_size;
    result->direct_type = type;
    result->this_type = &syms_type_node_nil;
    result->mods = mods;
  }
  
  SYMS_ProfEnd();
  return(result);
}

SYMS_API SYMS_TypeNode*
syms_type_cons_ptr(SYMS_TypeGraph *graph, SYMS_TypeKind ptr_kind, SYMS_TypeNode *type){
  SYMS_ProfBegin("syms_type_cons_ptr");
  
  SYMS_U64 content[2];
  content[0] = ptr_kind;
  content[1] = (SYMS_U64)type;
  SYMS_String8 data = {(SYMS_U8*)content, sizeof(content)};
  SYMS_TypeNode *result = syms_type_from_content_buckets(&graph->content_buckets, data);
  
  if (result == 0){
    result = syms_push_array_zero(graph->arena, SYMS_TypeNode, 1);
    syms_type_content_insert(graph->arena, &graph->content_buckets, data, result);
    
    result->kind = ptr_kind;
    result->byte_size = graph->address_size;
    result->direct_type = type;
    result->this_type = &syms_type_node_nil;
  }
  
  SYMS_ProfEnd();
  return(result);
}

SYMS_API SYMS_TypeNode*
syms_type_cons_array(SYMS_TypeGraph *graph, SYMS_TypeNode *type, SYMS_U64 count){
  SYMS_ProfBegin("syms_type_cons_array");
  
  SYMS_U64 content[3];
  content[0] = SYMS_TypeKind_Array;
  content[1] = count;
  content[2] = (SYMS_U64)type;
  SYMS_String8 data = {(SYMS_U8*)content, sizeof(content)};
  SYMS_TypeNode *result = syms_type_from_content_buckets(&graph->content_buckets, data);
  
  if (result == 0){
    result = syms_push_array_zero(graph->arena, SYMS_TypeNode, 1);
    syms_type_content_insert(graph->arena, &graph->content_buckets, data, result);
    
    result->kind = SYMS_TypeKind_Array;
    result->byte_size = count*type->byte_size;
    result->direct_type = type;
    result->this_type = &syms_type_node_nil;
    result->array_count = count;
  }
  
  SYMS_ProfEnd();
  return(result);
}

SYMS_API SYMS_TypeNode*
syms_type_cons_proc(SYMS_TypeGraph *graph, SYMS_TypeNode *ret_type, SYMS_TypeNode *this_type,
                    SYMS_TypeNode **param_types, SYMS_U64 param_count){
  SYMS_ProfBegin("syms_type_cons_proc");
  
  SYMS_String8 params_as_string = {(SYMS_U8*)param_types, sizeof(*param_types)*param_count};
  SYMS_String8 params_cons = syms_type_string_cons(graph, params_as_string);
  
  SYMS_U64 content[4];
  content[0] = SYMS_TypeKind_Proc;
  content[1] = (SYMS_U64)ret_type;
  content[2] = (SYMS_U64)this_type;
  content[3] = (SYMS_U64)params_cons.str;
  SYMS_String8 data = {(SYMS_U8*)content, sizeof(content)};
  SYMS_TypeNode *result = syms_type_from_content_buckets(&graph->content_buckets, data);
  
  if (result == 0){
    result = syms_push_array_zero(graph->arena, SYMS_TypeNode, 1);
    syms_type_content_insert(graph->arena, &graph->content_buckets, data, result);
    
    result->kind = SYMS_TypeKind_Proc;
    result->byte_size = 0;
    result->direct_type = ret_type;
    result->this_type = this_type;
    result->proc.params = (SYMS_TypeNode**)(params_cons.str);
    result->proc.param_count = param_count;
  }
  
  SYMS_ProfEnd();
  return(result);
}

SYMS_API SYMS_TypeNode*
syms_type_cons_mem_ptr(SYMS_TypeGraph *graph, SYMS_TypeNode *container, SYMS_TypeNode *type){
  SYMS_ProfBegin("syms_type_cons_mem_ptr");
  
  SYMS_U64 content[3];
  content[0] = SYMS_TypeKind_MemberPtr;
  content[1] = (SYMS_U64)container;
  content[2] = (SYMS_U64)type;
  SYMS_String8 data = {(SYMS_U8*)content, sizeof(content)};
  SYMS_TypeNode *result = syms_type_from_content_buckets(&graph->content_buckets, data);
  
  if (result == 0){
    result = syms_push_array_zero(graph->arena, SYMS_TypeNode, 1);
    syms_type_content_insert(graph->arena, &graph->content_buckets, data, result);
    
    SYMS_U64 byte_size = graph->address_size;
    if (type->kind == SYMS_TypeKind_Proc){
      byte_size *= 2;
    }
    
    result->kind = SYMS_TypeKind_MemberPtr;
    result->byte_size = byte_size;
    result->direct_type = type;
    result->this_type = container;
  }
  
  SYMS_ProfEnd();
  return(result);
}

SYMS_API SYMS_TypeNode*
syms_type_cons_bitfield(SYMS_TypeGraph *graph,
                        SYMS_TypeNode *underlying_type, SYMS_U32 bitoff, SYMS_U32 bitcount){
  SYMS_TypeNode *node = syms_push_array_zero(graph->arena, SYMS_TypeNode, 1);
  node->kind = SYMS_TypeKind_Bitfield;
  node->byte_size = underlying_type->byte_size;
  node->direct_type = underlying_type;
  node->this_type = &syms_type_node_nil;
  node->bits.off = bitoff;
  node->bits.count = bitcount;
  return(node);
}

SYMS_API SYMS_TypeNode*
syms_type_cons_record_stub(SYMS_TypeGraph *graph){
  SYMS_TypeNode *result = syms_push_array_zero(graph->arena, SYMS_TypeNode, 1);
  result->kind = SYMS_TypeKind_Stub;
  result->direct_type = &syms_type_node_nil;
  result->this_type = &syms_type_node_nil;
  return(result);
}

SYMS_API void
syms_type_cons_mem_push(SYMS_Arena *arena, SYMS_TypeConsMemberList *list,
                        SYMS_String8 name, SYMS_TypeNode *type){
  SYMS_TypeConsMember *member = syms_push_array(arena, SYMS_TypeConsMember, 1);
  SYMS_QueuePush(list->first, list->last, member);
  list->count += 1;
  member->name = name;
  member->type = type;
}

SYMS_API void
syms_type_cons_record_with_members(SYMS_TypeGraph *graph, SYMS_TypeNode *node,
                                   SYMS_TypeKind kind, SYMS_String8 name,
                                   SYMS_TypeConsMemberList *list){
  SYMS_ASSERT(node->kind == SYMS_TypeKind_Stub);
  SYMS_ASSERT(kind == SYMS_TypeKind_Struct ||
              kind == SYMS_TypeKind_Union ||
              kind == SYMS_TypeKind_Class);
  
  // setup members memory now (so we can fill it)
  SYMS_TypeMemberArray *array = syms_push_array(graph->arena, SYMS_TypeMemberArray, 1);
  
  SYMS_U64 member_count = list->count;
  SYMS_TypeMember *members = syms_push_array_zero(graph->arena, SYMS_TypeMember, member_count);
  
  array->mems = members;
  array->count = member_count;
  
  // fill members (fused compute layout & format to output) 
  SYMS_U64 byte_size = 0;
  switch (kind){
    case SYMS_TypeKind_Struct:
    case SYMS_TypeKind_Class:
    {
      SYMS_U64 byte_align = 1;
      SYMS_TypeMember *mem = members;
      for (SYMS_TypeConsMember *node = list->first;
           node != 0;
           node = node->next, mem += 1){
        // update layout w/ alignment (guess work - we don't know for sure the align of this type)
        SYMS_U64 member_byte_size = node->type->byte_size;
        SYMS_U64 member_byte_align = 1;
        if (member_byte_size >= 8 && (member_byte_size & 0x7) == 0){
          member_byte_align = 8;
        }
        else if (member_byte_size >= 4 && (member_byte_size & 0x3) == 0){
          member_byte_align = 4;
        }
        else if (member_byte_size >= 2 && (member_byte_size & 0x1) == 0){
          member_byte_align = 2;
        }
        byte_align = SYMS_MAX(byte_align, member_byte_align);
        byte_size = SYMS_AlignPow2(byte_size, member_byte_align);
        
        // fill this member
        mem->kind = SYMS_MemKind_DataField;
        mem->visibility = SYMS_MemVisibility_Public;
        mem->flags = 0;
        mem->name = syms_type_string_cons(graph, node->name);
        mem->off = byte_size;
        mem->virtual_off = 0;
        mem->type = node->type;
        
        // update layout position
        byte_size += node->type->byte_size;
      }
      byte_size = SYMS_AlignPow2(byte_size, byte_align);
    }break;
    
    case SYMS_TypeKind_Union:
    {
      SYMS_U64 byte_align = 1;
      SYMS_TypeMember *mem = members;
      for (SYMS_TypeConsMember *node = list->first;
           node != 0;
           node = node->next, mem += 1){
        // fill this member
        mem->kind = SYMS_MemKind_DataField;
        mem->visibility = SYMS_MemVisibility_Public;
        mem->flags = 0;
        mem->name = syms_type_string_cons(graph, node->name);
        mem->off = 0;
        mem->virtual_off = 0;
        mem->type = node->type;
        
        // update layout computation
        SYMS_U64 member_byte_size = node->type->byte_size;
        SYMS_U64 member_byte_align = 1;
        if (member_byte_size >= 8 && (member_byte_size & 0x7) == 0){
          member_byte_align = 8;
        }
        else if (member_byte_size >= 4 && (member_byte_size & 0x3) == 0){
          member_byte_align = 4;
        }
        else if (member_byte_size >= 2 && (member_byte_size & 0x1) == 0){
          member_byte_align = 2;
        }
        byte_size = SYMS_MAX(byte_size, member_byte_size);
        byte_align = SYMS_MAX(byte_align, member_byte_align);
      }
      byte_size = SYMS_AlignPow2(byte_size, byte_align);
    }break;
  }
  
  // fill the node
  SYMS_String8 name_cons = syms_type_string_cons(graph, name);
  node->kind = kind;
  node->name = name_cons;
  node->byte_size = byte_size;
  node->lazy_ptr = array;
}

SYMS_API SYMS_TypeNode*
syms_type_cons_record_defer_members(SYMS_TypeGraph *graph,
                                    SYMS_TypeKind kind, SYMS_String8 name, SYMS_U64 byte_size,
                                    SYMS_TypeUniqueInfo *unique_opt){
  SYMS_String8 name_cons = syms_type_string_cons(graph, name);
  SYMS_TypeUniqueInfo *unique = syms_type_unique_copy(graph->arena, unique_opt);
  
  SYMS_TypeNode *result = syms_push_array_zero(graph->arena, SYMS_TypeNode, 1);
  result->kind = kind;
  result->name = name_cons;
  result->byte_size = byte_size;
  result->unique = unique;
  result->direct_type = &syms_type_node_nil;
  result->this_type = &syms_type_node_nil;
  
  syms_type_name_insert(graph->arena, &graph->name_buckets, name_cons.str, result);
  
  return(result);
}

SYMS_API SYMS_TypeNode*
syms_type_cons_enum_defer_members(SYMS_TypeGraph *graph,
                                  SYMS_String8 name, SYMS_TypeNode *underlying_type,
                                  SYMS_TypeUniqueInfo *unique_opt){
  SYMS_String8 name_cons = syms_type_string_cons(graph, name);
  SYMS_TypeUniqueInfo *unique = syms_type_unique_copy(graph->arena, unique_opt);
  
  SYMS_TypeNode *result = syms_push_array_zero(graph->arena, SYMS_TypeNode, 1);
  result->kind = SYMS_TypeKind_Enum;
  result->name = name_cons;
  result->byte_size = underlying_type->byte_size;
  result->unique = unique;
  result->direct_type = underlying_type;
  result->this_type = &syms_type_node_nil;
  
  syms_type_name_insert(graph->arena, &graph->name_buckets, name_cons.str, result);
  
  return(result);
}

SYMS_API SYMS_TypeNode*
syms_type_cons_typedef(SYMS_TypeGraph *graph,
                       SYMS_String8 name, SYMS_TypeNode *type,
                       SYMS_TypeUniqueInfo *unique_opt){
  SYMS_String8 name_cons = syms_type_string_cons(graph, name);
  SYMS_TypeUniqueInfo *unique = syms_type_unique_copy(graph->arena, unique_opt);
  
  SYMS_TypeNode *result = syms_push_array_zero(graph->arena, SYMS_TypeNode, 1);
  result->kind = SYMS_TypeKind_Typedef;
  result->name = name_cons;
  result->byte_size = type->byte_size;
  result->unique = unique;
  result->direct_type = type;
  result->this_type = &syms_type_node_nil;
  
  syms_type_name_insert(graph->arena, &graph->name_buckets, name_cons.str, result);
  
  return(result);
}

SYMS_API SYMS_TypeNode*
syms_type_cons_fwd(SYMS_TypeGraph *graph,
                   SYMS_TypeKind kind, SYMS_String8 name, SYMS_TypeNode *type,
                   SYMS_TypeUniqueInfo *unique_opt){
  SYMS_String8 name_cons = syms_type_string_cons(graph, name);
  SYMS_TypeUniqueInfo *unique = syms_type_unique_copy(graph->arena, unique_opt);
  
  SYMS_TypeNode *result = syms_push_array_zero(graph->arena, SYMS_TypeNode, 1);
  result->kind = kind;
  result->name = name_cons;
  result->byte_size = type->byte_size;
  result->unique = unique;
  result->direct_type = type;
  result->this_type = &syms_type_node_nil;
  
  return(result);
}

// constructor usid place holders

SYMS_API SYMS_TypeUSIDPlaceHolder
syms_type_usid_place_holder_insert(SYMS_TypeGraph *graph, SYMS_USID usid){
  SYMS_TypeUSIDPlaceHolder result = {0};
  result.usid_node = syms_type_usid_insert(graph->arena, &graph->usid_buckets, usid, &syms_type_node_nil);
  return(result);
}

SYMS_API void
syms_type_usid_place_holder_replace(SYMS_TypeGraph *graph, SYMS_TypeUSIDPlaceHolder *place,
                                    SYMS_TypeNode *node){
  place->usid_node->type = node;
  syms_memzero_struct(place);
}

// artificial records helpers

SYMS_API SYMS_TypeMember*
syms_type_equip_mems_pre_allocate(SYMS_TypeGraph *graph, SYMS_TypeNode *node, SYMS_U64 member_count){
  SYMS_TypeMember *result = 0;
  if (node->lazy_ptr == 0 && syms_type_kind_is_record(node->kind)){
    SYMS_TypeMemberArray *array = syms_push_array(graph->arena, SYMS_TypeMemberArray, 1);
    SYMS_TypeMember *members = syms_push_array_zero(graph->arena, SYMS_TypeMember, member_count);
    array->mems = members;
    array->count = member_count;
    node->lazy_ptr = array;
    result = members;
  }
  return(result);
}


// artificial enums helpers

SYMS_API SYMS_EnumMember*
syms_type_equip_enum_mems_pre_allocate(SYMS_TypeGraph *graph, SYMS_TypeNode *node, SYMS_U64 member_count){
  SYMS_EnumMember *result = 0;
  if (node->lazy_ptr == 0 && syms_type_kind_is_enum(node->kind)){
    SYMS_EnumMemberArray *array = syms_push_array(graph->arena, SYMS_EnumMemberArray, 1);
    SYMS_EnumMember *members = syms_push_array_zero(graph->arena, SYMS_EnumMember, member_count);
    array->enum_members = members;
    array->count = member_count;
    node->lazy_ptr = array;
    result = members;
  }
  return(result);
}


// constructor helpers

SYMS_API SYMS_TypeUniqueInfo*
syms_type_unique_copy(SYMS_Arena *arena, SYMS_TypeUniqueInfo *unique_opt){
  SYMS_TypeUniqueInfo *result = 0;
  if (unique_opt != 0){
    result = syms_push_array_zero(arena, SYMS_TypeUniqueInfo, 1);
    *result = *unique_opt;
  }
  return(result);
}

SYMS_API SYMS_TypeUniqueInfo
syms_type_unique_from_usid_src_coord(SYMS_USID usid, SYMS_SrcCoord *src_coord_opt){
  SYMS_TypeUniqueInfo result = {0};
  result.usid = usid;
  if (src_coord_opt != 0){
    result.src_coord = *src_coord_opt;
  }
  return(result);
}


////////////////////////////////
//~ allen: Type Info Operators

SYMS_API SYMS_TypeNode*
syms_type_resolved(SYMS_TypeNode *type){
  SYMS_TypeNode *result = type;
  for (;syms_type_kind_is_forward(result->kind) ||
       result->kind == SYMS_TypeKind_Modifier ||
       result->kind == SYMS_TypeKind_Typedef;){
    result = result->direct_type;
  }
  return(result);
}

SYMS_API SYMS_B32
syms_type_node_match(SYMS_TypeNode *l, SYMS_TypeNode *r){
  SYMS_B32 result = syms_false;
  
  // resolve forward references
  SYMS_TypeNode *lres = syms_type_resolved(l);
  SYMS_TypeNode *rres = syms_type_resolved(r);
  
  if (lres == rres){
    result = syms_true;
  }
  else{
    if (lres->kind == rres->kind){
      switch (lres->kind){
        default:
        {
          result = syms_true;
        }break;
        
        case SYMS_TypeKind_Struct:
        case SYMS_TypeKind_Union:
        case SYMS_TypeKind_Enum:
        case SYMS_TypeKind_Typedef:
        {
          if (lres->unique != 0 && rres->unique != 0){
            SYMS_USID lusid = lres->unique->usid;
            SYMS_USID rusid = rres->unique->usid;
            if (lusid.uid == rusid.uid && lusid.sid == rusid.sid){
              result = syms_true;
            }
          }
        }break;
        
        case SYMS_TypeKind_Ptr:
        case SYMS_TypeKind_LValueReference:
        case SYMS_TypeKind_RValueReference:
        {
          if (syms_type_node_match(lres->direct_type, rres->direct_type)){
            result = syms_true;
          }
        }break;
        
        case SYMS_TypeKind_MemberPtr:
        {
          if (syms_type_node_match(lres->direct_type, rres->direct_type) &&
              syms_type_node_match(lres->this_type, rres->this_type)){
            result = syms_true;
          }
        }break;
        
        case SYMS_TypeKind_Array:
        {
          if (lres->array_count == rres->array_count &&
              syms_type_node_match(lres->direct_type, rres->direct_type)){
            result = syms_true;
          }
        }break;
        
        case SYMS_TypeKind_Proc:
        {
          if (lres->proc.param_count == rres->proc.param_count &&
              syms_type_node_match(lres->direct_type, rres->direct_type) &&
              syms_type_node_match(lres->this_type, rres->this_type)){
            SYMS_B32 params_match = syms_true;
            SYMS_TypeNode **lp = lres->proc.params;
            SYMS_TypeNode **rp = rres->proc.params;
            SYMS_U64 count = lres->proc.param_count;
            for (SYMS_U64 i = 0; i < count; i += 1, lp += 1, rp += 1){
              if (!syms_type_node_match(*lp, *rp)){
                params_match = syms_false;
                break;
              }
            }
            result = params_match;
          }
        }break;
      }
    }
  }
  return(result);
}

SYMS_API SYMS_TypeNode*
syms_type_resolve_enum_to_basic(SYMS_TypeGraph *graph, SYMS_TypeNode *t){
  SYMS_TypeNode *result = t;
  if (t->kind == SYMS_TypeKind_Enum){
    result = t->direct_type;
  }
  return(result);
}

SYMS_API SYMS_TypeNode*
syms_type_promoted_from_type_node(SYMS_TypeGraph *graph, SYMS_TypeNode *c){
  SYMS_TypeNode *result = c;
  SYMS_TypeKind kind = c->kind;
  if (kind == SYMS_TypeKind_Bool ||
      kind == SYMS_TypeKind_Int8  ||
      kind == SYMS_TypeKind_Int16 ||
      kind == SYMS_TypeKind_UInt8 ||
      kind == SYMS_TypeKind_UInt16){
    result = syms_type_cons_basic(graph, SYMS_TypeKind_Int32, 4, syms_str8_lit("int32"));
  }
  return(result);
}

SYMS_API SYMS_TypeNode*
syms_type_auto_casted_from_type_nodes(SYMS_TypeGraph *graph, SYMS_TypeNode *l, SYMS_TypeNode *r){
  SYMS_ASSERT(syms_type_kind_is_basic_or_enum(l->kind) &&
              syms_type_kind_is_basic_or_enum(r->kind));
  
  // replace enums with corresponding ints
  SYMS_TypeNode *lt = syms_type_resolve_enum_to_basic(graph, l);
  SYMS_TypeNode *rt = syms_type_resolve_enum_to_basic(graph, r);
  
  // first promote each
  SYMS_TypeNode *lp = syms_type_promoted_from_type_node(graph, lt);
  SYMS_TypeNode *rp = syms_type_promoted_from_type_node(graph, rt);
  SYMS_TypeKind lk = lp->kind;
  SYMS_TypeKind rk = rp->kind;
  
  // pick the type with higher rank
  static SYMS_TypeKind type_table[] = {
    SYMS_TypeKind_Float64,
    SYMS_TypeKind_Float32,
    SYMS_TypeKind_UInt64,
    SYMS_TypeKind_Int64,
    SYMS_TypeKind_UInt32,
  };
  SYMS_TypeKind k = SYMS_TypeKind_Int32;
  for (SYMS_U64 i = 0; i < SYMS_ARRAY_SIZE(type_table); i += 1){
    SYMS_TypeKind check = type_table[i];
    if (check == lk || check == rk){
      k = check;
      break;
    }
  }
  
  // construct the output type
  SYMS_TypeNode *result = &syms_type_node_nil;
  if (k == lk){
    result = l;
  }
  else if (k == rk){
    result = r;
  }
  else{
    SYMS_String8 name = syms_string_from_enum_value(SYMS_TypeKind, k);
    SYMS_U64 bit_size = syms_bit_size_from_type_kind(k);
    SYMS_U64 byte_size = bit_size >> 3;
    result = syms_type_cons_basic(graph, k, byte_size, name);
  }
  
  return(result);
}

////////////////////////////////
//~ allen: Type Stringizing

SYMS_API SYMS_String8
syms_type_string_from_type(SYMS_Arena *arena, SYMS_TypeNode *type){
  SYMS_ProfBegin("syms_type_string_from_type");
  SYMS_ArenaTemp scratch = syms_get_scratch(&arena, 1);
  SYMS_String8List list = {0};
  syms_type_lhs_string_from_type(scratch.arena, type, &list);
  syms_type_rhs_string_from_type(scratch.arena, type, &list);
  SYMS_String8 result = syms_string_list_join(arena, &list, 0);
  syms_release_scratch(scratch);
  SYMS_ProfEnd();
  return(result);
}

SYMS_API void
syms_type_lhs_string_from_type(SYMS_Arena *arena, SYMS_TypeNode *type, SYMS_String8List *out){
  SYMS_ProfBegin("syms_type_lhs_string_from_type");
  syms_type_lhs_string_from_type__internal(arena, type, out, 0, syms_false);
  SYMS_ProfEnd();
}

SYMS_API void
syms_type_rhs_string_from_type(SYMS_Arena *arena, SYMS_TypeNode *type, SYMS_String8List *out){
  SYMS_ProfBegin("syms_type_rhs_string_from_type");
  syms_type_rhs_string_from_type__internal(arena, type, out, 0);
  SYMS_ProfEnd();
}

SYMS_API void
syms_type_lhs_string_from_type_skip_return(SYMS_Arena *arena, SYMS_TypeNode *type, SYMS_String8List *out){
  SYMS_ProfBegin("syms_type_lhs_string_from_type_skip_return");
  syms_type_lhs_string_from_type__internal(arena, type, out, 0, syms_true);
  SYMS_ProfEnd();
}

SYMS_API void
syms_type_lhs_string_from_type__internal(SYMS_Arena *arena, SYMS_TypeNode *type, SYMS_String8List *out,
                                         SYMS_U32 precedence_level, SYMS_B32 skip_return){
  switch (type->kind){
    default:
    {
      syms_string_list_push(arena, out, type->name);
      syms_string_list_push(arena, out, syms_str8_lit(" "));
    }break;
    
    case SYMS_TypeKind_Bitfield:
    {
      syms_type_lhs_string_from_type__internal(arena, type->direct_type, out, precedence_level, skip_return);
    }break;
    
    case SYMS_TypeKind_Modifier:
    {
      syms_type_lhs_string_from_type__internal(arena, type->direct_type, out, 1, skip_return);
      if (type->mods & SYMS_TypeModifier_Const){
        syms_string_list_push(arena, out, syms_str8_lit("const "));
      }
      if (type->mods & SYMS_TypeModifier_Restrict){
        syms_string_list_push(arena, out, syms_str8_lit("restrict "));
      }
      if (type->mods & SYMS_TypeModifier_Volatile){
        syms_string_list_push(arena, out, syms_str8_lit("volatile "));
      }
    }break;
    
    case SYMS_TypeKind_Variadic:
    {
      syms_string_list_push(arena, out, syms_str8_lit("..."));
    }break;
    
    case SYMS_TypeKind_Label:
    {
      // TODO(allen): ???
    }break;
    
    case SYMS_TypeKind_Struct:
    case SYMS_TypeKind_Union:
    case SYMS_TypeKind_Enum:
    case SYMS_TypeKind_Class:
    case SYMS_TypeKind_Typedef:
    {
      syms_string_list_push(arena, out, type->name);
      syms_string_list_push(arena, out, syms_str8_lit(" "));
    }break;
    
    case SYMS_TypeKind_ForwardStruct:
    case SYMS_TypeKind_ForwardUnion:
    case SYMS_TypeKind_ForwardEnum:
    case SYMS_TypeKind_ForwardClass:
    {
      SYMS_String8 keyword = syms_string_from_enum_value(SYMS_TypeKind, type->kind);
      syms_string_list_push(arena, out, keyword);
      syms_string_list_push(arena, out, syms_str8_lit(" "));
      syms_string_list_push(arena, out, type->name);
      syms_string_list_push(arena, out, syms_str8_lit(" "));
    }break;
    
    case SYMS_TypeKind_Array:
    {
      syms_type_lhs_string_from_type__internal(arena, type->direct_type, out, 2, skip_return);
      if (precedence_level == 1){
        syms_string_list_push(arena, out, syms_str8_lit("("));
      }
    }break;
    
    case SYMS_TypeKind_Proc:
    {
      if (!skip_return){
        syms_type_lhs_string_from_type__internal(arena, type->direct_type, out, 2, syms_false);
      }
      if (precedence_level == 1){
        syms_string_list_push(arena, out, syms_str8_lit("("));
      }
    }break;
    
    case SYMS_TypeKind_Ptr:
    {
      syms_type_lhs_string_from_type__internal(arena, type->direct_type, out, 1, skip_return);
      syms_string_list_push(arena, out, syms_str8_lit("*"));
    }break;
    
    case SYMS_TypeKind_LValueReference:
    {
      syms_type_lhs_string_from_type__internal(arena, type->direct_type, out, 1, skip_return);
      syms_string_list_push(arena, out, syms_str8_lit("&"));
    }break;
    
    case SYMS_TypeKind_RValueReference:
    {
      syms_type_lhs_string_from_type__internal(arena, type->direct_type, out, 1, skip_return);
      syms_string_list_push(arena, out, syms_str8_lit("&&"));
    }break;
    
    case SYMS_TypeKind_MemberPtr:
    {
      syms_type_lhs_string_from_type__internal(arena, type->direct_type, out, 1, skip_return);
      SYMS_TypeNode *container = type->this_type;
      if (container != 0){
        syms_string_list_push(arena, out, container->name);
      }
      else{
        syms_string_list_push(arena, out, syms_str8_lit("<unknown-class>"));
      }
      syms_string_list_push(arena, out, syms_str8_lit("::*"));
    }break;
  }
}

SYMS_API void
syms_type_rhs_string_from_type__internal(SYMS_Arena *arena, SYMS_TypeNode *type, SYMS_String8List *out,
                                         SYMS_U32 precedence_level){
  switch (type->kind){
    default:break;
    
    case SYMS_TypeKind_Bitfield:
    {
      syms_type_rhs_string_from_type__internal(arena, type->direct_type, out, precedence_level);
    }break;
    
    case SYMS_TypeKind_Modifier:
    case SYMS_TypeKind_Ptr:
    case SYMS_TypeKind_LValueReference:
    case SYMS_TypeKind_RValueReference:
    case SYMS_TypeKind_MemberPtr:
    {
      syms_type_rhs_string_from_type__internal(arena, type->direct_type, out, 1);
    }break;
    
    case SYMS_TypeKind_Array:
    {
      if (precedence_level == 1){
        syms_string_list_push(arena, out, syms_str8_lit(")"));
      }
      
      SYMS_String8 count_str = syms_string_from_u64(arena, type->array_count);
      syms_string_list_push(arena, out, syms_str8_lit("["));
      syms_string_list_push(arena, out, count_str);
      syms_string_list_push(arena, out, syms_str8_lit("]"));
      
      syms_type_rhs_string_from_type__internal(arena, type->direct_type, out, 2);
    }break;
    
    case SYMS_TypeKind_Proc:
    {
      if (precedence_level == 1){
        syms_string_list_push(arena, out, syms_str8_lit(")"));
      }
      
      // parameters
      if (type->proc.param_count == 0){
        syms_string_list_push(arena, out, syms_str8_lit("(void)"));
      }
      else{
        syms_string_list_push(arena, out, syms_str8_lit("("));
        SYMS_U64 param_count = type->proc.param_count;
        SYMS_TypeNode **param = type->proc.params;
        for (SYMS_U64 i = 0; i < param_count; i += 1, param += 1){
          SYMS_String8 param_str = syms_type_string_from_type(arena, *param);
          SYMS_String8 param_str_trimmed = syms_str8_skip_chop_whitespace(param_str);
          syms_string_list_push(arena, out, param_str_trimmed);
          if (i + 1 < param_count){
            syms_string_list_push(arena, out, syms_str8_lit(", "));
          }
        }
        syms_string_list_push(arena, out, syms_str8_lit(")"));
      }
      
      syms_type_rhs_string_from_type__internal(arena, type->direct_type, out, 2);
    }break;
    
    case SYMS_TypeKind_Label:
    {
      // TODO(allen): ???
    }break;
  }
}

////////////////////////////////
//~ allen: Type Info Construct From Dbg Info

SYMS_API SYMS_TypeNode*
syms_type_from_dbg_sid(SYMS_TypeGraph *graph, SYMS_TypeParseParams *params, SYMS_SymbolID sid){
  SYMS_ProfBegin("syms_type_from_dbg_sid");
  //- look at unit features
  SYMS_TypeNode *result = &syms_type_node_nil;
  SYMS_UnitInfo info = syms_unit_info_from_uid(params->unit_set, params->uid);
  if (info.features & SYMS_UnitFeature_Types){
    result = syms_type_from_dbg_sid__rec(graph, params, sid);
  }
  SYMS_ProfEnd();
  return(result);
}

SYMS_API SYMS_TypeNode*
syms_type_from_dbg_sid__rec(SYMS_TypeGraph *graph, SYMS_TypeParseParams *params, SYMS_SymbolID sid){
  SYMS_USID usid = {params->uid, sid};
  
  //- get cached version of type
  SYMS_TypeNode *type_from_cache = syms_type_from_usid(graph, usid);
  
  //- when the cache is empty, immediately put in a place holder to stop data cycles from
  //  becoming infinite recusion - later, if we successfully finish constructing a type,
  //  we'll replace the place holder - otherwise it can sit there forever.
  SYMS_TypeUSIDPlaceHolder usid_place_holder = {0};
  if (type_from_cache == 0){
    usid_place_holder = syms_type_usid_place_holder_insert(graph, usid);
  }
  
  //- setup for constructing new type
  SYMS_SymbolKind kind = SYMS_SymbolKind_Null;
  if (type_from_cache == 0){
    kind = syms_symbol_kind_from_sid(params->data, params->dbg, params->unit, sid);
  }
  
  //- construct new type node
  SYMS_TypeNode *new_type = 0;
  if (type_from_cache == 0 && kind == SYMS_SymbolKind_Type){
    SYMS_ArenaTemp scratch = syms_get_scratch(0, 0);
    
    SYMS_TypeInfo type_info = syms_type_info_from_sid(params->data, params->dbg, params->unit, sid);
    
    if (syms_type_kind_is_basic(type_info.kind)){
      SYMS_String8 name = syms_symbol_name_from_sid(scratch.arena, params->data, params->dbg, params->unit, sid);
      new_type = syms_type_cons_basic(graph, type_info.kind, type_info.reported_size, name);
    }
    else{
      switch (type_info.kind){
        case SYMS_TypeKind_Struct:
        case SYMS_TypeKind_Class:
        case SYMS_TypeKind_Union:
        {
          // get name
          SYMS_String8 name = syms_symbol_name_from_sid(scratch.arena, params->data, params->dbg, params->unit, sid);
          
          // get size
          SYMS_U64 byte_size = 0;
          if (type_info.reported_size_interp == SYMS_SizeInterpretation_ByteCount){
            byte_size = type_info.reported_size;
          }
          
          // unique info
          SYMS_TypeUniqueInfo unique = syms_type_unique_from_usid_src_coord(usid, &type_info.src_coord);
          
          // construct type
          new_type = syms_type_cons_record_defer_members(graph, type_info.kind, name, byte_size, &unique);
        }break;
        
        case SYMS_TypeKind_Enum:
        {
          // get name
          SYMS_String8 name = syms_symbol_name_from_sid(scratch.arena, params->data, params->dbg, params->unit, sid);
          
          // get direct type
          SYMS_TypeNode *direct = &syms_type_node_nil;
          if (type_info.direct_type.uid == params->uid){
            direct = syms_type_from_dbg_sid__rec(graph, params, type_info.direct_type.sid);
          }
          
          // unique info
          SYMS_TypeUniqueInfo unique = syms_type_unique_from_usid_src_coord(usid, &type_info.src_coord);
          
          // construct type
          new_type = syms_type_cons_enum_defer_members(graph, name, direct, &unique);
        }break;
        
        case SYMS_TypeKind_Typedef:
        {
          // get name
          SYMS_String8 name = syms_symbol_name_from_sid(scratch.arena, params->data, params->dbg, params->unit, sid);
          
          // get direct type
          SYMS_TypeNode *direct = &syms_type_node_nil;
          if (type_info.direct_type.uid == params->uid){
            direct = syms_type_from_dbg_sid__rec(graph, params, type_info.direct_type.sid);
          }
          
          // unique info
          SYMS_TypeUniqueInfo unique = syms_type_unique_from_usid_src_coord(usid, &type_info.src_coord);
          
          // construct type
          new_type = syms_type_cons_typedef(graph, name, direct, &unique);
        }break;
        
        case SYMS_TypeKind_ForwardStruct:
        case SYMS_TypeKind_ForwardClass:
        case SYMS_TypeKind_ForwardUnion:
        case SYMS_TypeKind_ForwardEnum:
        {
          // get name
          SYMS_String8 name = syms_symbol_name_from_sid(scratch.arena, params->data, params->dbg, params->unit, sid);
          
          // resolve references
          SYMS_TypeNode *referenced_type = &syms_type_node_nil;
          if (type_info.reported_size_interp == SYMS_SizeInterpretation_ResolveForwardReference){
            SYMS_TypeKind match_kind = syms_type_kind_main_from_fwd(type_info.kind);
            
            SYMS_USID match_usid = {0};
            if (params->type_map != 0){
              SYMS_USIDList matches = syms_usid_list_from_string(scratch.arena, params->data, params->dbg,
                                                                 params->type_map, name);
              for (SYMS_USIDNode *node = matches.first;
                   node != 0;
                   node = node->next){
                if (node->usid.uid == params->uid){
                  SYMS_TypeInfo check_info = syms_type_info_from_sid(params->data, params->dbg, params->unit,
                                                                     node->usid.sid);
                  if (check_info.kind == match_kind){
                    match_usid = node->usid;
                    break;
                  }
                }
              }
            }
            
            if (match_usid.uid == params->uid){
              referenced_type = syms_type_from_dbg_sid__rec(graph, params, match_usid.sid);
            }
          }
          
          // unique info
          SYMS_TypeUniqueInfo unique = syms_type_unique_from_usid_src_coord(usid, &type_info.src_coord);
          
          // construct type
          new_type = syms_type_cons_fwd(graph, type_info.kind, name, referenced_type, &unique);
        }break;
        
        case SYMS_TypeKind_Bitfield:
        {
          // get direct type
          SYMS_TypeNode *direct = &syms_type_node_nil;
          if (type_info.direct_type.uid == params->uid){
            direct = syms_type_from_dbg_sid__rec(graph, params, type_info.direct_type.sid);
          }
          
          // TODO(allen): get actual bit off & count
          
          // construct type
          new_type = syms_type_cons_bitfield(graph, direct, 0, 0);
        }break;
        
        case SYMS_TypeKind_Modifier:
        {
          // get direct type
          SYMS_TypeNode *direct = &syms_type_node_nil;
          if (type_info.direct_type.uid == params->uid){
            direct = syms_type_from_dbg_sid__rec(graph, params, type_info.direct_type.sid);
          }
          
          // construct type
          new_type = syms_type_cons_mod(graph, direct, type_info.mods);
        }break;
        
        case SYMS_TypeKind_Ptr:
        case SYMS_TypeKind_LValueReference:
        case SYMS_TypeKind_RValueReference:
        {
          // get direct type
          SYMS_TypeNode *direct = &syms_type_node_nil;
          if (type_info.direct_type.uid == params->uid){
            direct = syms_type_from_dbg_sid__rec(graph, params, type_info.direct_type.sid);
          }
          
          // construct type
          new_type = syms_type_cons_ptr(graph, type_info.kind, direct);
        }break;
        
        case SYMS_TypeKind_Array:
        {
          // get direct type
          SYMS_TypeNode *direct = &syms_type_node_nil;
          if (type_info.direct_type.uid == params->uid){
            direct = syms_type_from_dbg_sid__rec(graph, params, type_info.direct_type.sid);
          }
          
          // determine array count
          SYMS_U64 array_count = type_info.reported_size;
          if (type_info.reported_size_interp == SYMS_SizeInterpretation_ByteCount){
            SYMS_U64 next_size = 1;
            if (direct->byte_size != 0){
              next_size = direct->byte_size;
            }
            array_count = type_info.reported_size/next_size;
          }
          
          // construct type
          new_type = syms_type_cons_array(graph, direct, array_count);
        }break;
        
        case SYMS_TypeKind_Proc:
        {
          // get direct type
          SYMS_TypeNode *direct = &syms_type_node_nil;
          if (type_info.direct_type.uid == params->uid){
            direct = syms_type_from_dbg_sid__rec(graph, params, type_info.direct_type.sid);
          }
          
          // read signature
          SYMS_SigInfo sig_info =
            syms_sig_info_from_type_sid(scratch.arena, params->data, params->dbg, params->unit, sid);
          
          SYMS_U64 param_count = sig_info.param_type_ids.count;
          SYMS_TypeNode **param_types = syms_push_array(scratch.arena, SYMS_TypeNode*, param_count);
          
          {
            SYMS_TypeNode **param_ptr = param_types;
            SYMS_TypeNode **param_opl = param_types + param_count;
            SYMS_SymbolID *sid_ptr = sig_info.param_type_ids.ids;
            for (;param_ptr < param_opl; sid_ptr += 1, param_ptr += 1){
              SYMS_USID param_usid = syms_make_usid(sig_info.uid, *sid_ptr);
              SYMS_TypeNode *param = &syms_type_node_nil;
              if (param_usid.uid == params->uid){
                param = syms_type_from_dbg_sid__rec(graph, params, param_usid.sid);
              }
              *param_ptr = param;
            }
          }
          
          SYMS_USID this_usid = syms_make_usid(sig_info.uid, sig_info.this_type_id);
          SYMS_TypeNode *this_type = &syms_type_node_nil;
          if (type_info.direct_type.uid == params->uid){
            this_type = syms_type_from_dbg_sid__rec(graph, params, this_usid.sid);
          }
          
          // construct type
          new_type = syms_type_cons_proc(graph, direct, this_type, param_types, param_count);
        }break;
        
        case SYMS_TypeKind_MemberPtr:
        {
          // get direct type
          SYMS_TypeNode *direct = &syms_type_node_nil;
          if (type_info.direct_type.uid == params->uid){
            direct = syms_type_from_dbg_sid__rec(graph, params, type_info.direct_type.sid);
          }
          
          // get container type
          SYMS_TypeNode *container = &syms_type_node_nil;
          if (type_info.containing_type.uid == params->uid){
            container = syms_type_from_dbg_sid__rec(graph, params, type_info.containing_type.sid);
          }
          
          // construct type
          new_type = syms_type_cons_mem_ptr(graph, container, direct);
          // TODO(allen): better flow here
          if (type_info.reported_size_interp == SYMS_SizeInterpretation_ByteCount){
            new_type->byte_size = type_info.reported_size;
          }
        }break;
        
        case SYMS_TypeKind_Variadic:
        {
          // TODO(allen): ?
        }break;
        
        case SYMS_TypeKind_Label:
        {
          // TODO(allen): ?
        }break;
      }
    }
    
    //- split out modifiers into stand alone nodes
    if (type_info.mods != 0 && type_info.kind != SYMS_TypeKind_Modifier){
      SYMS_TypeNode *direct = new_type;
      new_type = syms_type_cons_mod(graph, direct, type_info.mods);
    }
    
    // save this type as the final value for this usid
    syms_type_usid_place_holder_replace(graph, &usid_place_holder, new_type);
    
    syms_release_scratch(scratch);
  }
  
  //- set result
  SYMS_TypeNode *result = type_from_cache;
  if (result == 0){
    result = new_type;
  }
  
  //- null -> nil
  if (result == 0){
    result = &syms_type_node_nil;
  }
  
  return(result);
}

SYMS_API void
syms_type_equip_members_from_dbg(SYMS_TypeGraph *graph, SYMS_TypeParseParams *params, SYMS_TypeNode *node){
  SYMS_ProfBegin("syms_type_equip_members_from_dbg");
  
  if ((syms_type_kind_is_record(node->kind) || 
       syms_type_kind_is_enum(node->kind)) &&
      !syms_type_members_are_equipped(graph, node)){
    SYMS_SymbolID sid = 0;
    if (node->unique != 0){
      sid = node->unique->usid.sid;
    }
    
    if (sid != 0){
      switch (node->kind){
        case SYMS_TypeKind_Struct:
        case SYMS_TypeKind_Union:
        case SYMS_TypeKind_Class:
        {
          SYMS_ArenaTemp scratch = syms_get_scratch(0, 0);
          
          SYMS_MemsAccel *mems_accel =
            syms_mems_accel_from_sid(scratch.arena, params->data, params->dbg, params->unit, sid);
          SYMS_U64 mem_count = syms_mem_count_from_mems(mems_accel);
          SYMS_TypeMember *mems = syms_type_equip_mems_pre_allocate(graph, node, mem_count);
          
          SYMS_TypeMember *mem_ptr = mems;
          for (SYMS_U64 n = 1; n <= mem_count; n += 1, mem_ptr += 1){
            SYMS_ArenaTemp temp = syms_arena_temp_begin(scratch.arena);
            
            SYMS_MemInfo mem_info =
              syms_mem_info_from_number(graph->arena, params->data, params->dbg, params->unit, mems_accel, n);
            
            // determine the member's type
            SYMS_TypeNode *type = &syms_type_node_nil;
            switch (mem_info.kind){
              case SYMS_MemKind_VTablePtr: /*no type*/ break;
              
              case SYMS_MemKind_DataField:
              case SYMS_MemKind_StaticData:
              {
                // TODO(allen): here we lose access to the fast path from StaticData to a virtual offset.
                // This path is only available from the DWARF backend, but there are a few ways we could
                // preserve it.
                SYMS_USID type_usid = syms_type_from_mem_number(params->data, params->dbg, params->unit, mems_accel, n);
                if (type_usid.uid == params->uid){
                  type = syms_type_from_dbg_sid(graph, params, type_usid.sid);
                }
              }break;
              
              case SYMS_MemKind_BaseClass:
              case SYMS_MemKind_VBaseClassPtr:
              case SYMS_MemKind_NestedType:
              {
                // directly get type from member id
                SYMS_USID type_usid = syms_type_from_mem_number(params->data, params->dbg, params->unit, mems_accel, n);
                if (type_usid.uid == params->uid){
                  type = syms_type_from_dbg_sid(graph, params, type_usid.sid);
                }
              }break;
              
              case SYMS_MemKind_Method:
              case SYMS_MemKind_StaticMethod:
              {
                // TODO(allen): here we lose access to the fast path to a procedure symbol.
                // This path is only available from the DWARF backend, but there are a few ways we could
                // preserve it.
                SYMS_SigInfo sig =
                  syms_sig_info_from_mem_number(scratch.arena, params->data, params->dbg, params->unit, mems_accel, n);
                
                SYMS_TypeNode *ret_type  = &syms_type_node_nil;
                SYMS_TypeNode *this_type = &syms_type_node_nil;
                if (sig.uid == params->uid){
                  ret_type  = syms_type_from_dbg_sid(graph, params, sig.return_type_id);
                  this_type = syms_type_from_dbg_sid(graph, params, sig.this_type_id);
                }
                
                SYMS_U64 param_count = sig.param_type_ids.count;
                SYMS_TypeNode **param_types = syms_push_array(graph->arena, SYMS_TypeNode*, param_count);
                {
                  SYMS_TypeNode **param_ptr = param_types;
                  SYMS_TypeNode **param_opl = param_types + param_count;
                  SYMS_SymbolID *param_id_ptr = sig.param_type_ids.ids;
                  for (; param_ptr < param_opl; param_id_ptr += 1, param_ptr += 1){
                    *param_ptr = syms_type_from_dbg_sid(graph, params, *param_id_ptr);
                  }
                }
                
                type = syms_type_cons_proc(graph, ret_type, this_type, param_types, param_count);
              }break;
            }
            
            // fill the member slot
            mem_ptr->kind = mem_info.kind;
            mem_ptr->visibility = mem_info.visibility;
            mem_ptr->flags = mem_info.flags;
            mem_ptr->name = syms_type_string_cons(graph, mem_info.name);
            mem_ptr->off = mem_info.off;
            mem_ptr->virtual_off = mem_info.virtual_off;
            mem_ptr->type = type;
            
            syms_arena_temp_end(temp);
          }
          
          syms_release_scratch(scratch);
        }break;
        
        case SYMS_TypeKind_Enum:
        {
          SYMS_ArenaTemp scratch = syms_get_scratch(0, 0);
          
          SYMS_EnumMemberArray array = syms_enum_member_array_from_sid(scratch.arena, params->data, params->dbg,
                                                                       params->unit, sid);
          
          SYMS_U64 mem_count = array.count;
          SYMS_EnumMember *mems = syms_type_equip_enum_mems_pre_allocate(graph, node, mem_count);
          SYMS_EnumMember *dst_ptr = mems;
          SYMS_EnumMember *src_ptr = array.enum_members;
          for (SYMS_U64 i = 0; i < mem_count; i += 1, src_ptr += 1, dst_ptr += 1){
            dst_ptr->name = syms_type_string_cons(graph, src_ptr->name);
            dst_ptr->val = src_ptr->val;
          }
          
          syms_release_scratch(scratch);
        }break;
      }
    }
  }
  
  SYMS_ProfEnd();
}

////////////////////////////////
//~ allen: Type Content Table Functions

SYMS_API SYMS_U64
syms_type_content_hash(SYMS_String8 data){
  SYMS_U64 result = syms_hash_djb2(data);
  return(result);
}

SYMS_API SYMS_TypeNode*
syms_type_from_content_buckets(SYMS_TypeContentBuckets *buckets, SYMS_String8 data){
  SYMS_ProfBegin("syms_type_from_content");
  SYMS_TypeNode *result = 0;
  if (buckets->buckets != 0){
    SYMS_U64 hash = syms_type_content_hash(data);
    SYMS_U64 bucket_index = hash%SYMS_TYPE_GRAPH_TABLE_BUCKET_COUNT;
    for (SYMS_TypeContentNode *node = buckets->buckets[bucket_index];
         node != 0;
         node = node->next){
      if (node->hash == hash && syms_string_match(node->key, data, 0)){
        result = node->type;
        break;
      }
    }
  }
  SYMS_ProfEnd();
  return(result);
}

SYMS_API SYMS_String8
syms_type_content_insert(SYMS_Arena *arena, SYMS_TypeContentBuckets *buckets,
                         SYMS_String8 key, SYMS_TypeNode *type){
  SYMS_ProfBegin("syms_type_content_insert");
  SYMS_U64 hash = syms_type_content_hash(key);
  SYMS_String8 result = {0};
  if (buckets->buckets != 0){
    SYMS_U64 bucket_index = hash%SYMS_TYPE_GRAPH_TABLE_BUCKET_COUNT;
    
    result = syms_push_string_copy(arena, key);
    
    SYMS_TypeContentNode *new_node = syms_push_array(arena, SYMS_TypeContentNode, 1);
    SYMS_StackPush(buckets->buckets[bucket_index], new_node);
    new_node->key = result;
    new_node->hash = hash;
    new_node->type = type;
  }
  SYMS_ProfEnd();
  return(result);
}

////////////////////////////////
//~ allen: Type USID Table Functions

SYMS_API SYMS_U64
syms_type_usid_hash(SYMS_USID usid){
  SYMS_U64 result = syms_hash_djb2(syms_str8((SYMS_U8 *)&usid, sizeof(usid)));
  return(result);
}

SYMS_API SYMS_TypeNode*
syms_type_from_usid_buckets(SYMS_TypeUSIDBuckets *buckets, SYMS_USID usid){
  SYMS_ProfBegin("syms_type_from_usid");
  SYMS_TypeNode *result = 0;
  if (buckets->buckets != 0){
    SYMS_U64 hash = syms_type_usid_hash(usid);
    SYMS_U64 bucket_index = hash%SYMS_TYPE_GRAPH_TABLE_BUCKET_COUNT;
    for (SYMS_TypeUSIDNode *node = buckets->buckets[bucket_index];
         node != 0;
         node = node->next){
      SYMS_USID key = node->key;
      if (key.uid == usid.uid && key.sid == usid.sid){
        result = node->type;
        break;
      }
    }
  }
  SYMS_ProfEnd();
  return(result);
}

SYMS_API SYMS_TypeUSIDNode*
syms_type_usid_insert(SYMS_Arena *arena, SYMS_TypeUSIDBuckets *buckets, SYMS_USID key, SYMS_TypeNode *type){
  SYMS_TypeUSIDNode *result = 0;
  
  SYMS_ProfBegin("syms_type_usid_insert");
  if (buckets->buckets != 0){
    SYMS_U64 hash = syms_type_usid_hash(key);
    SYMS_U64 bucket_index = hash%SYMS_TYPE_GRAPH_TABLE_BUCKET_COUNT;
    SYMS_TypeUSIDNode *new_node = syms_push_array(arena, SYMS_TypeUSIDNode, 1);
    SYMS_StackPush(buckets->buckets[bucket_index], new_node);
    new_node->key = key;
    new_node->type = type;
    
    result = new_node;
    buckets->bucket_counts[bucket_index] += 1;
  }
  SYMS_ProfEnd();
  
  return(result);
}

////////////////////////////////
//~ allen: Type Name Table Functions

SYMS_API SYMS_U64
syms_type_name_hash(SYMS_U8 *ptr){
  SYMS_U64 u64 = (SYMS_U64)(ptr);
  SYMS_U64 bits = u64 >> 3;
  SYMS_U64 result = bits ^ (bits << 32);
  return(result);
}

SYMS_API SYMS_TypeChain*
syms_type_chain_from_name_buckets(SYMS_TypeNameBuckets *buckets, SYMS_U8 *name_ptr){
  SYMS_TypeChain *result = 0;
  SYMS_ProfBegin("syms_type_chain_from_name");
  if (buckets->buckets != 0){
    SYMS_U64 hash = syms_type_name_hash(name_ptr);
    SYMS_U64 bucket_index = hash%SYMS_TYPE_GRAPH_TABLE_BUCKET_COUNT;
    for (SYMS_TypeNameNode *node = buckets->buckets[bucket_index];
         node != 0;
         node = node->next){
      if (node->name_ptr == name_ptr){
        result = &node->chain;
        break;
      }
    }
  }
  SYMS_ProfEnd();
  return(result);
}

SYMS_API void
syms_type_name_insert(SYMS_Arena *arena, SYMS_TypeNameBuckets *buckets, SYMS_U8 *name_ptr, SYMS_TypeNode *type){
  SYMS_ProfBegin("syms_type_name_insert");
  if (buckets->buckets != 0){
    // find existing name node
    SYMS_TypeNameNode *match = 0;
    SYMS_U64 hash = syms_type_name_hash(name_ptr);
    SYMS_U64 bucket_index = hash%SYMS_TYPE_GRAPH_TABLE_BUCKET_COUNT;
    for (SYMS_TypeNameNode *node = buckets->buckets[bucket_index];
         node != 0;
         node = node->next){
      if (node->name_ptr == name_ptr){
        match = node;
        break;
      }
    }
    
    // new name node if necessary
    if (match == 0){
      SYMS_TypeNameNode *new_node = syms_push_array_zero(arena, SYMS_TypeNameNode, 1);
      SYMS_StackPush(buckets->buckets[bucket_index], new_node);
      new_node->name_ptr = name_ptr;
      
      match = new_node;
    }
    
    // new type chain node
    {
      SYMS_TypeChainNode *new_chain = syms_push_array(arena, SYMS_TypeChainNode, 1);
      SYMS_QueuePush(match->chain.first, match->chain.last, new_chain);
      match->chain.count += 1;
      new_chain->type = type;
    }
  }
  SYMS_ProfEnd();
}

#endif //SYMS_TYPE_GRAPH_C
