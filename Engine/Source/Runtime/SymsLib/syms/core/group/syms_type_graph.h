// Copyright Epic Games, Inc. All Rights Reserved.
/* date = February 16th 2022 1:40 pm */

#ifndef SYMS_TYPE_GRAPH_H
#define SYMS_TYPE_GRAPH_H

// (Notes for maintainers at the bottom)

////////////////////////////////
//~ allen: Syms Type Constructor Helper Types

typedef struct SYMS_TypeConsMember{
  struct SYMS_TypeConsMember *next;
  SYMS_String8 name;
  struct SYMS_TypeNode *type;
} SYMS_TypeConsMember;

typedef struct SYMS_TypeConsMemberList{
  SYMS_TypeConsMember *first;
  SYMS_TypeConsMember *last;
  SYMS_U64 count;
} SYMS_TypeConsMemberList;

////////////////////////////////
//~ allen: Syms Cycle-Safe Type Constructor Helper Types

// NOTE(allen): This is a "linear" type (it should not be used more than once, it should not be coppied)

typedef struct SYMS_TypeUSIDPlaceHolder{
  struct SYMS_TypeUSIDNode *usid_node;
} SYMS_TypeUSIDPlaceHolder;

////////////////////////////////
//~ allen: Syms Type Graph Helper Tables

#define SYMS_TYPE_GRAPH_TABLE_BUCKET_COUNT 1024

// modifiers,ptrs,arrays,procs -> type node
typedef struct SYMS_TypeContentNode{
  struct SYMS_TypeContentNode *next;
  SYMS_String8 key;
  SYMS_U64 hash;
  struct SYMS_TypeNode *type;
} SYMS_TypeContentNode;

typedef struct SYMS_TypeContentBuckets{
  SYMS_TypeContentNode **buckets;
} SYMS_TypeContentBuckets;

// usid -> type node
typedef struct SYMS_TypeUSIDNode{
  struct SYMS_TypeUSIDNode *next;
  SYMS_USID key;
  struct SYMS_TypeNode *type;
} SYMS_TypeUSIDNode;

typedef struct SYMS_TypeUSIDBuckets{
  SYMS_TypeUSIDNode **buckets;
  SYMS_U64 *bucket_counts;
} SYMS_TypeUSIDBuckets;

// consed name -> chain of type nodes
typedef struct SYMS_TypeChainNode{
  struct SYMS_TypeChainNode *next;
  struct SYMS_TypeNode *type;
} SYMS_TypeChainNode;

typedef struct SYMS_TypeChain{
  SYMS_TypeChainNode *first;
  SYMS_TypeChainNode *last;
  SYMS_U64 count;
} SYMS_TypeChain;

typedef struct SYMS_TypeNameNode{
  struct SYMS_TypeNameNode *next;
  SYMS_U8 *name_ptr;
  SYMS_TypeChain chain;
} SYMS_TypeNameNode;

typedef struct SYMS_TypeNameBuckets{
  SYMS_TypeNameNode **buckets;
} SYMS_TypeNameBuckets;


////////////////////////////////
//~ allen: Syms Type Graph

typedef struct SYMS_TypeMember{
  SYMS_MemKind kind;
  SYMS_MemVisibility visibility;
  SYMS_MemFlags flags;
  SYMS_String8 name;
  SYMS_U32 off;
  SYMS_U32 virtual_off;
  struct SYMS_TypeNode *type;
} SYMS_TypeMember;

typedef struct SYMS_TypeUniqueInfo{
  SYMS_USID usid;
  SYMS_SrcCoord src_coord;
} SYMS_TypeUniqueInfo;

typedef struct SYMS_TypeMemberArray{
  SYMS_TypeMember *mems;
  SYMS_U64 count;
} SYMS_TypeMemberArray;

typedef struct SYMS_TypeNode{
  // SYMS_TypeNode extends and completes the information from SYMS_TypeInfo.
  // See SYMS_TypeInfo for more interpretation info.
  
  SYMS_TypeKind kind;
  SYMS_String8 name;
  SYMS_U64 byte_size;
  
  // when non-null contains the type's usid and/or source location of the type's definition
  SYMS_TypeUniqueInfo *unique;
  
  // (in addition to interpretations of SYMS_TypeInfo 'direct_type')
  //  SYMS_TypeKind_Forward*   -> the concrete type referenced by the forward reference
  struct SYMS_TypeNode *direct_type;
  
  // 'this_type' meaning depends on kind:
  //  SYMS_TypeKind_MemberPtr  -> the container type of the member pointer
  //  SYMS_TypeKind_Proc       -> if non-nil this is the type of an implicit 'this' in a C++ method
  struct SYMS_TypeNode *this_type;
  
  union{
    // kind: SYMS_TypeKind_Modifier
    SYMS_TypeModifiers mods;
    
    // kind: SYMS_TypeKind_Array
    SYMS_U64 array_count;
    
    // kind: SYMS_TypeKind_Bitfield
    struct{
      SYMS_U32 off;
      SYMS_U32 count;
    } bits;
    
    // kind: SYMS_TypeKind_Proc
    struct{
      struct SYMS_TypeNode **params;
      SYMS_U64 param_count;
    } proc;
    
    // kind: SYMS_TypeKind_Struct, SYMS_TypeKind_Union, SYMS_TypeKind_Class, SYMS_TypeKind_Enum
    // opaque pointer for lazy eval attachments to the type node.
    void *lazy_ptr;
  };
} SYMS_TypeNode;

typedef struct SYMS_TypeGraph{
  SYMS_Arena *arena;
  SYMS_StringCons *string_cons;
  SYMS_U64 address_size;
  SYMS_TypeContentBuckets content_buckets;
  SYMS_TypeUSIDBuckets usid_buckets;
  SYMS_TypeNameBuckets name_buckets;
  
  SYMS_TypeNode *type_void;
  SYMS_TypeNode *type_bool;
  SYMS_TypeNode *type_u8;
  SYMS_TypeNode *type_u16;
  SYMS_TypeNode *type_u32;
  SYMS_TypeNode *type_u64;
  SYMS_TypeNode *type_u128;
  SYMS_TypeNode *type_s8;
  SYMS_TypeNode *type_s16;
  SYMS_TypeNode *type_s32;
  SYMS_TypeNode *type_s64;
  SYMS_TypeNode *type_s128;
  SYMS_TypeNode *type_f32;
  SYMS_TypeNode *type_f64;
} SYMS_TypeGraph;

////////////////////////////////
//~ allen: Syms Type Info Parse Param Bundle Type

typedef struct SYMS_TypeParseParams{
  SYMS_String8 data;
  SYMS_DbgAccel *dbg;
  SYMS_UnitSetAccel *unit_set;
  SYMS_UnitAccel *unit;
  SYMS_UnitID uid;
  
  // needed for PDBs to correctly resolve forward types
  SYMS_MapAndUnit *type_map;
} SYMS_TypeParseParams;

////////////////////////////////
//~ allen: Syms Type Graph Nils

SYMS_READ_ONLY SYMS_GLOBAL SYMS_TypeNode syms_type_node_nil = {
  SYMS_TypeKind_Null,     // kind
  {(SYMS_U8*)"(nil)", 5}, // name
  0,                      // byte_size
  0,                      // src_coord
  &syms_type_node_nil,    // direct_type
  &syms_type_node_nil,    // this_type
};

SYMS_READ_ONLY SYMS_GLOBAL SYMS_TypeMemberArray syms_type_member_array_nil = {0};
SYMS_READ_ONLY SYMS_GLOBAL SYMS_EnumMemberArray syms_type_enum_member_array_nil = {0};

////////////////////////////////
//~ allen: Type Graph Setup Functions

// The graph_arena is a "permanent arena". It is used for all
// memory allocations on the graph. It should not be popped or
// released until the graph is no longer in use. It should not
// be a scratch arena. Similarly the string cons structure should
// not be released until the graph is no longer in use.

SYMS_API void syms_type_graph_init(SYMS_TypeGraph *graph,
                                   SYMS_Arena *graph_arena, SYMS_StringCons *graph_string_cons,
                                   SYMS_U64 address_size_bytes);

////////////////////////////////
//~ allen: Type Mapping Functions

SYMS_API SYMS_String8   syms_type_string_cons(SYMS_TypeGraph *graph, SYMS_String8 name);

SYMS_API SYMS_TypeNode* syms_type_from_usid(SYMS_TypeGraph *graph, SYMS_USID usid);
SYMS_API SYMS_TypeChain syms_type_from_name(SYMS_TypeGraph *graph, SYMS_String8 name);

////////////////////////////////
//~ allen: Type Node Info Getters

SYMS_API SYMS_TypeMemberArray syms_type_members_from_type(SYMS_TypeGraph *graph, SYMS_TypeNode *node);
SYMS_API SYMS_EnumMemberArray syms_type_enum_members_from_type(SYMS_TypeGraph *graph,
                                                               SYMS_TypeNode *node);

SYMS_API SYMS_B32 syms_type_members_are_equipped(SYMS_TypeGraph *graph, SYMS_TypeNode *node);

////////////////////////////////
//~ allen: Type Node Basic Type Getters

SYMS_API SYMS_TypeNode* syms_type_void(SYMS_TypeGraph *graph);

SYMS_API SYMS_TypeNode* syms_type_bool(SYMS_TypeGraph *graph);

SYMS_API SYMS_TypeNode* syms_type_u8(SYMS_TypeGraph *graph);
SYMS_API SYMS_TypeNode* syms_type_u16(SYMS_TypeGraph *graph);
SYMS_API SYMS_TypeNode* syms_type_u32(SYMS_TypeGraph *graph);
SYMS_API SYMS_TypeNode* syms_type_u64(SYMS_TypeGraph *graph);
SYMS_API SYMS_TypeNode* syms_type_u128(SYMS_TypeGraph *graph);

SYMS_API SYMS_TypeNode* syms_type_s8(SYMS_TypeGraph *graph);
SYMS_API SYMS_TypeNode* syms_type_s16(SYMS_TypeGraph *graph);
SYMS_API SYMS_TypeNode* syms_type_s32(SYMS_TypeGraph *graph);
SYMS_API SYMS_TypeNode* syms_type_s64(SYMS_TypeGraph *graph);
SYMS_API SYMS_TypeNode* syms_type_s128(SYMS_TypeGraph *graph);

SYMS_API SYMS_TypeNode* syms_type_f32(SYMS_TypeGraph *graph);
SYMS_API SYMS_TypeNode* syms_type_f64(SYMS_TypeGraph *graph);

////////////////////////////////
//~ allen: Type Node Constructors

//- deduplicated types
SYMS_API SYMS_TypeNode* syms_type_cons_basic(SYMS_TypeGraph *graph,
                                             SYMS_TypeKind kind, SYMS_U64 size, SYMS_String8 name);
SYMS_API SYMS_TypeNode* syms_type_cons_mod(SYMS_TypeGraph *graph, SYMS_TypeNode *type, SYMS_TypeModifiers mods);
SYMS_API SYMS_TypeNode* syms_type_cons_ptr(SYMS_TypeGraph *graph, SYMS_TypeKind ptr_kind, SYMS_TypeNode *type);
SYMS_API SYMS_TypeNode* syms_type_cons_array(SYMS_TypeGraph *graph, SYMS_TypeNode *type, SYMS_U64 count);
SYMS_API SYMS_TypeNode* syms_type_cons_proc(SYMS_TypeGraph *graph,
                                            SYMS_TypeNode *ret_type, SYMS_TypeNode *this_type,
                                            SYMS_TypeNode **params, SYMS_U64 count);
SYMS_API SYMS_TypeNode* syms_type_cons_mem_ptr(SYMS_TypeGraph *graph,
                                               SYMS_TypeNode *container, SYMS_TypeNode *type);
SYMS_API SYMS_TypeNode* syms_type_cons_bitfield(SYMS_TypeGraph *graph, SYMS_TypeNode *underlying_type,
                                                SYMS_U32 bitoff, SYMS_U32 bitcount);

//- record types from member lists
SYMS_API SYMS_TypeNode* syms_type_cons_record_stub(SYMS_TypeGraph *graph);
SYMS_API void           syms_type_cons_mem_list_push(SYMS_Arena *arena, SYMS_TypeConsMemberList *list,
                                                     SYMS_String8 name, SYMS_TypeNode *type);
SYMS_API void           syms_type_cons_record_with_members(SYMS_TypeGraph *graph, SYMS_TypeNode *stub,
                                                           SYMS_TypeKind kind, SYMS_String8 name,
                                                           SYMS_TypeConsMemberList *list);

//- record types with defered member lists
SYMS_API SYMS_TypeNode* syms_type_cons_record_defer_members(SYMS_TypeGraph *graph,
                                                            SYMS_TypeKind kind, SYMS_String8 name,
                                                            SYMS_U64 byte_size,
                                                            SYMS_TypeUniqueInfo *unique_opt);

//- other UDTs
SYMS_API SYMS_TypeNode* syms_type_cons_enum_defer_members(SYMS_TypeGraph *graph,
                                                          SYMS_String8 name, SYMS_TypeNode *underlying_type,
                                                          SYMS_TypeUniqueInfo *unique_opt);
SYMS_API SYMS_TypeNode* syms_type_cons_typedef(SYMS_TypeGraph *graph,
                                               SYMS_String8 name, SYMS_TypeNode *type,
                                               SYMS_TypeUniqueInfo *unique_opt);
SYMS_API SYMS_TypeNode* syms_type_cons_fwd(SYMS_TypeGraph *graph,
                                           SYMS_TypeKind kind, SYMS_String8 name, SYMS_TypeNode *type,
                                           SYMS_TypeUniqueInfo *unique_opt);

//- usid place holders
SYMS_API SYMS_TypeUSIDPlaceHolder syms_type_usid_place_holder_insert(SYMS_TypeGraph *graph, SYMS_USID usid);
SYMS_API void                     syms_type_usid_place_holder_replace(SYMS_TypeGraph *graph,
                                                                      SYMS_TypeUSIDPlaceHolder *place,
                                                                      SYMS_TypeNode *node);

//- equipping record members
SYMS_API SYMS_TypeMember* syms_type_equip_mems_pre_allocate(SYMS_TypeGraph *graph, SYMS_TypeNode *node,
                                                            SYMS_U64 member_count);


//- equipping enum members
SYMS_API SYMS_EnumMember* syms_type_equip_enum_mems_pre_allocate(SYMS_TypeGraph *graph, SYMS_TypeNode *node,
                                                                 SYMS_U64 member_count);

//- unique info helpers
SYMS_API SYMS_TypeUniqueInfo* syms_type_unique_copy(SYMS_Arena *arena, SYMS_TypeUniqueInfo *unique_opt);
SYMS_API SYMS_TypeUniqueInfo  syms_type_unique_from_usid_src_coord(SYMS_USID usid, SYMS_SrcCoord *src_coord);


////////////////////////////////
//~ allen: Type Info Operators

SYMS_API SYMS_TypeNode* syms_type_resolved(SYMS_TypeNode *type);
SYMS_API SYMS_B32       syms_type_node_match(SYMS_TypeNode *l, SYMS_TypeNode *r);

SYMS_API SYMS_TypeNode* syms_type_resolve_enum_to_basic(SYMS_TypeGraph *graph, SYMS_TypeNode *t);

SYMS_API SYMS_TypeNode* syms_type_promoted_from_type_node(SYMS_TypeGraph *graph, SYMS_TypeNode *c);
SYMS_API SYMS_TypeNode* syms_type_auto_casted_from_type_nodes(SYMS_TypeGraph *graph,
                                                              SYMS_TypeNode *l, SYMS_TypeNode *r);


////////////////////////////////
//~ allen: Type Stringizing

SYMS_API SYMS_String8 syms_type_string_from_type(SYMS_Arena *arena, SYMS_TypeNode *type);

SYMS_API void         syms_type_lhs_string_from_type(SYMS_Arena *arena, SYMS_TypeNode *type,
                                                     SYMS_String8List *out);
SYMS_API void         syms_type_rhs_string_from_type(SYMS_Arena *arena, SYMS_TypeNode *type,
                                                     SYMS_String8List *out);

SYMS_API void         syms_type_lhs_string_from_type_skip_return(SYMS_Arena *arena,
                                                                 SYMS_TypeNode *type,
                                                                 SYMS_String8List *out);

SYMS_API void         syms_type_lhs_string_from_type__internal(SYMS_Arena *arena,
                                                               SYMS_TypeNode *type,
                                                               SYMS_String8List *out,
                                                               SYMS_U32 prec, SYMS_B32 skip);
SYMS_API void         syms_type_rhs_string_from_type__internal(SYMS_Arena *arena,
                                                               SYMS_TypeNode *type,
                                                               SYMS_String8List *out,
                                                               SYMS_U32 prec);

////////////////////////////////
//~ allen: Type Info Construct From Dbg Info

SYMS_API SYMS_TypeNode* syms_type_from_dbg_sid(SYMS_TypeGraph *graph, SYMS_TypeParseParams *params,
                                               SYMS_SymbolID sid);

SYMS_API SYMS_TypeNode* syms_type_from_dbg_sid__rec(SYMS_TypeGraph *graph, SYMS_TypeParseParams *params,
                                                    SYMS_SymbolID sid);

// NOTE(allen): The "node" must be one that was returned by "syms_type_from_dbg_sid".
// The "params" used here must match the "params" used from that call (same debug data, same unit).
// This function has no mechanism for checking this - so it's up to the user to arrange this correctly!
SYMS_API void           syms_type_equip_members_from_dbg(SYMS_TypeGraph *graph, SYMS_TypeParseParams *params,
                                                         SYMS_TypeNode *node);

////////////////////////////////
//~ allen: Type Content Table Functions

SYMS_API SYMS_U64       syms_type_content_hash(SYMS_String8 data);
SYMS_API SYMS_TypeNode* syms_type_from_content_buckets(SYMS_TypeContentBuckets *buckets, SYMS_String8 data);
SYMS_API SYMS_String8   syms_type_content_insert(SYMS_Arena *arena, SYMS_TypeContentBuckets *buckets,
                                                 SYMS_String8 key, SYMS_TypeNode *type);

////////////////////////////////
//~ allen: Type USID Table Functions

SYMS_API SYMS_U64       syms_type_usid_hash(SYMS_USID usid);
SYMS_API SYMS_TypeNode* syms_type_from_usid_buckets(SYMS_TypeUSIDBuckets *buckets, SYMS_USID usid);
SYMS_API SYMS_TypeUSIDNode* syms_type_usid_insert(SYMS_Arena *arena, SYMS_TypeUSIDBuckets *buckets,
                                                  SYMS_USID key, SYMS_TypeNode *type);

////////////////////////////////
//~ allen: Type Name Table Functions

SYMS_API SYMS_U64       syms_type_name_hash(SYMS_U8 *ptr);
SYMS_API SYMS_TypeChain*syms_type_chain_from_name_buckets(SYMS_TypeNameBuckets *buckets, SYMS_U8 *name_ptr);
SYMS_API void           syms_type_name_insert(SYMS_Arena *arena, SYMS_TypeNameBuckets *buckets,
                                              SYMS_U8 *name_ptr, SYMS_TypeNode *type);

////////////////////////////////
//~ allen: Notes for Maintainers

// Depending on the types, type information can require various combinations of:
// 1. Deduplicating identical types - which requires a consing map of (TypeContents -> Type)
// 2. Remembering a map of (Name -> ListOfTypes)
// 3. Remembering a map of (ID -> Type)
// 4. Preventing data cycles from creating infinite loops in the parser
// 5. Defering member construction to a later pass
// 6. Infering type size and layout from members
// 7. Attaching an ID to a type (allowing the mapping Type -> ID)
// 8. Attaching source location information

// 1 - For basic types, modified types, pointers (& references), arrays, procedures, and bitfields.
//     Users constructing these will want to know they can summon up something like a pointer to an
//     arbitrary type without creating lots of duplicated versions taking up memory.
//     "ContentBuckets" handles this.
//     The "cons" APIs for these types check for existing copies of the requested type before constructing them.
//     Because of deduplication unique info (7 and 8) are impossible for these types.

// 2 - For basic types, structs, unions, classes, enums, and typedefs.
//     The same name may appear multiple times, hence why this is a list of types.
//     "NameBuckets" handles this.
//     The "cons" APIs for these types automatically insert the type (if it is new) into the name map.

// 3 - For *any* kind of type *if* it is parsed from debug info (which is where IDs come from)
//     "USIDBuckets" handles this.
//     The "usid_place_holder" API handles this.
//     Since any type can have an ID, the API for handling IDs is orthogonal to the "cons" APIs.

// 4 - When constructing type info from serialized data, it is always possible that malformed input
//     may give an unconstructable structure. For instance if X is the ID of a pointer, then that pointer
//     will give it's base type by giving an ID for another type in the serialized data. If that ID is
//     X, the pointer has _itself_ as it's own base type. Since this isn't an allowed structure in type
//     info the only thing the parser needs to do is avoid an infinite loop. To achieve this we insert
//     a nil in the (ID -> Type) map *first* and then later replace it with the final version of the type
//     info. That way if the type info has a data cycle, it will resolve to broken type data and return
//     rather than getting caught in a loop. We cannot do this with a pre-allocated "stub" node because
//     sometimes the final node pointer will be one that _already existed_ and was discovered by
//     deduplication (1).
//     The "usid_place_holder" API handles this.

// 5 - Structs, unions and classes have members which are an exception to the from rule #4. These *can*
//     create data cycles (A struct Foo containing a Foo pointer as a member).
//     When constructing type info from debug info, the debug info gives the size of the struct directly
//     so we don't have to resolve members to find the size anyways. Therefore we can make everything work
//     by defering member construction to a later pass (when all the type IDs are resolved).
//     The "cons_record_defer_members" API handles constructing the type.
//     The "syms_type_equip" APIs handle equipping the members later.
//     Later - to figure out what members this type is supposed to have - the type node needs to remember
//     the ID of the type. Therefore this requires unique info (7).

// 6 - When user code wants to construct a struct, union, or class, it often knows the members but not
//     their exact layout or the size of the final type. This case calls for an API that takes in the
//     list of members and computes the layout and final size when constructing the type. However these
//     types can still want to form cycles (A struct Foo containing a Foo pointer as a member).
//     To support this we need an API that creates a stub type that can be finished later.
//     This is _very different_ from the USID place holders (4). That creates a mapping (ID -> Type)
//     for a type with a known ID, before the type node pointer itself can be known. In this case there
//     is _no ID at all_ and the user neds the type node pointer to proceed.
//     The "cons_artificial_stub" API creates the stub node.
//     The "cons_artificial_finish" API computes the layout & size, finishes the record type, and equips
//     members (no 5 in this case).

// 7 - Types with defered member construction (see 5) need to know their own ID so that their members can
//     be filled in later. For this the types have an optional "UniqueInfo" extension. This unique info
//     can carry the ID of the type. "cons" APIs for structs, unions, classes, and enums have to take an
//     optional "UniqueInfo" to support this case.

// 8 - Structs, unions, classes, enums, typedefs, and "forward" types can have source location information
//     when they come from debug info. To handle this the "cons" APIs for these types have to take an
//     optional "UniqueInfo" which can carry the source location.

#endif //SYMS_TYPE_GRAPH_H
