// Copyright Epic Games, Inc. All Rights Reserved.

#ifndef SYMS_MACH_C
#define SYMS_MACH_C

////////////////////////////////
//~ NOTE(allen): Generated

#include "syms/core/generated/syms_meta_mach.c"

////////////////////////////////
//~ NOTE(allen): MACH Format Functions

SYMS_API void
syms_mach_header64_from_header32(SYMS_MachHeader64 *dst, SYMS_MachHeader32 *header32){
  dst->magic = header32->magic;
  dst->cputype = header32->cputype;
  dst->cpusubtype = header32->cpusubtype;
  dst->filetype = header32->filetype;
  dst->ncmds = header32->ncmds;
  dst->sizeofcmds = header32->sizeofcmds;
  dst->flags = header32->flags;
  dst->reserved = 0;
}

SYMS_API void
syms_mach_nlist64_from_nlist32(SYMS_MachNList64 *dst, SYMS_MachNList32 *nlist32){
  dst->n_strx = nlist32->n_strx;
  dst->n_type = nlist32->n_type;
  dst->n_sect = nlist32->n_sect;
  dst->n_desc = nlist32->n_desc;
  dst->n_value = nlist32->n_value;
}

SYMS_API void
syms_mach_segment64_from_segment32(SYMS_MachSegmentCommand64 *dst, SYMS_MachSegmentCommand32 *seg32)
{
  dst->cmd = seg32->cmd;
  syms_memmove(&dst->segname[0], &seg32->segname[0], sizeof(seg32->segname));
  dst->vmaddr = seg32->vmaddr;
  dst->vmsize = seg32->vmsize;
  dst->fileoff = seg32->fileoff;
  dst->filesize = seg32->filesize;
  dst->maxprot = seg32->maxprot;
  dst->initprot = seg32->initprot;
  dst->nsects = seg32->nsects;
  dst->flags = seg32->flags;
}

SYMS_API void
syms_mach_section64_from_section32(SYMS_MachSection64 *dst, SYMS_MachSection32 *sect32)
{
  syms_memmove(&dst->sectname[0], &sect32->sectname[0], sizeof(sect32->sectname));
  syms_memmove(&dst->segname[0], &sect32->segname[0], sizeof(sect32->segname));
  dst->addr = sect32->addr;
  dst->size = sect32->size;
  dst->offset = sect32->offset;
  dst->align = sect32->align;
  dst->relocoff = sect32->relocoff;
  dst->nreloc = sect32->nreloc;
  dst->flags = sect32->flags;
  dst->reserved1 = sect32->reserved1;
  dst->reserved2 = sect32->reserved2;
}

#endif // SYMS_MACH_C
