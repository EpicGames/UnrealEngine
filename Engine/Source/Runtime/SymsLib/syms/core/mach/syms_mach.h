// Copyright Epic Games, Inc. All Rights Reserved.

#ifndef SYMS_MACH_H
#define SYMS_MACH_H

/*
** Reference materials:
** machine.h https://opensource.apple.com/source/xnu/xnu-4570.41.2/osfmk/mach/machine.h.auto.html
** loader.h  https://opensource.apple.com/source/xnu/xnu-4570.71.2/EXTERNAL_HEADERS/mach-o/loader.h.auto.html
** nlist.h   https://opensource.apple.com/source/xnu/xnu-201/EXTERNAL_HEADERS/mach-o/nlist.h.auto.html
** stab.h    https://opensource.apple.com/source/xnu/xnu-4903.241.1/EXTERNAL_HEADERS/mach-o/stab.h.auto.html
*/

////////////////////////////////
//~ NOTE(allen): Constants from machine.h

#define SYMS_MACH_CPU_ABI64     0x01000000
#define SYMS_MACH_CPU_ABI64_MAS 0xff000000

#define SYMS_MACH_CPU_SUBTYPE_MASK  0xff000000
#define SYMS_MACH_CPU_SUBTYPE_LIB64 0x80000000

////////////////////////////////
//~ NOTE(allen): Types & Constants from loader.h

typedef SYMS_S32 SYMS_MachVMProt;

//- magic numbers
#define SYMS_MACH_MAGIC_32 0xFEEDFACE
#define SYMS_MACH_CIGAM_32 0xCEFAEDFE

#define SYMS_MACH_MAGIC_64 0xFEEDFACF
#define SYMS_MACH_CIGAM_64 0xCFFAEDFE

#define SYMS_MACH_FAT_MAGIC 0xCAFEBABE
#define SYMS_MACH_FAT_CIGAM 0xBEBAFECA

#define SYMS_MACH_IS_HEADER_32(x)      ((x) == SYMS_MACH_MAGIC_32 || (x) == SYMS_MACH_CIGAM_32)
#define SYMS_MACH_IS_HEADER_64(x)      ((x) == SYMS_MACH_MAGIC_64 || (x) == SYMS_MACH_CIGAM_64)
#define SYMS_MACH_HEADER_IS_SWAPPED(x) ((x) == SYMS_MACH_CIGAM_64 || (x) == SYMS_MACH_CIGAM_32)
#define SYMS_MACH_IS_FAT(x)            ((x) == SYMS_MACH_FAT_MAGIC || (x) == SYMS_MACH_FAT_CIGAM)
#define SYMS_MACH_FAT_IS_SWAPPED(x)    ((x) == SYMS_MACH_FAT_CIGAM)

//- types

#define SYMS_MACH_VERSION_NIBBLE_A(x) (((x) & 0xFFFF0000) >> 16)
#define SYMS_MACH_VERSION_NIBBLE_B(x) (((x) & 0x0000FF00) >> 8)
#define SYMS_MACH_VERSION_NIBBLE_C(x) (((x) & 0x000000FF) >> 0)

////////////////////////////////
//~ NOTE(allen): Generated

#include "syms/core/generated/syms_meta_mach.h"

////////////////////////////////
//~ NOTE(allen): Types & Constants from nlist.h

#define SYMS_MACH_NLIST_STAB(x) ((x) & 0xE0)
#define SYMS_MACH_NLIST_PEXT(x) ((x) & 0x10)
#define SYMS_MACH_NLIST_TYPE(x) ((x) & 0x0E)
#define SYMS_MACH_NLIST_EXT(x)  ((x) & 0x01)

////////////////////////////////
//~ NOTE(allen): MACH Format Functions

SYMS_C_LINKAGE_BEGIN

SYMS_API void syms_mach_header64_from_header32(SYMS_MachHeader64 *dst, SYMS_MachHeader32 *header32);
SYMS_API void syms_mach_nlist64_from_nlist32(SYMS_MachNList64 *dst, SYMS_MachNList32 *nlist32);
SYMS_API void syms_mach_segment64_from_segment32(SYMS_MachSegmentCommand64 *dst, SYMS_MachSegmentCommand32 *seg32);
SYMS_API void syms_mach_section64_from_section32(SYMS_MachSection64 *dst, SYMS_MachSection32 *sect32);

SYMS_C_LINKAGE_END

#endif // SYMS_MACH_H
