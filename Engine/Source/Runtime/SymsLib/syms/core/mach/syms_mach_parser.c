// Copyright Epic Games, Inc. All Rights Reserved.

#ifndef SYMS_MACH_PARSER_C
#define SYMS_MACH_PARSER_C

////////////////////////////////
//~ NOTE(allen): MACH Parser Functions

SYMS_API SYMS_MachBinAccel*
syms_mach_bin_from_base_range(SYMS_Arena *arena, void *base, SYMS_U64Range range){
  SYMS_MachBinAccel *result = (SYMS_MachBinAccel*)&syms_format_nil;
  
  //- read properties from magic
  SYMS_U32 magic = 0;
  syms_based_range_read_struct(base, range, 0, &magic);
  
  SYMS_B32 is_mach = syms_false;
  SYMS_B32 is_swapped = syms_false;
  SYMS_B32 is_32 = syms_false;
  switch (magic){
    case SYMS_MACH_MAGIC_32:
    {
      is_mach = syms_true;
      is_32 = syms_true;
    }break;
    case SYMS_MACH_MAGIC_64:
    {
      is_mach = syms_true;
    }break;
    
    case SYMS_MACH_CIGAM_32:
    {
      is_mach = syms_true;
      is_swapped = syms_true;
      is_32 = syms_true;
    }break;
    case SYMS_MACH_CIGAM_64:
    {
      is_mach = syms_true;
      is_swapped = syms_true;
    }break;
  }
  
  if (is_mach){
    //- read header
    SYMS_U64 after_header_off = 0;
    SYMS_MachHeader64 header = {0};
    if (is_32){
      SYMS_MachHeader32 header32 = {0};
      after_header_off = syms_based_range_read_struct(base, range, 0, &header32);
      if (is_swapped){
        syms_bswap_in_place(SYMS_MachHeader32, &header32);
      }
      syms_mach_header64_from_header32(&header, &header32);
    }
    else{
      after_header_off = syms_based_range_read_struct(base, range, 0, &header);
      if (is_swapped){
        syms_bswap_in_place(SYMS_MachHeader64, &header);
      }
    }
    
    //- gather segment and section lists
    SYMS_ArenaTemp scratch = syms_get_scratch(&arena, 1);
    
    SYMS_MachSegmentNode *segment_first = 0;
    SYMS_MachSegmentNode *segment_last = 0;
    SYMS_U32 segment_count = 0;
    
    SYMS_MachSectionNode *section_first = 0;
    SYMS_MachSectionNode *section_last = 0;
    SYMS_U32 section_count = 0;
    
    SYMS_U64Range bind_ranges[SYMS_MachBindTable_COUNT]; syms_memzero_struct(&bind_ranges[0]);
    SYMS_U64Range export_range = syms_make_u64_range(0,0);

    SYMS_MachSymtabCommand symtab; syms_memzero_struct(&symtab);
    
    SYMS_MachDylibList dylib_list; syms_memzero_struct(&dylib_list);
    
    { // push null dylib
      SYMS_MachDylib dylib; syms_memzero_struct(&dylib);
      syms_mach_dylib_list_push(scratch.arena, &dylib_list, &dylib, syms_make_u64_range(0,0));
    }
    
    SYMS_U64 next_cmd_off = after_header_off;
    for (SYMS_U32 i = 0; i < header.ncmds; i += 1){
      //- align read offset
      SYMS_U64 cmd_off = 0;
      if (is_32){
        cmd_off = SYMS_AlignPow2(next_cmd_off, 4);
      }
      else{
        cmd_off = SYMS_AlignPow2(next_cmd_off, 8);
      }
      
      //- read command
      SYMS_MachLoadCommand lc = {0};
      syms_based_range_read_struct(base, range, cmd_off, &lc);
      switch (lc.type){
        case SYMS_MachLoadCommandType_SYMTAB: 
        {
          syms_based_range_read_struct(base, range, cmd_off, &symtab);
        }break;

        case SYMS_MachLoadCommandType_SEGMENT: 
        {
          SYMS_MachSegmentCommand32 segment_command32 = {0};
          syms_based_range_read_struct(base, range, cmd_off, &segment_command32);
          if (is_swapped){
            syms_bswap_in_place(SYMS_MachSegmentCommand32, &segment_command32);
          }
          SYMS_U64 after_seg_off = cmd_off + sizeof(SYMS_MachSegmentCommand32);

          // push segment node
          SYMS_MachSegmentNode *segment_node = syms_push_array_zero(scratch.arena, SYMS_MachSegmentNode, 1);
          syms_mach_segment64_from_segment32(&segment_node->data, &segment_command32);
          SYMS_QueuePush(segment_first, segment_last, segment_node);
          segment_count += 1;

          // loop over sections
          SYMS_U64 next_sec_off = after_seg_off;
          SYMS_U64 sec_count = segment_command32.nsects;
          for (SYMS_U32 k = 0; k < sec_count; k += 1){
            // read section 64
            SYMS_U64 sec_off = next_sec_off;
            SYMS_MachSection32 section32 = {0};
            syms_based_range_read_struct(base, range, sec_off, &section32);
            if (is_swapped){
              syms_bswap_in_place(SYMS_MachSection32, &section32);
            }
            next_sec_off = sec_off + sizeof(SYMS_MachSection32);

            // push section node
            SYMS_MachSectionNode *section_node = syms_push_array_zero(scratch.arena, SYMS_MachSectionNode, 1);
            syms_mach_section64_from_section32(&section_node->data, &section32);
            SYMS_QueuePush(section_first, section_last, section_node);
            section_count += 1;
          }
        }break;

        case SYMS_MachLoadCommandType_SEGMENT_64:
        {
          // read segment 64
          SYMS_MachSegmentCommand64 segment_command64 = {0};
          syms_based_range_read_struct(base, range, cmd_off, &segment_command64);
          if (is_swapped){
            syms_bswap_in_place(SYMS_MachSegmentCommand64, &segment_command64);
          }
          SYMS_U64 after_seg_off = cmd_off + sizeof(SYMS_MachSegmentCommand64);
          
          // push segment node
          SYMS_MachSegmentNode *segment_node = syms_push_array_zero(scratch.arena, SYMS_MachSegmentNode, 1);
          segment_node->data = segment_command64;
          SYMS_QueuePush(segment_first, segment_last, segment_node);
          segment_count += 1;
          
          // loop over sections
          SYMS_U64 next_sec_off = after_seg_off;
          SYMS_U64 sec_count = segment_command64.nsects;
          for (SYMS_U32 k = 0; k < sec_count; k += 1){
            // read section 64
            SYMS_U64 sec_off = next_sec_off;
            SYMS_MachSection64 section64 = {0};
            syms_based_range_read_struct(base, range, sec_off, &section64);
            if (is_swapped){
              syms_bswap_in_place(SYMS_MachSection64, &section64);
            }
            next_sec_off = sec_off + sizeof(SYMS_MachSection64);
            
            // push section node
            SYMS_MachSectionNode *section_node = syms_push_array_zero(scratch.arena, SYMS_MachSectionNode, 1);
            section_node->data = section64;
            SYMS_QueuePush(section_first, section_last, section_node);
            section_count += 1;
          }
        }break;
        
        case SYMS_MachLoadCommandType_DYLD_INFO_ONLY: 
        {
          SYMS_MachDyldInfoCommand dyld;
          syms_based_range_read_struct(base, range, cmd_off, &dyld);
          if (is_swapped){
            syms_bswap_in_place(SYMS_MachDyldInfoCommand, &dyld);
          }
          
          bind_ranges[SYMS_MachBindTable_REGULAR] = syms_make_u64_inrange(range, dyld.bind_off, dyld.bind_size);
          bind_ranges[SYMS_MachBindTable_LAZY]    = syms_make_u64_inrange(range, dyld.lazy_bind_off, dyld.lazy_bind_size);
          bind_ranges[SYMS_MachBindTable_WEAK]    = syms_make_u64_inrange(range, dyld.weak_bind_off, dyld.weak_bind_size);
          export_range = syms_make_u64_inrange(range, dyld.export_off, dyld.export_size);
        }break;
        
        case SYMS_MachLoadCommandType_LOAD_WEAK_DYLIB:
        case SYMS_MachLoadCommandType_LOAD_DYLIB: 
        case SYMS_MachLoadCommandType_LOAD_UPWARD_DYLIB:
        case SYMS_MachLoadCommandType_LAZY_LOAD_DYLIB:
        {
          SYMS_MachDylibCommand cmd; syms_memzero_struct(&cmd);
          syms_based_range_read_struct(base, range, cmd_off, &cmd);
          SYMS_MachDylib *dylib = &cmd.dylib;
          if (is_swapped){
            syms_bswap_in_place(SYMS_MachDylib, dylib);
          }
          SYMS_U64      name_offset = cmd_off + dylib->name.offset;
          SYMS_U64Range name_range  = syms_make_u64_inrange(range, name_offset, (cmd_off + lc.size) - name_offset);
          syms_mach_dylib_list_push(scratch.arena, &dylib_list, dylib, name_range);
        }break;
      }
      
      next_cmd_off = cmd_off + lc.size;
    }
    
    //- segment array from list
    SYMS_MachSegmentCommand64 *segments = syms_push_array(arena, SYMS_MachSegmentCommand64, segment_count);
    {
      SYMS_MachSegmentNode *segment_node = segment_first;
      SYMS_MachSegmentCommand64 *segment_ptr = segments;
      SYMS_MachSegmentCommand64 *segment_opl = segments + segment_count;
      for (; segment_ptr < segment_opl; segment_ptr += 1, segment_node = segment_node->next){
        *segment_ptr = segment_node->data;
      }
    }
    
    //- section array from list
    SYMS_MachSection64 *sections = syms_push_array(arena, SYMS_MachSection64, section_count);
    {
      SYMS_MachSectionNode *section_node = section_first;
      SYMS_MachSection64 *section_ptr = sections;
      SYMS_MachSection64 *section_opl = sections + section_count;
      for (; section_ptr < section_opl; section_ptr += 1, section_node = section_node->next){
        *section_ptr = section_node->data;
      }
    }
    
    //- fill result
    result                     = syms_push_array(arena, SYMS_MachBinAccel, 1);
    result->load_command_count = header.ncmds;
    result->load_commands      = syms_make_u64_inrange(range, after_header_off, header.sizeofcmds);
    result->format             = SYMS_FileFormat_MACH;
    result->arch               = syms_mach_arch_from_cputype(header.cputype);
    result->is_swapped         = is_swapped;
    result->symtab             = symtab;
    result->segment_count      = segment_count;
    result->segments           = segments;
    result->section_count      = section_count;
    result->sections           = sections;
    syms_memmove(&result->bind_ranges[0], &bind_ranges[0], sizeof(bind_ranges));
    result->export_range  = export_range;
    { // convert dylib list to array
      result->dylib_count = 0;
      result->dylibs = syms_push_array(arena, SYMS_MachParsedDylib, dylib_list.count);
      for (SYMS_MachDylibNode *n = dylib_list.first; n != 0; n = n->next){
        result->dylibs[result->dylib_count++] = n->data;
      }
    }
    
    syms_release_scratch(scratch);
  }
  
  return(result);
}

SYMS_API SYMS_MachFileAccel*
syms_mach_file_accel_from_data(SYMS_Arena *arena, SYMS_String8 data){
  void *base = data.str;
  SYMS_U64Range range = syms_make_u64_range(0, data.size);
  
  SYMS_U32 magic = 0;
  syms_based_range_read_struct(base, range, 0, &magic);
  
  SYMS_B32 is_mach = syms_false;
  SYMS_B32 is_fat = syms_false;
  switch (magic){
    case SYMS_MACH_MAGIC_32:
    case SYMS_MACH_CIGAM_32:
    case SYMS_MACH_MAGIC_64:
    case SYMS_MACH_CIGAM_64:
    {
      is_mach = syms_true;
    }break;
    
    case SYMS_MACH_FAT_MAGIC:
    case SYMS_MACH_FAT_CIGAM:
    {
      is_mach = syms_true;
      is_fat = syms_true;
    }break;
  }
  
  SYMS_B32 is_swapped = syms_false;
  if (is_mach){
    switch (magic){
      case SYMS_MACH_CIGAM_32:
      case SYMS_MACH_CIGAM_64:
      case SYMS_MACH_FAT_CIGAM:
      {
        is_swapped = syms_true;
      }break;
    }
  }
  
  SYMS_MachFileAccel *result = (SYMS_MachFileAccel *)&syms_format_nil;
  if (is_mach){
    result = syms_push_array(arena, SYMS_MachFileAccel, 1);
    result->format = SYMS_FileFormat_MACH;
    result->is_swapped = is_swapped;
    result->is_fat = is_fat;
  }
  return(result);
}

SYMS_API SYMS_B32
syms_mach_file_is_bin(SYMS_MachFileAccel *file){
  SYMS_B32 result = (!file->is_fat);
  return(result);
}

SYMS_API SYMS_MachBinAccel *
syms_mach_bin_accel_from_file(SYMS_Arena *arena, SYMS_String8 data, SYMS_MachFileAccel *file){
  SYMS_MachBinAccel *result = (SYMS_MachBinAccel*)&syms_format_nil;
  if (!file->is_fat){
    SYMS_U64Range range = syms_make_u64_range(0, data.size);
    result = syms_mach_bin_from_base_range(arena, data.str, range);
  }
  return(result);
}

SYMS_API SYMS_B32
syms_mach_file_is_bin_list(SYMS_MachFileAccel *file_accel){
  SYMS_B32 result = file_accel->is_fat;
  return(result);
}

SYMS_API SYMS_MachBinListAccel*
syms_mach_bin_list_accel_from_file(SYMS_Arena *arena, SYMS_String8 data, SYMS_MachFileAccel *file){
  SYMS_MachBinListAccel *result = (SYMS_MachBinListAccel*)&syms_format_nil;
  
  if (file->is_fat){
    void *base = data.str;
    SYMS_U64Range range = syms_make_u64_range(0, data.size);
    
    SYMS_B32 is_swapped = file->is_swapped;
    
    SYMS_U32 read_offset = 0;
    SYMS_MachFatHeader fat_header = {0};
    read_offset += syms_based_range_read_struct(base, range, read_offset, &fat_header);
    if (is_swapped){
      syms_bswap_in_place(SYMS_MachFatHeader, &fat_header);
    }
    
    SYMS_U64 fat_count = fat_header.nfat_arch;
    SYMS_MachFatArch *fats = syms_push_array_zero(arena, SYMS_MachFatArch, fat_count);
    
    SYMS_MachFatArch *fat_ptr = fats;
    for (SYMS_U32 i = 0; i < fat_count; i += 1, fat_ptr += 1){
      read_offset += syms_based_range_read_struct(base, range, read_offset, fat_ptr);
      if (is_swapped){
        syms_bswap_in_place(SYMS_MachFatArch, fat_ptr);
      }
    }
    
    result = syms_push_array(arena, SYMS_MachBinListAccel, 1);
    result->format = SYMS_FileFormat_MACH;
    result->count = fat_count;
    result->fats = fats;
  }
  
  return(result);
}


////////////////////////////////
// NOTE(allen): Arch

SYMS_API SYMS_Arch
syms_mach_arch_from_bin(SYMS_MachBinAccel *bin){
  SYMS_Arch result = bin->arch;
  return(result);
}


////////////////////////////////
// NOTE(allen): Bin List

SYMS_API SYMS_BinInfoArray
syms_mach_bin_info_array_from_bin_list(SYMS_Arena *arena, SYMS_MachBinListAccel *bin_list){
  // allocate bin info array
  SYMS_U64 count = bin_list->count;
  SYMS_BinInfo *bin_info = syms_push_array(arena, SYMS_BinInfo, count);
  
  // fill bin info array
  SYMS_MachFatArch *fat_ptr = bin_list->fats;
  SYMS_BinInfo *bin_info_ptr = bin_info;
  for (SYMS_U64 i = 0; i < count; i += 1, bin_info_ptr += 1, fat_ptr += 1){
    bin_info_ptr->arch = syms_mach_arch_from_cputype(fat_ptr->cputype);
    bin_info_ptr->range = syms_make_u64_range(fat_ptr->offset, fat_ptr->offset + fat_ptr->size);
  } 
  
  // package up and return
  SYMS_BinInfoArray result = {0};
  result.count = count;
  result.bin_info = bin_info;
  return(result);
}

SYMS_API SYMS_MachBinAccel*
syms_mach_bin_accel_from_bin_list_number(SYMS_Arena *arena, SYMS_String8 data,
                                         SYMS_MachBinListAccel *bin_list, SYMS_U64 n){
  SYMS_MachBinAccel *result = (SYMS_MachBinAccel*)&syms_format_nil;
  if (1 <= n && n <= bin_list->count){
    SYMS_MachFatArch *fat = &bin_list->fats[n - 1];
    SYMS_U64Range range = syms_make_u64_range(fat->offset, fat->offset + fat->size);
    result = syms_mach_bin_from_base_range(arena, data.str, range);
  }
  return(result);
}


////////////////////////////////
// NOTE(allen): Binary Secs

SYMS_API SYMS_SecInfoArray
syms_mach_sec_info_array_from_bin(SYMS_Arena *arena, SYMS_String8 data, SYMS_MachBinAccel *bin){
  SYMS_SecInfoArray array;
  array.count = bin->section_count;
  array.sec_info = syms_push_array_zero(arena, SYMS_SecInfo, array.count);
  
  SYMS_MachSection64 *mach_sec = bin->sections;
  SYMS_MachSection64 *mach_sec_opl = bin->sections + array.count;
  SYMS_SecInfo *sec_info = array.sec_info;
  for (; mach_sec < mach_sec_opl; sec_info += 1, mach_sec += 1){
    SYMS_U8 *name_ptr = mach_sec->sectname;
    SYMS_U8 *ptr = name_ptr;
    SYMS_U8 *opl = name_ptr + SYMS_ARRAY_SIZE(mach_sec->sectname);
    for (;ptr < opl && *ptr != 0; ptr += 1);
    
    SYMS_String8 name = syms_str8_range(name_ptr, ptr);
    
    sec_info->name = syms_push_string_copy(arena, name);
    // TODO(allen): figure out when these ranges actually apply and when the section isn't
    // there one side or the other.
    sec_info->vrange = syms_make_u64_range(mach_sec->addr,   mach_sec->addr   + mach_sec->size);
    sec_info->frange = syms_make_u64_range(mach_sec->offset, mach_sec->offset + mach_sec->size);
  }
  return array;
}
SYMS_API SYMS_U64
syms_mach_default_vbase_from_bin(SYMS_MachBinAccel *bin){
  // TODO(rjf): @nick verify
  SYMS_U64 min_vbase = 0;
  for(SYMS_U64 segment_idx = 0; segment_idx < bin->segment_count; segment_idx += 1)
  {
    SYMS_MachSegmentCommand64 *segment = bin->segments + segment_idx;
    if(min_vbase == 0 || segment->vmaddr < min_vbase)
    {
      min_vbase = segment->vmaddr;
    }
  }
  return min_vbase;
}

////////////////////////////////

SYMS_API void
syms_mach_dylib_list_push(SYMS_Arena *arena, SYMS_MachDylibList *list, SYMS_MachDylib *dylib, SYMS_U64Range name){
  SYMS_MachDylibNode *node = syms_push_array(arena, SYMS_MachDylibNode, 1);
  node->data.header = *dylib;
  node->data.name = name;
  node->next = 0;
  SYMS_QueuePush(list->first, list->last, node);
  list->count += 1;
}

////////////////////////////////

SYMS_API SYMS_MachBindList
syms_mach_binds_from_base_range(SYMS_Arena *arena, void *base, SYMS_U64Range range, SYMS_U32 address_size, SYMS_MachBindTable bind_type){
  SYMS_U64 read_offset = 0;
  SYMS_MachBindList list; syms_memzero_struct(&list);
  SYMS_MachBind state;    syms_memzero_struct(&state);
  for (;;){
    SYMS_U8 encoded_byte = 0;
    read_offset += syms_based_range_read_struct(base, range, read_offset, &encoded_byte);
    
    SYMS_U8 opcode = encoded_byte & SYMS_MachBindOpcode_MASK;
    SYMS_U8 imm    = encoded_byte & SYMS_MachBindOpcode_IMM_MASK;
    
    switch (opcode){
      case SYMS_MachBindOpcode_DONE: goto exit;
      
      case SYMS_MachBindOpcode_SET_DYLIB_ORDINAL_IMM:
      {
        state.dylib = imm;
      } break;
      case SYMS_MachBindOpcode_SET_DYLIB_ORDINAL_ULEB:
      {
        read_offset += syms_based_range_read_uleb128(base, range, read_offset, &state.dylib);
      } break;
      case SYMS_MachBindOpcode_SET_DYLIB_SPECIAL_IMM:
      {
        SYMS_ASSERT_PARANOID("TODO: SET_DYLIB_SPECIAL_IMM");
      } break;
      case SYMS_MachBindOpcode_SET_SYMBOL_TRAILING_FLAGS_IMM:
      {
        state.flags = imm;
        state.symbol_name = syms_based_range_read_string(base, range, read_offset);
        read_offset += (state.symbol_name.size + 1);
      } break;
      case SYMS_MachBindOpcode_SET_TYPE_IMM:
      {
        state.type = imm;
      } break;
      case SYMS_MachBindOpcode_SET_ADDEND_SLEB:
      {
        read_offset += syms_based_range_read_sleb128(base, range, read_offset, &state.addend);
      } break;
      case SYMS_MachBindOpcode_SET_SEGMENT_AND_OFFSET_ULEB:
      {
        state.segment = imm;
        read_offset += syms_based_range_read_uleb128(base, range, read_offset, &state.segment_offset);
      } break;
      case SYMS_MachBindOpcode_ADD_ADDR_ULEB:
      {
        SYMS_U64 addend = 0;
        read_offset += syms_based_range_read_uleb128(base, range, read_offset, &addend);
        state.segment_offset += addend;
      } break;
      case SYMS_MachBindOpcode_DO_BIND_ADD_ADDR_ULEB:
      {
        SYMS_U64 addend = 0;
        read_offset += syms_based_range_read_uleb128(base, range, read_offset, &addend);
        state.segment_offset += addend;
      } goto do_bind;
      case SYMS_MachBindOpcode_DO_BIND_ADD_ADDR_IMM_SCALED:
      {
        state.segment_offset += imm;
      } goto do_bind;
      case SYMS_MachBindOpcode_DO_BIND_ULEB_TIMES_SKIPPING_ULEB:
      {
        SYMS_U64 count = 0, skip = 0;
        read_offset += syms_based_range_read_uleb128(base, range, read_offset, &count);
        read_offset += syms_based_range_read_uleb128(base, range, read_offset, &skip);
        
        state.segment_offset += skip * address_size;
      } goto do_bind;
      case SYMS_MachBindOpcode_DO_BIND:
      {
      } goto do_bind;
      do_bind:
      {
        if (bind_type == SYMS_MachBindTable_LAZY){
          for (;read_offset < syms_u64_range_size(range);){
            SYMS_U8 value = 0;
            SYMS_U64 read_size = syms_based_range_read_struct(base, range, read_offset, &value);
            if (value != 0){
              break;
            }
            read_offset += read_size;
          }
        }
        SYMS_MachBindNode *node = syms_push_array(arena, SYMS_MachBindNode, 1);
        node->data = state;
        node->next = 0;
        SYMS_QueuePush(list.first, list.last, node);
        list.count += 1;
      } break;
    }
  }
  exit:;
  
  return list;
}

SYMS_API SYMS_ImportArray
syms_mach_imports_from_bin(SYMS_Arena *arena, SYMS_String8 data, SYMS_MachBinAccel *bin){
  SYMS_ArenaTemp scratch = syms_get_scratch(&arena, 1);
  
  // read and copy library names
  SYMS_String8 *dylib_names = syms_push_array(scratch.arena, SYMS_String8, bin->dylib_count);
  for (SYMS_U32 i = 0; i < bin->dylib_count; ++i){
    SYMS_String8 name = syms_based_range_read_string((void*)data.str, bin->dylibs[i].name, 0);
    dylib_names[i] = syms_push_string_copy(arena, name);
  }
  
  // build binds
  SYMS_U64 address_size = syms_address_size_from_arch(bin->arch);
  SYMS_U64 total_bind_count = 0;
  SYMS_MachBindList binds[SYMS_MachBindTable_COUNT];
  for (SYMS_U32 i = 0; i < SYMS_ARRAY_SIZE(binds); ++i){
    binds[i] = syms_mach_binds_from_base_range(scratch.arena, (void*)data.str, bin->bind_ranges[i], address_size, (SYMS_MachBindTable)i);
    total_bind_count += binds[i].count;
  }
  
  // convert mach binds to syms imports
  SYMS_ImportArray import_array; 
  import_array.count = 0;
  import_array.imports = syms_push_array(arena, SYMS_Import, total_bind_count);
  for (SYMS_U32 i = 0; i < SYMS_ARRAY_SIZE(binds); ++i){
    for (SYMS_MachBindNode *node = binds[i].first; node != 0; node = node->next){
      SYMS_MachBind *bind = &node->data;
      SYMS_Import *import = &import_array.imports[import_array.count++];
      import->name = syms_push_string_copy(arena, bind->symbol_name);
      if (bind->dylib < bin->dylib_count){
        import->library_name = dylib_names[bind->dylib];
      } else{
        import->library_name = syms_str8(0,0);
      }
      import->ordinal = 0;
    }
  }
  
  syms_release_scratch(scratch);
  return import_array;
}

static SYMS_MachExport *
syms_mach_parse_export_node(SYMS_Arena *arena, void *base, SYMS_U64Range range, SYMS_String8 name, SYMS_U64 read_offset){
  SYMS_MachExport *node = 0;
  SYMS_U64 export_size = 0;
  SYMS_U64 read_size = syms_based_range_read_uleb128(base, range, read_offset, &export_size);
  if (read_size > 0){
    read_offset += read_size;
    
    // push new node
    node = syms_push_array_zero(arena, SYMS_MachExport, 1);
    node->name = name;
    node->is_export_info = export_size != 0;
    
    // read export discriminator
    SYMS_U64Range export_range = syms_make_u64_inrange(range, read_offset, export_size);
    read_offset += export_size;
    
    // parse export info
    if (node->is_export_info){
      SYMS_U64 export_read_offset = 0;
      export_read_offset += syms_based_range_read_uleb128(base, export_range, export_read_offset, &node->flags);
      if (node->flags & SYMS_MachExportSymbolFlags_REEXPORT){
        export_read_offset += syms_based_range_read_uleb128(base, export_range, export_read_offset, &node->dylib_ordinal);
        node->import_name = syms_based_range_read_string(base, export_range, export_read_offset);
        export_read_offset += (node->import_name.size + 1);
      } else{
        export_read_offset += syms_based_range_read_uleb128(base, export_range, export_read_offset, &node->address);
        if (node->flags & SYMS_MachExportSymbolFlags_STUB_AND_RESOLVED){
          export_read_offset += syms_based_range_read_uleb128(base, export_range, export_read_offset, &node->resolver);
        }
      }
    }
    
    // read child info
    read_offset += syms_based_range_read_struct(base, range, read_offset, &node->child_count);
    if (node->child_count > 0){
      node->children = syms_push_array(arena, SYMS_MachExport *, node->child_count);
      for (SYMS_U8 child_idx = 0; child_idx < node->child_count; child_idx += 1){
        SYMS_String8 name = syms_based_range_read_string(base, range, read_offset);
        read_offset += (name.size + 1);
        SYMS_U64 child_offset = 0;
        read_offset += syms_based_range_read_uleb128(base, range, read_offset, &child_offset);
        node->children[child_idx] = syms_mach_parse_export_node(arena, base, range, name, child_offset);
      }
    }
  }
  
  return node;
}

SYMS_API SYMS_MachExport *
syms_build_mach_export_trie(SYMS_Arena *arena, void *base, SYMS_U64Range range){
  // TODO(nick): rewrite function to use arena for recursion.
  return syms_mach_parse_export_node(arena, base, range, syms_str8_lit(""), 0);
}

SYMS_API SYMS_ExportArray
syms_mach_exports_from_bin(SYMS_Arena *arena, SYMS_String8 data, SYMS_MachBinAccel *bin){
  SYMS_ArenaTemp scratch = syms_get_scratch(&arena, 1);
  
  // read and copy library names
  SYMS_String8 *dylib_names = syms_push_array(scratch.arena, SYMS_String8, bin->dylib_count);
  for (SYMS_U32 i = 0; i < bin->dylib_count; ++i){
    SYMS_String8 name = syms_based_range_read_string((void*)data.str, bin->dylibs[i].name, 0);
    dylib_names[i] = syms_push_string_copy(arena, name);
  }
  
  // build export trie
  SYMS_MachExport *root_export = syms_build_mach_export_trie(scratch.arena, (void*)data.str, bin->export_range);
  
  // traverse trie
  SYMS_ExportNode *first_export = 0;
  SYMS_ExportNode *last_export = 0;
  SYMS_U32 export_count = 0;
  if (root_export){
    SYMS_MachExportFrame *stack = 0;
    SYMS_MachExportFrame *free_frames = 0;
    
    SYMS_MachExportFrame *frame = syms_push_array_zero(arena, SYMS_MachExportFrame, 1);
    frame->node = root_export;
    SYMS_StackPush(stack, frame);
    
    for (; stack != 0; ){
      SYMS_B32 pop_frame = frame->child_idx >= frame->node->child_count;
      if (pop_frame){
        if (frame->node->is_export_info){
          // append strings from nodes on the stack
          SYMS_ArenaTemp temp = syms_arena_temp_begin(scratch.arena);
          SYMS_String8List name_parts; syms_memzero_struct(&name_parts);
          for (SYMS_MachExportFrame *f = frame; f != 0; f = f->next){
            syms_string_list_push_front(temp.arena, &name_parts, f->node->name);
          }
          SYMS_StringJoin join; syms_memzero_struct(&join);
          SYMS_String8 name = syms_string_list_join(arena, &name_parts, &join);
          syms_arena_temp_end(temp);
          
          SYMS_ExportNode *export_node = syms_push_array(scratch.arena, SYMS_ExportNode, 1); 
          
          // add export to the list
          SYMS_QueuePush(first_export, last_export, export_node);
          export_count += 1;
          
          // initialize export
          SYMS_Export *exp = &export_node->data;
          if (frame->node->flags & SYMS_MachExportSymbolFlags_REEXPORT){
            // TODO(nick): need dylib with reexports to test this code path
            if (frame->node->dylib_ordinal > 0 && frame->node->dylib_ordinal < bin->dylib_count){
              exp->name = name;
              exp->address = 0;
              exp->ordinal = 0;
              exp->forwarder_library_name = dylib_names[frame->node->dylib_ordinal-1];
              exp->forwarder_import_name = syms_push_string_copy(arena, frame->node->import_name);
            } else{
              SYMS_ASSERT_PARANOID(!"invalid dylib ordinal");
            }
          } else{
            exp->name = name;
            exp->address = frame->node->address;
            exp->ordinal = 0;
            exp->forwarder_library_name = syms_str8(0,0);
            exp->forwarder_import_name = syms_str8(0,0);
          }
        }
        
        // pop current frame 
        SYMS_MachExportFrame *to_free = frame;
        frame = SYMS_StackPop(stack);
        
        // push frame to free list for later reuse
        to_free->next = 0;
        SYMS_StackPush(free_frames, to_free);
      } else{
        // push frame
        SYMS_MachExportFrame *new_frame = free_frames;
        if (new_frame){
          SYMS_StackPop(free_frames);
        } else{
          new_frame = syms_push_array_zero(arena, SYMS_MachExportFrame, 1);
        }
        new_frame->next = 0;
        new_frame->child_idx = 0;
        new_frame->node = frame->node->children[frame->child_idx++];
        frame = SYMS_StackPush(stack, new_frame);
      }
    }
  }
  
  SYMS_ExportArray export_array;
  export_array.count = 0;
  export_array.exports = syms_push_array(arena, SYMS_Export, export_count);
  for (SYMS_ExportNode *n = first_export; n != 0; n = n->next){
    export_array.exports[export_array.count++] = n->data;
  }
  
  syms_release_scratch(scratch);
  return export_array;
}

#endif // SYMS_MACH_PARSER_C
