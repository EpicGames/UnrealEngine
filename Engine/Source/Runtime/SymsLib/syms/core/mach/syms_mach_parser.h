// Copyright Epic Games, Inc. All Rights Reserved.

#ifndef SYMS_MACH_PARSER_H
#define SYMS_MACH_PARSER_H

////////////////////////////////
//~ NOTE(allen): MACH Parser Types

typedef struct SYMS_MachFileAccel{
  SYMS_FileFormat format;
  SYMS_B32 is_swapped;
  SYMS_B32 is_fat;
} SYMS_MachFileAccel;

typedef struct SYMS_MachBinListAccel{
  SYMS_FileFormat format;
  SYMS_MachFatArch *fats;
  SYMS_U32 count;
} SYMS_MachBinListAccel;

typedef struct SYMS_MachSegmentNode{
  struct SYMS_MachSegmentNode *next;
  SYMS_MachSegmentCommand64 data;
} SYMS_MachSegmentNode;

typedef struct SYMS_MachSectionNode{
  struct SYMS_MachSectionNode *next;
  SYMS_MachSection64 data;
} SYMS_MachSectionNode;

enum
{
  SYMS_MachBindTable_REGULAR,
  SYMS_MachBindTable_LAZY,
  SYMS_MachBindTable_WEAK,
  SYMS_MachBindTable_COUNT
};
typedef SYMS_U32 SYMS_MachBindTable;

typedef struct SYMS_MachBinAccel{
  SYMS_FileFormat format;
  
  SYMS_Arch arch;

  SYMS_B32 is_swapped;

  SYMS_U32 load_command_count;
  SYMS_U64Range load_commands;

  SYMS_MachSymtabCommand symtab;
  
  SYMS_MachSegmentCommand64 *segments;
  SYMS_U32 segment_count;
  
  SYMS_MachSection64 *sections;
  SYMS_U32 section_count;
  
  SYMS_U64Range bind_ranges[SYMS_MachBindTable_COUNT];
  SYMS_U64Range export_range;
  
  SYMS_U32 dylib_count;
  struct SYMS_MachParsedDylib *dylibs;
} SYMS_MachBinAccel;

////////////////////////////////
// Dylib

typedef struct SYMS_MachParsedDylib
{
  SYMS_MachDylib header;
  SYMS_U64Range name;
} SYMS_MachParsedDylib;


typedef struct SYMS_MachDylibNode
{
  struct SYMS_MachDylibNode *next;
  SYMS_MachParsedDylib data;
} SYMS_MachDylibNode;

typedef struct
{
  SYMS_MachDylibNode *first;
  SYMS_MachDylibNode *last;
  SYMS_U32 count;
} SYMS_MachDylibList;

////////////////////////////////
// Binds

typedef struct
{
  SYMS_U32 segment;
  SYMS_U64 segment_offset;
  SYMS_U64 dylib;
  SYMS_String8 symbol_name;
  SYMS_U8 flags;
  SYMS_MachBindType type;
  SYMS_S64 addend;
} SYMS_MachBind;

typedef struct SYMS_MachBindNode
{
  SYMS_MachBind data;
  struct SYMS_MachBindNode *next;
} SYMS_MachBindNode;

typedef struct
{
  SYMS_MachBindNode *first;
  SYMS_MachBindNode *last;
  SYMS_U32 count;
} SYMS_MachBindList;

////////////////////////////////
// Exports

typedef struct SYMS_MachExport
{
  struct SYMS_MachExport **children;
  SYMS_U8 child_count;
  
  SYMS_B32 is_export_info;
  
  SYMS_String8 name;
  SYMS_U64 flags;
  SYMS_U64 address;
  
  // SYMS_MachExportSymbolFlags_REEXPORT
  SYMS_U64 dylib_ordinal;
  SYMS_String8 import_name; 
  
  // SYMS_MachExportSymbolFlags_STUB_AND_RESOLVER
  SYMS_U64 resolver;
} SYMS_MachExport;

typedef struct SYMS_MachExportNode
{
  struct SYMS_MachExportNode *next;
  SYMS_MachExport data;
} SYMS_MachExportNode;

typedef struct SYMS_MachExportFrame
{
  struct SYMS_MachExportFrame *next;
  SYMS_U8 child_idx;
  SYMS_MachExport *node;
} SYMS_MachExportFrame;

////////////////////////////////
//~ NOTE(allen): MACH Parser Functions

// accelerators
//  mach specific
SYMS_API SYMS_MachBinAccel*     syms_mach_bin_from_base_range(SYMS_Arena *arena, void *base, SYMS_U64Range range);

// main api
SYMS_API SYMS_MachFileAccel*    syms_mach_file_accel_from_data(SYMS_Arena *arena, SYMS_String8 data);
SYMS_API SYMS_B32               syms_mach_file_is_bin(SYMS_MachFileAccel *file);
SYMS_API SYMS_MachBinAccel*     syms_mach_bin_accel_from_file(SYMS_Arena *arena, SYMS_String8 data,
                                                              SYMS_MachFileAccel *file);
SYMS_API SYMS_B32               syms_mach_file_is_bin_list(SYMS_MachFileAccel *file_accel);
SYMS_API SYMS_MachBinListAccel* syms_mach_bin_list_accel_from_file(SYMS_Arena *arena, SYMS_String8 data,
                                                                   SYMS_MachFileAccel *file);

// arch
SYMS_API SYMS_Arch              syms_mach_arch_from_bin(SYMS_MachBinAccel *bin);

// bin list
SYMS_API SYMS_BinInfoArray      syms_mach_bin_info_array_from_bin_list(SYMS_Arena *arena,
                                                                       SYMS_MachBinListAccel *bin_list);
SYMS_API SYMS_MachBinAccel*     syms_mach_bin_accel_from_bin_list_number(SYMS_Arena *arena, SYMS_String8 data,
                                                                         SYMS_MachBinListAccel *bin_list,
                                                                         SYMS_U64 n);

// binary secs
SYMS_API SYMS_SecInfoArray      syms_mach_sec_info_array_from_bin(SYMS_Arena *arena, SYMS_String8 data,
                                                                  SYMS_MachBinAccel *bin);

SYMS_API SYMS_U64               syms_mach_default_vbase_from_bin(SYMS_MachBinAccel *bin);

// dylib
SYMS_API void syms_mach_dylib_list_push(SYMS_Arena *arena, SYMS_MachDylibList *list, SYMS_MachDylib *dylib, SYMS_U64Range name);

// binds
SYMS_API SYMS_MachBindList syms_mach_binds_from_base_range(SYMS_Arena *arena, void *base, SYMS_U64Range range, SYMS_U32 address_size, SYMS_MachBindTable bind_type);
SYMS_API SYMS_ImportArray  syms_mach_imports_from_bin(SYMS_Arena *arena, SYMS_String8 data, SYMS_MachBinAccel *bin);

// exports
SYMS_API SYMS_MachExport * syms_build_mach_export_trie(SYMS_Arena *arena, void *base, SYMS_U64Range range);
SYMS_API SYMS_ExportArray  syms_mach_exports_from_bin(SYMS_Arena *arena, SYMS_String8 data, SYMS_MachBinAccel *bin);

#endif // SYMS_MACH_PARSER_H
