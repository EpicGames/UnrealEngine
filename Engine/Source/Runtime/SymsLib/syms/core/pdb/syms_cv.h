// Copyright Epic Games, Inc. All Rights Reserved.
/* date = April 5th 2021 3:55 pm */

#ifndef SYMS_CV_H
#define SYMS_CV_H

////////////////////////////////
//~ allen: Index Types

typedef SYMS_U32 SYMS_CvTypeIndex;
// "TPI"
typedef SYMS_U32 SYMS_CvTypeId;
// "IPI" leaf kinds: FUNC_ID,MFUNC_ID,BUILDINFO,SUBSTR_LIST,STRING_ID
typedef SYMS_U32 SYMS_CvItemId;

SYMS_GLOBAL SYMS_CvTypeId syms_cv_type_id_variadic = SYMS_U32_MAX;

typedef SYMS_U16 SYMS_CvModIndex;
typedef SYMS_U16 SYMS_CvSectionIndex;

////////////////////////////////
//~ allen: Registers

typedef SYMS_U16 SYMS_CvReg;

////////////////////////////////
//~ allen: Generated Enums

#include "syms/core/generated/syms_meta_cv.h"

////////////////////////////////
//~ allen: Basic Types

#define SYMS_CvBasicTypeFromTypeId(x)        (((x)   )&0xff)
#define SYMS_CvBasicPointerKindFromTypeId(x) (((x)>>8)&0xff)

////////////////////////////////
//~ rjf: Checksum

// This format likes to use packed types.
#pragma pack(push, 1)

typedef struct SYMS_CvChecksum{
  SYMS_U32 name_off;
  SYMS_U8 len;
  SYMS_CvChecksumKind kind;
} SYMS_CvChecksum;

////////////////////////////////
//~ allen: Registers

enum{
  SYMS_CvReg_NONE     =   0,
  
  ////////////////////////////////
  // NOTE(allen): 80x86 and ix86 registers
  
  SYMS_CvReg_X86_AL       =   1,
  SYMS_CvReg_X86_CL       =   2,
  SYMS_CvReg_X86_DL       =   3,
  SYMS_CvReg_X86_BL       =   4,
  SYMS_CvReg_X86_AH       =   5,
  SYMS_CvReg_X86_CH       =   6,
  SYMS_CvReg_X86_DH       =   7,
  SYMS_CvReg_X86_BH       =   8,
  SYMS_CvReg_X86_AX       =   9,
  SYMS_CvReg_X86_CX       =  10,
  SYMS_CvReg_X86_DX       =  11,
  SYMS_CvReg_X86_BX       =  12,
  SYMS_CvReg_X86_SP       =  13,
  SYMS_CvReg_X86_BP       =  14,
  SYMS_CvReg_X86_SI       =  15,
  SYMS_CvReg_X86_DI       =  16,
  SYMS_CvReg_X86_EAX      =  17,
  SYMS_CvReg_X86_ECX      =  18,
  SYMS_CvReg_X86_EDX      =  19,
  SYMS_CvReg_X86_EBX      =  20,
  SYMS_CvReg_X86_ESP      =  21,
  SYMS_CvReg_X86_EBP      =  22,
  SYMS_CvReg_X86_ESI      =  23,
  SYMS_CvReg_X86_EDI      =  24,
  SYMS_CvReg_X86_ES       =  25,
  SYMS_CvReg_X86_CS       =  26,
  SYMS_CvReg_X86_SS       =  27,
  SYMS_CvReg_X86_DS       =  28,
  SYMS_CvReg_X86_FS       =  29,
  SYMS_CvReg_X86_GS       =  30,
  SYMS_CvReg_X86_IP       =  31,
  SYMS_CvReg_X86_FLAGS    =  32,
  SYMS_CvReg_X86_EIP      =  33,
  SYMS_CvReg_X86_EFLAGS   =  34,
  
  SYMS_CvReg_X86_MM0      =  146,
  SYMS_CvReg_X86_MM1      =  147,
  SYMS_CvReg_X86_MM2      =  148,
  SYMS_CvReg_X86_MM3      =  149,
  SYMS_CvReg_X86_MM4      =  150,
  SYMS_CvReg_X86_MM5      =  151,
  SYMS_CvReg_X86_MM6      =  152,
  SYMS_CvReg_X86_MM7      =  153,
  SYMS_CvReg_X86_XMM0     =  154,
  SYMS_CvReg_X86_XMM1     =  155,
  SYMS_CvReg_X86_XMM2     =  156,
  SYMS_CvReg_X86_XMM3     =  157,
  SYMS_CvReg_X86_XMM4     =  158,
  SYMS_CvReg_X86_XMM5     =  159,
  SYMS_CvReg_X86_XMM6     =  160,
  SYMS_CvReg_X86_XMM7     =  161,
  
  SYMS_CvReg_X86_XMM00    =  162,
  SYMS_CvReg_X86_XMM01    =  163,
  SYMS_CvReg_X86_XMM02    =  164,
  SYMS_CvReg_X86_XMM03    =  165,
  SYMS_CvReg_X86_XMM10    =  166,
  SYMS_CvReg_X86_XMM11    =  167,
  SYMS_CvReg_X86_XMM12    =  168,
  SYMS_CvReg_X86_XMM13    =  169,
  SYMS_CvReg_X86_XMM20    =  170,
  SYMS_CvReg_X86_XMM21    =  171,
  SYMS_CvReg_X86_XMM22    =  172,
  SYMS_CvReg_X86_XMM23    =  173,
  SYMS_CvReg_X86_XMM30    =  174,
  SYMS_CvReg_X86_XMM31    =  175,
  SYMS_CvReg_X86_XMM32    =  176,
  SYMS_CvReg_X86_XMM33    =  177,
  SYMS_CvReg_X86_XMM40    =  178,
  SYMS_CvReg_X86_XMM41    =  179,
  SYMS_CvReg_X86_XMM42    =  180,
  SYMS_CvReg_X86_XMM43    =  181,
  SYMS_CvReg_X86_XMM50    =  182,
  SYMS_CvReg_X86_XMM51    =  183,
  SYMS_CvReg_X86_XMM52    =  184,
  SYMS_CvReg_X86_XMM53    =  185,
  SYMS_CvReg_X86_XMM60    =  186,
  SYMS_CvReg_X86_XMM61    =  187,
  SYMS_CvReg_X86_XMM62    =  188,
  SYMS_CvReg_X86_XMM63    =  189,
  SYMS_CvReg_X86_XMM70    =  190,
  SYMS_CvReg_X86_XMM71    =  191,
  SYMS_CvReg_X86_XMM72    =  192,
  SYMS_CvReg_X86_XMM73    =  193,
  
  SYMS_CvReg_X86_XMM0L    =  194,
  SYMS_CvReg_X86_XMM1L    =  195,
  SYMS_CvReg_X86_XMM2L    =  196,
  SYMS_CvReg_X86_XMM3L    =  197,
  SYMS_CvReg_X86_XMM4L    =  198,
  SYMS_CvReg_X86_XMM5L    =  199,
  SYMS_CvReg_X86_XMM6L    =  200,
  SYMS_CvReg_X86_XMM7L    =  201,
  SYMS_CvReg_X86_XMM0H    =  202,
  SYMS_CvReg_X86_XMM1H    =  203,
  SYMS_CvReg_X86_XMM2H    =  204,
  SYMS_CvReg_X86_XMM3H    =  205,
  SYMS_CvReg_X86_XMM4H    =  206,
  SYMS_CvReg_X86_XMM5H    =  207,
  SYMS_CvReg_X86_XMM6H    =  208,
  SYMS_CvReg_X86_XMM7H    =  209,
  
  // AVX registers
  
  SYMS_CvReg_X86_YMM0  = 252,
  SYMS_CvReg_X86_YMM1  = 253,
  SYMS_CvReg_X86_YMM2  = 254,
  SYMS_CvReg_X86_YMM3  = 255,
  SYMS_CvReg_X86_YMM4  = 256,
  SYMS_CvReg_X86_YMM5  = 257,
  SYMS_CvReg_X86_YMM6  = 258,
  SYMS_CvReg_X86_YMM7  = 259,
  SYMS_CvReg_X86_YMM0H = 260,
  SYMS_CvReg_X86_YMM1H = 261,
  SYMS_CvReg_X86_YMM2H = 262,
  SYMS_CvReg_X86_YMM3H = 263,
  SYMS_CvReg_X86_YMM4H = 264,
  SYMS_CvReg_X86_YMM5H = 265,
  SYMS_CvReg_X86_YMM6H = 266,
  SYMS_CvReg_X86_YMM7H = 267,
  
  // AVX integer registers
  
  SYMS_CvReg_X86_YMM0I0 = 268,
  SYMS_CvReg_X86_YMM0I1 = 269,
  SYMS_CvReg_X86_YMM0I2 = 270,
  SYMS_CvReg_X86_YMM0I3 = 271,
  SYMS_CvReg_X86_YMM1I0 = 272,
  SYMS_CvReg_X86_YMM1I1 = 273,
  SYMS_CvReg_X86_YMM1I2 = 274,
  SYMS_CvReg_X86_YMM1I3 = 275,
  SYMS_CvReg_X86_YMM2I0 = 276,
  SYMS_CvReg_X86_YMM2I1 = 277,
  SYMS_CvReg_X86_YMM2I2 = 278,
  SYMS_CvReg_X86_YMM2I3 = 279,
  SYMS_CvReg_X86_YMM3I0 = 280,
  SYMS_CvReg_X86_YMM3I1 = 281,
  SYMS_CvReg_X86_YMM3I2 = 282,
  SYMS_CvReg_X86_YMM3I3 = 283,
  SYMS_CvReg_X86_YMM4I0 = 284,
  SYMS_CvReg_X86_YMM4I1 = 285,
  SYMS_CvReg_X86_YMM4I2 = 286,
  SYMS_CvReg_X86_YMM4I3 = 287,
  SYMS_CvReg_X86_YMM5I0 = 288,
  SYMS_CvReg_X86_YMM5I1 = 289,
  SYMS_CvReg_X86_YMM5I2 = 290,
  SYMS_CvReg_X86_YMM5I3 = 291,
  SYMS_CvReg_X86_YMM6I0 = 292,
  SYMS_CvReg_X86_YMM6I1 = 293,
  SYMS_CvReg_X86_YMM6I2 = 294,
  SYMS_CvReg_X86_YMM6I3 = 295,
  SYMS_CvReg_X86_YMM7I0 = 296,
  SYMS_CvReg_X86_YMM7I1 = 297,
  SYMS_CvReg_X86_YMM7I2 = 298,
  SYMS_CvReg_X86_YMM7I3 = 299,
  
  // AVX floating-point single precise registers
  
  SYMS_CvReg_X86_YMM0F0 = 300,
  SYMS_CvReg_X86_YMM0F1 = 301,
  SYMS_CvReg_X86_YMM0F2 = 302,
  SYMS_CvReg_X86_YMM0F3 = 303,
  SYMS_CvReg_X86_YMM0F4 = 304,
  SYMS_CvReg_X86_YMM0F5 = 305,
  SYMS_CvReg_X86_YMM0F6 = 306,
  SYMS_CvReg_X86_YMM0F7 = 307,
  SYMS_CvReg_X86_YMM1F0 = 308,
  SYMS_CvReg_X86_YMM1F1 = 309,
  SYMS_CvReg_X86_YMM1F2 = 310,
  SYMS_CvReg_X86_YMM1F3 = 311,
  SYMS_CvReg_X86_YMM1F4 = 312,
  SYMS_CvReg_X86_YMM1F5 = 313,
  SYMS_CvReg_X86_YMM1F6 = 314,
  SYMS_CvReg_X86_YMM1F7 = 315,
  SYMS_CvReg_X86_YMM2F0 = 316,
  SYMS_CvReg_X86_YMM2F1 = 317,
  SYMS_CvReg_X86_YMM2F2 = 318,
  SYMS_CvReg_X86_YMM2F3 = 319,
  SYMS_CvReg_X86_YMM2F4 = 320,
  SYMS_CvReg_X86_YMM2F5 = 321,
  SYMS_CvReg_X86_YMM2F6 = 322,
  SYMS_CvReg_X86_YMM2F7 = 323,
  SYMS_CvReg_X86_YMM3F0 = 324,
  SYMS_CvReg_X86_YMM3F1 = 325,
  SYMS_CvReg_X86_YMM3F2 = 326,
  SYMS_CvReg_X86_YMM3F3 = 327,
  SYMS_CvReg_X86_YMM3F4 = 328,
  SYMS_CvReg_X86_YMM3F5 = 329,
  SYMS_CvReg_X86_YMM3F6 = 330,
  SYMS_CvReg_X86_YMM3F7 = 331,
  SYMS_CvReg_X86_YMM4F0 = 332,
  SYMS_CvReg_X86_YMM4F1 = 333,
  SYMS_CvReg_X86_YMM4F2 = 334,
  SYMS_CvReg_X86_YMM4F3 = 335,
  SYMS_CvReg_X86_YMM4F4 = 336,
  SYMS_CvReg_X86_YMM4F5 = 337,
  SYMS_CvReg_X86_YMM4F6 = 338,
  SYMS_CvReg_X86_YMM4F7 = 339,
  SYMS_CvReg_X86_YMM5F0 = 340,
  SYMS_CvReg_X86_YMM5F1 = 341,
  SYMS_CvReg_X86_YMM5F2 = 342,
  SYMS_CvReg_X86_YMM5F3 = 343,
  SYMS_CvReg_X86_YMM5F4 = 344,
  SYMS_CvReg_X86_YMM5F5 = 345,
  SYMS_CvReg_X86_YMM5F6 = 346,
  SYMS_CvReg_X86_YMM5F7 = 347,
  SYMS_CvReg_X86_YMM6F0 = 348,
  SYMS_CvReg_X86_YMM6F1 = 349,
  SYMS_CvReg_X86_YMM6F2 = 350,
  SYMS_CvReg_X86_YMM6F3 = 351,
  SYMS_CvReg_X86_YMM6F4 = 352,
  SYMS_CvReg_X86_YMM6F5 = 353,
  SYMS_CvReg_X86_YMM6F6 = 354,
  SYMS_CvReg_X86_YMM6F7 = 355,
  SYMS_CvReg_X86_YMM7F0 = 356,
  SYMS_CvReg_X86_YMM7F1 = 357,
  SYMS_CvReg_X86_YMM7F2 = 358,
  SYMS_CvReg_X86_YMM7F3 = 359,
  SYMS_CvReg_X86_YMM7F4 = 360,
  SYMS_CvReg_X86_YMM7F5 = 361,
  SYMS_CvReg_X86_YMM7F6 = 362,
  SYMS_CvReg_X86_YMM7F7 = 363,
  
  // AVX floating-point double precise registers
  
  SYMS_CvReg_X86_YMM0D0 = 364,
  SYMS_CvReg_X86_YMM0D1 = 365,
  SYMS_CvReg_X86_YMM0D2 = 366,
  SYMS_CvReg_X86_YMM0D3 = 367,
  SYMS_CvReg_X86_YMM1D0 = 368,
  SYMS_CvReg_X86_YMM1D1 = 369,
  SYMS_CvReg_X86_YMM1D2 = 370,
  SYMS_CvReg_X86_YMM1D3 = 371,
  SYMS_CvReg_X86_YMM2D0 = 372,
  SYMS_CvReg_X86_YMM2D1 = 373,
  SYMS_CvReg_X86_YMM2D2 = 374,
  SYMS_CvReg_X86_YMM2D3 = 375,
  SYMS_CvReg_X86_YMM3D0 = 376,
  SYMS_CvReg_X86_YMM3D1 = 377,
  SYMS_CvReg_X86_YMM3D2 = 378,
  SYMS_CvReg_X86_YMM3D3 = 379,
  SYMS_CvReg_X86_YMM4D0 = 380,
  SYMS_CvReg_X86_YMM4D1 = 381,
  SYMS_CvReg_X86_YMM4D2 = 382,
  SYMS_CvReg_X86_YMM4D3 = 383,
  SYMS_CvReg_X86_YMM5D0 = 384,
  SYMS_CvReg_X86_YMM5D1 = 385,
  SYMS_CvReg_X86_YMM5D2 = 386,
  SYMS_CvReg_X86_YMM5D3 = 387,
  SYMS_CvReg_X86_YMM6D0 = 388,
  SYMS_CvReg_X86_YMM6D1 = 389,
  SYMS_CvReg_X86_YMM6D2 = 390,
  SYMS_CvReg_X86_YMM6D3 = 391,
  SYMS_CvReg_X86_YMM7D0 = 392,
  SYMS_CvReg_X86_YMM7D1 = 393,
  SYMS_CvReg_X86_YMM7D2 = 394,
  SYMS_CvReg_X86_YMM7D3 = 395,
  
  SYMS_CvReg_X86_COUNT,
  
  ////////////////////////////////
  // NOTE(allen): AMD64 registers
  
  SYMS_CvReg_X64_AL     =  1,
  SYMS_CvReg_X64_CL     =  2,
  SYMS_CvReg_X64_DL     =  3,
  SYMS_CvReg_X64_BL     =  4,
  SYMS_CvReg_X64_AH     =  5,
  SYMS_CvReg_X64_CH     =  6,
  SYMS_CvReg_X64_DH     =  7,
  SYMS_CvReg_X64_BH     =  8,
  SYMS_CvReg_X64_AX     =  9,
  SYMS_CvReg_X64_CX     = 10,
  SYMS_CvReg_X64_DX     = 11,
  SYMS_CvReg_X64_BX     = 12,
  SYMS_CvReg_X64_SP     = 13,
  SYMS_CvReg_X64_BP     = 14,
  SYMS_CvReg_X64_SI     = 15,
  SYMS_CvReg_X64_DI     = 16,
  SYMS_CvReg_X64_EAX    = 17,
  SYMS_CvReg_X64_ECX    = 18,
  SYMS_CvReg_X64_EDX    = 19,
  SYMS_CvReg_X64_EBX    = 20,
  SYMS_CvReg_X64_ESP    = 21,
  SYMS_CvReg_X64_EBP    = 22,
  SYMS_CvReg_X64_ESI    = 23,
  SYMS_CvReg_X64_EDI    = 24,
  SYMS_CvReg_X64_ES     = 25,
  SYMS_CvReg_X64_CS     = 26,
  SYMS_CvReg_X64_SS     = 27,
  SYMS_CvReg_X64_DS     = 28,
  SYMS_CvReg_X64_FS     = 29,
  SYMS_CvReg_X64_GS     = 30,
  SYMS_CvReg_X64_FLAGS  = 32,
  SYMS_CvReg_X64_RIP    = 33,
  SYMS_CvReg_X64_EFLAGS = 34,
  
  // Control registers
  
  SYMS_CvReg_X64_CR0 = 80,
  SYMS_CvReg_X64_CR1 = 81,
  SYMS_CvReg_X64_CR2 = 82,
  SYMS_CvReg_X64_CR3 = 83,
  SYMS_CvReg_X64_CR4 = 84,
  SYMS_CvReg_X64_CR8 = 88,
  
  // Debug registers
  
  SYMS_CvReg_X64_DR0  = 90,
  SYMS_CvReg_X64_DR1  = 91,
  SYMS_CvReg_X64_DR2  = 92,
  SYMS_CvReg_X64_DR3  = 93,
  SYMS_CvReg_X64_DR4  = 94,
  SYMS_CvReg_X64_DR5  = 95,
  SYMS_CvReg_X64_DR6  = 96,
  SYMS_CvReg_X64_DR7  = 97,
  SYMS_CvReg_X64_DR8  = 98,
  SYMS_CvReg_X64_DR9  = 99,
  SYMS_CvReg_X64_DR10 = 100,
  SYMS_CvReg_X64_DR11 = 101,
  SYMS_CvReg_X64_DR12 = 102,
  SYMS_CvReg_X64_DR13 = 103,
  SYMS_CvReg_X64_DR14 = 104,
  SYMS_CvReg_X64_DR15 = 105,
  
  SYMS_CvReg_X64_GDTR = 110,
  SYMS_CvReg_X64_GDTL = 111,
  SYMS_CvReg_X64_IDTR = 112,
  SYMS_CvReg_X64_IDTL = 113,
  SYMS_CvReg_X64_LDTR = 114,
  SYMS_CvReg_X64_TR   = 115,
  
  SYMS_CvReg_X64_ST0   = 128,
  SYMS_CvReg_X64_ST1   = 129,
  SYMS_CvReg_X64_ST2   = 130,
  SYMS_CvReg_X64_ST3   = 131,
  SYMS_CvReg_X64_ST4   = 132,
  SYMS_CvReg_X64_ST5   = 133,
  SYMS_CvReg_X64_ST6   = 134,
  SYMS_CvReg_X64_ST7   = 135,
  SYMS_CvReg_X64_CTRL  = 136,
  SYMS_CvReg_X64_STAT  = 137,
  SYMS_CvReg_X64_TAG   = 138,
  SYMS_CvReg_X64_FPIP  = 139,
  SYMS_CvReg_X64_FPCS  = 140,
  SYMS_CvReg_X64_FPDO  = 141,
  SYMS_CvReg_X64_FPDS  = 142,
  SYMS_CvReg_X64_ISEM  = 143,
  SYMS_CvReg_X64_FPEIP = 144,
  SYMS_CvReg_X64_FPEDO = 145,
  
  SYMS_CvReg_X64_MM0 = 146,
  SYMS_CvReg_X64_MM1 = 147,
  SYMS_CvReg_X64_MM2 = 148,
  SYMS_CvReg_X64_MM3 = 149,
  SYMS_CvReg_X64_MM4 = 150,
  SYMS_CvReg_X64_MM5 = 151,
  SYMS_CvReg_X64_MM6 = 152,
  SYMS_CvReg_X64_MM7 = 153,
  
  SYMS_CvReg_X64_XMM0 = 154,
  SYMS_CvReg_X64_XMM1 = 155,
  SYMS_CvReg_X64_XMM2 = 156,
  SYMS_CvReg_X64_XMM3 = 157,
  SYMS_CvReg_X64_XMM4 = 158,
  SYMS_CvReg_X64_XMM5 = 159,
  SYMS_CvReg_X64_XMM6 = 160,
  SYMS_CvReg_X64_XMM7 = 161,
  
  SYMS_CvReg_X64_XMM0_0 = 162,
  SYMS_CvReg_X64_XMM0_1 = 163,
  SYMS_CvReg_X64_XMM0_2 = 164,
  SYMS_CvReg_X64_XMM0_3 = 165,
  SYMS_CvReg_X64_XMM1_0 = 166,
  SYMS_CvReg_X64_XMM1_1 = 167,
  SYMS_CvReg_X64_XMM1_2 = 168,
  SYMS_CvReg_X64_XMM1_3 = 169,
  SYMS_CvReg_X64_XMM2_0 = 170,
  SYMS_CvReg_X64_XMM2_1 = 171,
  SYMS_CvReg_X64_XMM2_2 = 172,
  SYMS_CvReg_X64_XMM2_3 = 173,
  SYMS_CvReg_X64_XMM3_0 = 174,
  SYMS_CvReg_X64_XMM3_1 = 175,
  SYMS_CvReg_X64_XMM3_2 = 176,
  SYMS_CvReg_X64_XMM3_3 = 177,
  SYMS_CvReg_X64_XMM4_0 = 178,
  SYMS_CvReg_X64_XMM4_1 = 179,
  SYMS_CvReg_X64_XMM4_2 = 180,
  SYMS_CvReg_X64_XMM4_3 = 181,
  SYMS_CvReg_X64_XMM5_0 = 182,
  SYMS_CvReg_X64_XMM5_1 = 183,
  SYMS_CvReg_X64_XMM5_2 = 184,
  SYMS_CvReg_X64_XMM5_3 = 185,
  SYMS_CvReg_X64_XMM6_0 = 186,
  SYMS_CvReg_X64_XMM6_1 = 187,
  SYMS_CvReg_X64_XMM6_2 = 188,
  SYMS_CvReg_X64_XMM6_3 = 189,
  SYMS_CvReg_X64_XMM7_0 = 190,
  SYMS_CvReg_X64_XMM7_1 = 191,
  SYMS_CvReg_X64_XMM7_2 = 192,
  SYMS_CvReg_X64_XMM7_3 = 193,
  
  SYMS_CvReg_X64_XMM0L = 194,
  SYMS_CvReg_X64_XMM1L = 195,
  SYMS_CvReg_X64_XMM2L = 196,
  SYMS_CvReg_X64_XMM3L = 197,
  SYMS_CvReg_X64_XMM4L = 198,
  SYMS_CvReg_X64_XMM5L = 199,
  SYMS_CvReg_X64_XMM6L = 200,
  SYMS_CvReg_X64_XMM7L = 201,
  
  SYMS_CvReg_X64_XMM0H = 202,
  SYMS_CvReg_X64_XMM1H = 203,
  SYMS_CvReg_X64_XMM2H = 204,
  SYMS_CvReg_X64_XMM3H = 205,
  SYMS_CvReg_X64_XMM4H = 206,
  SYMS_CvReg_X64_XMM5H = 207,
  SYMS_CvReg_X64_XMM6H = 208,
  SYMS_CvReg_X64_XMM7H = 209,
  
  // XMM status register
  SYMS_CvReg_X64_MXCSR = 211,
  
  // XMM sub-registers (WNI integer)
  SYMS_CvReg_X64_EMM0L = 220,
  SYMS_CvReg_X64_EMM1L = 221,
  SYMS_CvReg_X64_EMM2L = 222,
  SYMS_CvReg_X64_EMM3L = 223,
  SYMS_CvReg_X64_EMM4L = 224,
  SYMS_CvReg_X64_EMM5L = 225,
  SYMS_CvReg_X64_EMM6L = 226,
  SYMS_CvReg_X64_EMM7L = 227,
  
  SYMS_CvReg_X64_EMM0H = 228,
  SYMS_CvReg_X64_EMM1H = 229,
  SYMS_CvReg_X64_EMM2H = 230,
  SYMS_CvReg_X64_EMM3H = 231,
  SYMS_CvReg_X64_EMM4H = 232,
  SYMS_CvReg_X64_EMM5H = 233,
  SYMS_CvReg_X64_EMM6H = 234,
  SYMS_CvReg_X64_EMM7H = 235,
  
  // do not change the order of these regs, first one must be even too
  
  SYMS_CvReg_X64_MM00 = 236,
  SYMS_CvReg_X64_MM01 = 237,
  SYMS_CvReg_X64_MM10 = 238,
  SYMS_CvReg_X64_MM11 = 239,
  SYMS_CvReg_X64_MM20 = 240,
  SYMS_CvReg_X64_MM21 = 241,
  SYMS_CvReg_X64_MM30 = 242,
  SYMS_CvReg_X64_MM31 = 243,
  SYMS_CvReg_X64_MM40 = 244,
  SYMS_CvReg_X64_MM41 = 245,
  SYMS_CvReg_X64_MM50 = 246,
  SYMS_CvReg_X64_MM51 = 247,
  SYMS_CvReg_X64_MM60 = 248,
  SYMS_CvReg_X64_MM61 = 249,
  SYMS_CvReg_X64_MM70 = 250,
  SYMS_CvReg_X64_MM71 = 251,
  
  SYMS_CvReg_X64_XMM8  = 252,
  SYMS_CvReg_X64_XMM9  = 253,
  SYMS_CvReg_X64_XMM10 = 254,
  SYMS_CvReg_X64_XMM11 = 255,
  SYMS_CvReg_X64_XMM12 = 256,
  SYMS_CvReg_X64_XMM13 = 257,
  SYMS_CvReg_X64_XMM14 = 258,
  SYMS_CvReg_X64_XMM15 = 259,
  
  SYMS_CvReg_X64_XMM8_0  = 260,
  SYMS_CvReg_X64_XMM8_1  = 261,
  SYMS_CvReg_X64_XMM8_2  = 262,
  SYMS_CvReg_X64_XMM8_3  = 263,
  SYMS_CvReg_X64_XMM9_0  = 264,
  SYMS_CvReg_X64_XMM9_1  = 265,
  SYMS_CvReg_X64_XMM9_2  = 266,
  SYMS_CvReg_X64_XMM9_3  = 267,
  SYMS_CvReg_X64_XMM10_0 = 268,
  SYMS_CvReg_X64_XMM10_1 = 269,
  SYMS_CvReg_X64_XMM10_2 = 270,
  SYMS_CvReg_X64_XMM10_3 = 271,
  SYMS_CvReg_X64_XMM11_0 = 272,
  SYMS_CvReg_X64_XMM11_1 = 273,
  SYMS_CvReg_X64_XMM11_2 = 274,
  SYMS_CvReg_X64_XMM11_3 = 275,
  SYMS_CvReg_X64_XMM12_0 = 276,
  SYMS_CvReg_X64_XMM12_1 = 277,
  SYMS_CvReg_X64_XMM12_2 = 278,
  SYMS_CvReg_X64_XMM12_3 = 279,
  SYMS_CvReg_X64_XMM13_0 = 280,
  SYMS_CvReg_X64_XMM13_1 = 281,
  SYMS_CvReg_X64_XMM13_2 = 282,
  SYMS_CvReg_X64_XMM13_3 = 283,
  SYMS_CvReg_X64_XMM14_0 = 284,
  SYMS_CvReg_X64_XMM14_1 = 285,
  SYMS_CvReg_X64_XMM14_2 = 286,
  SYMS_CvReg_X64_XMM14_3 = 287,
  SYMS_CvReg_X64_XMM15_0 = 288,
  SYMS_CvReg_X64_XMM15_1 = 289,
  SYMS_CvReg_X64_XMM15_2 = 290,
  SYMS_CvReg_X64_XMM15_3 = 291,
  
  SYMS_CvReg_X64_XMM8L  = 292,
  SYMS_CvReg_X64_XMM9L  = 293,
  SYMS_CvReg_X64_XMM10L = 294,
  SYMS_CvReg_X64_XMM11L = 295,
  SYMS_CvReg_X64_XMM12L = 296,
  SYMS_CvReg_X64_XMM13L = 297,
  SYMS_CvReg_X64_XMM14L = 298,
  SYMS_CvReg_X64_XMM15L = 299,
  
  SYMS_CvReg_X64_XMM8H  = 300,
  SYMS_CvReg_X64_XMM9H  = 301,
  SYMS_CvReg_X64_XMM10H = 302,
  SYMS_CvReg_X64_XMM11H = 303,
  SYMS_CvReg_X64_XMM12H = 304,
  SYMS_CvReg_X64_XMM13H = 305,
  SYMS_CvReg_X64_XMM14H = 306,
  SYMS_CvReg_X64_XMM15H = 307,
  
  // XMM sub-registers (WNI integer)
  
  SYMS_CvReg_X64_EMM8L  = 308,
  SYMS_CvReg_X64_EMM9L  = 309,
  SYMS_CvReg_X64_EMM10L = 310,
  SYMS_CvReg_X64_EMM11L = 311,
  SYMS_CvReg_X64_EMM12L = 312,
  SYMS_CvReg_X64_EMM13L = 313,
  SYMS_CvReg_X64_EMM14L = 314,
  SYMS_CvReg_X64_EMM15L = 315,
  
  SYMS_CvReg_X64_EMM8H  = 316,
  SYMS_CvReg_X64_EMM9H  = 317,
  SYMS_CvReg_X64_EMM10H = 318,
  SYMS_CvReg_X64_EMM11H = 319,
  SYMS_CvReg_X64_EMM12H = 320,
  SYMS_CvReg_X64_EMM13H = 321,
  SYMS_CvReg_X64_EMM14H = 322,
  SYMS_CvReg_X64_EMM15H = 323,
  
  // Low byte forms of some standard registers
  SYMS_CvReg_X64_SIL = 324,
  SYMS_CvReg_X64_DIL = 325,
  SYMS_CvReg_X64_BPL = 326,
  SYMS_CvReg_X64_SPL = 327,
  
  // 64-bit regular registers
  SYMS_CvReg_X64_RAX = 328,
  SYMS_CvReg_X64_RBX = 329,
  SYMS_CvReg_X64_RCX = 330,
  SYMS_CvReg_X64_RDX = 331,
  SYMS_CvReg_X64_RSI = 332,
  SYMS_CvReg_X64_RDI = 333,
  SYMS_CvReg_X64_RBP = 334,
  SYMS_CvReg_X64_RSP = 335,
  
  // 64-bit integer registers with 8-, 16-, and 32-bit forms (B, W, and D)
  SYMS_CvReg_X64_R8  = 336,
  SYMS_CvReg_X64_R9  = 337,
  SYMS_CvReg_X64_R10 = 338,
  SYMS_CvReg_X64_R11 = 339,
  SYMS_CvReg_X64_R12 = 340,
  SYMS_CvReg_X64_R13 = 341,
  SYMS_CvReg_X64_R14 = 342,
  SYMS_CvReg_X64_R15 = 343,
  
  SYMS_CvReg_X64_R8B  = 344,
  SYMS_CvReg_X64_R9B  = 345,
  SYMS_CvReg_X64_R10B = 346,
  SYMS_CvReg_X64_R11B = 347,
  SYMS_CvReg_X64_R12B = 348,
  SYMS_CvReg_X64_R13B = 349,
  SYMS_CvReg_X64_R14B = 350,
  SYMS_CvReg_X64_R15B = 351,
  
  SYMS_CvReg_X64_R8W  = 352,
  SYMS_CvReg_X64_R9W  = 353,
  SYMS_CvReg_X64_R10W = 354,
  SYMS_CvReg_X64_R11W = 355,
  SYMS_CvReg_X64_R12W = 356,
  SYMS_CvReg_X64_R13W = 357,
  SYMS_CvReg_X64_R14W = 358,
  SYMS_CvReg_X64_R15W = 359,
  
  SYMS_CvReg_X64_R8D  = 360,
  SYMS_CvReg_X64_R9D  = 361,
  SYMS_CvReg_X64_R10D = 362,
  SYMS_CvReg_X64_R11D = 363,
  SYMS_CvReg_X64_R12D = 364,
  SYMS_CvReg_X64_R13D = 365,
  SYMS_CvReg_X64_R14D = 366,
  SYMS_CvReg_X64_R15D = 367,
  
  // AVX registers 256 bits
  
  SYMS_CvReg_X64_YMM0  = 368,
  SYMS_CvReg_X64_YMM1  = 369,
  SYMS_CvReg_X64_YMM2  = 370,
  SYMS_CvReg_X64_YMM3  = 371,
  SYMS_CvReg_X64_YMM4  = 372,
  SYMS_CvReg_X64_YMM5  = 373,
  SYMS_CvReg_X64_YMM6  = 374,
  SYMS_CvReg_X64_YMM7  = 375,
  SYMS_CvReg_X64_YMM8  = 376,
  SYMS_CvReg_X64_YMM9  = 377,
  SYMS_CvReg_X64_YMM10 = 378,
  SYMS_CvReg_X64_YMM11 = 379,
  SYMS_CvReg_X64_YMM12 = 380,
  SYMS_CvReg_X64_YMM13 = 381,
  SYMS_CvReg_X64_YMM14 = 382,
  SYMS_CvReg_X64_YMM15 = 383,
  
  // AVX registers upper 128 bits
  SYMS_CvReg_X64_YMM0H  = 384,
  SYMS_CvReg_X64_YMM1H  = 385,
  SYMS_CvReg_X64_YMM2H  = 386,
  SYMS_CvReg_X64_YMM3H  = 387,
  SYMS_CvReg_X64_YMM4H  = 388,
  SYMS_CvReg_X64_YMM5H  = 389,
  SYMS_CvReg_X64_YMM6H  = 390,
  SYMS_CvReg_X64_YMM7H  = 391,
  SYMS_CvReg_X64_YMM8H  = 392,
  SYMS_CvReg_X64_YMM9H  = 393,
  SYMS_CvReg_X64_YMM10H = 394,
  SYMS_CvReg_X64_YMM11H = 395,
  SYMS_CvReg_X64_YMM12H = 396,
  SYMS_CvReg_X64_YMM13H = 397,
  SYMS_CvReg_X64_YMM14H = 398,
  SYMS_CvReg_X64_YMM15H = 399,
  
  /* Lower/upper 8 bytes of XMM registers.  Unlike CV_AMD64_XMM<regnum><H/L>, these
   * values reprsesent the bit patterns of the registers as 64-bit integers, not
   * the representation of these registers as a double. */
  SYMS_CvReg_X64_XMM0IL  = 400,
  SYMS_CvReg_X64_XMM1IL  = 401,
  SYMS_CvReg_X64_XMM2IL  = 402,
  SYMS_CvReg_X64_XMM3IL  = 403,
  SYMS_CvReg_X64_XMM4IL  = 404,
  SYMS_CvReg_X64_XMM5IL  = 405,
  SYMS_CvReg_X64_XMM6IL  = 406,
  SYMS_CvReg_X64_XMM7IL  = 407,
  SYMS_CvReg_X64_XMM8IL  = 408,
  SYMS_CvReg_X64_XMM9IL  = 409,
  SYMS_CvReg_X64_XMM10IL = 410,
  SYMS_CvReg_X64_XMM11IL = 411,
  SYMS_CvReg_X64_XMM12IL = 412,
  SYMS_CvReg_X64_XMM13IL = 413,
  SYMS_CvReg_X64_XMM14IL = 414,
  SYMS_CvReg_X64_XMM15IL = 415,
  
  SYMS_CvReg_X64_XMM0IH  = 416,
  SYMS_CvReg_X64_XMM1IH  = 417,
  SYMS_CvReg_X64_XMM2IH  = 418,
  SYMS_CvReg_X64_XMM3IH  = 419,
  SYMS_CvReg_X64_XMM4IH  = 420,
  SYMS_CvReg_X64_XMM5IH  = 421,
  SYMS_CvReg_X64_XMM6IH  = 422,
  SYMS_CvReg_X64_XMM7IH  = 423,
  SYMS_CvReg_X64_XMM8IH  = 424,
  SYMS_CvReg_X64_XMM9IH  = 425,
  SYMS_CvReg_X64_XMM10IH = 426,
  SYMS_CvReg_X64_XMM11IH = 427,
  SYMS_CvReg_X64_XMM12IH = 428,
  SYMS_CvReg_X64_XMM13IH = 429,
  SYMS_CvReg_X64_XMM14IH = 430,
  SYMS_CvReg_X64_XMM15IH = 431,
  
  // AVX integer registers
  
  SYMS_CvReg_X64_YMM0I0  = 432,
  SYMS_CvReg_X64_YMM0I1  = 433,
  SYMS_CvReg_X64_YMM0I2  = 434,
  SYMS_CvReg_X64_YMM0I3  = 435,
  SYMS_CvReg_X64_YMM1I0  = 436,
  SYMS_CvReg_X64_YMM1I1  = 437,
  SYMS_CvReg_X64_YMM1I2  = 438,
  SYMS_CvReg_X64_YMM1I3  = 439,
  SYMS_CvReg_X64_YMM2I0  = 440,
  SYMS_CvReg_X64_YMM2I1  = 441,
  SYMS_CvReg_X64_YMM2I2  = 442,
  SYMS_CvReg_X64_YMM2I3  = 443,
  SYMS_CvReg_X64_YMM3I0  = 444,
  SYMS_CvReg_X64_YMM3I1  = 445,
  SYMS_CvReg_X64_YMM3I2  = 446,
  SYMS_CvReg_X64_YMM3I3  = 447,
  SYMS_CvReg_X64_YMM4I0  = 448,
  SYMS_CvReg_X64_YMM4I1  = 449,
  SYMS_CvReg_X64_YMM4I2  = 450,
  SYMS_CvReg_X64_YMM4I3  = 451,
  SYMS_CvReg_X64_YMM5I0  = 452,
  SYMS_CvReg_X64_YMM5I1  = 453,
  SYMS_CvReg_X64_YMM5I2  = 454,
  SYMS_CvReg_X64_YMM5I3  = 455,
  SYMS_CvReg_X64_YMM6I0  = 456,
  SYMS_CvReg_X64_YMM6I1  = 457,
  SYMS_CvReg_X64_YMM6I2  = 458,
  SYMS_CvReg_X64_YMM6I3  = 459,
  SYMS_CvReg_X64_YMM7I0  = 460,
  SYMS_CvReg_X64_YMM7I1  = 461,
  SYMS_CvReg_X64_YMM7I2  = 462,
  SYMS_CvReg_X64_YMM7I3  = 463,
  SYMS_CvReg_X64_YMM8I0  = 464,
  SYMS_CvReg_X64_YMM8I1  = 465,
  SYMS_CvReg_X64_YMM8I2  = 466,
  SYMS_CvReg_X64_YMM8I3  = 467,
  SYMS_CvReg_X64_YMM9I0  = 468,
  SYMS_CvReg_X64_YMM9I1  = 469,
  SYMS_CvReg_X64_YMM9I2  = 470,
  SYMS_CvReg_X64_YMM9I3  = 471,
  SYMS_CvReg_X64_YMM10I0 = 472,
  SYMS_CvReg_X64_YMM10I1 = 473,
  SYMS_CvReg_X64_YMM10I2 = 474,
  SYMS_CvReg_X64_YMM10I3 = 475,
  SYMS_CvReg_X64_YMM11I0 = 476,
  SYMS_CvReg_X64_YMM11I1 = 477,
  SYMS_CvReg_X64_YMM11I2 = 478,
  SYMS_CvReg_X64_YMM11I3 = 479,
  SYMS_CvReg_X64_YMM12I0 = 480,
  SYMS_CvReg_X64_YMM12I1 = 481,
  SYMS_CvReg_X64_YMM12I2 = 482,
  SYMS_CvReg_X64_YMM12I3 = 483,
  SYMS_CvReg_X64_YMM13I0 = 484,
  SYMS_CvReg_X64_YMM13I1 = 485,
  SYMS_CvReg_X64_YMM13I2 = 486,
  SYMS_CvReg_X64_YMM13I3 = 487,
  SYMS_CvReg_X64_YMM14I0 = 488,
  SYMS_CvReg_X64_YMM14I1 = 489,
  SYMS_CvReg_X64_YMM14I2 = 490,
  SYMS_CvReg_X64_YMM14I3 = 491,
  SYMS_CvReg_X64_YMM15I0 = 492,
  SYMS_CvReg_X64_YMM15I1 = 493,
  SYMS_CvReg_X64_YMM15I2 = 494,
  SYMS_CvReg_X64_YMM15I3 = 495,
  
  // AVX floating-point single precise registers
  
  SYMS_CvReg_X64_YMM0F0 = 496,
  SYMS_CvReg_X64_YMM0F1 = 497,
  SYMS_CvReg_X64_YMM0F2 = 498,
  SYMS_CvReg_X64_YMM0F3 = 499,
  SYMS_CvReg_X64_YMM0F4 = 500,
  SYMS_CvReg_X64_YMM0F5 = 501,
  SYMS_CvReg_X64_YMM0F6 = 502,
  SYMS_CvReg_X64_YMM0F7 = 503,
  SYMS_CvReg_X64_YMM1F0 = 504,
  SYMS_CvReg_X64_YMM1F1 = 505,
  SYMS_CvReg_X64_YMM1F2 = 506,
  SYMS_CvReg_X64_YMM1F3 = 507,
  SYMS_CvReg_X64_YMM1F4 = 508,
  SYMS_CvReg_X64_YMM1F5 = 509,
  SYMS_CvReg_X64_YMM1F6 = 510,
  SYMS_CvReg_X64_YMM1F7 = 511,
  SYMS_CvReg_X64_YMM2F0 = 512,
  SYMS_CvReg_X64_YMM2F1 = 513,
  SYMS_CvReg_X64_YMM2F2 = 514,
  SYMS_CvReg_X64_YMM2F3 = 515,
  SYMS_CvReg_X64_YMM2F4 = 516,
  SYMS_CvReg_X64_YMM2F5 = 517,
  SYMS_CvReg_X64_YMM2F6 = 518,
  SYMS_CvReg_X64_YMM2F7 = 519,
  SYMS_CvReg_X64_YMM3F0 = 520,
  SYMS_CvReg_X64_YMM3F1 = 521,
  SYMS_CvReg_X64_YMM3F2 = 522,
  SYMS_CvReg_X64_YMM3F3 = 523,
  SYMS_CvReg_X64_YMM3F4 = 524,
  SYMS_CvReg_X64_YMM3F5 = 525,
  SYMS_CvReg_X64_YMM3F6 = 526,
  SYMS_CvReg_X64_YMM3F7 = 527,
  
  SYMS_CvReg_X64_YMM4F0 = 528,
  SYMS_CvReg_X64_YMM4F1 = 529,
  SYMS_CvReg_X64_YMM4F2 = 530,
  SYMS_CvReg_X64_YMM4F3 = 531,
  SYMS_CvReg_X64_YMM4F4 = 532,
  SYMS_CvReg_X64_YMM4F5 = 533,
  SYMS_CvReg_X64_YMM4F6 = 534,
  SYMS_CvReg_X64_YMM4F7 = 535,
  SYMS_CvReg_X64_YMM5F0 = 536,
  SYMS_CvReg_X64_YMM5F1 = 537,
  SYMS_CvReg_X64_YMM5F2 = 538,
  SYMS_CvReg_X64_YMM5F3 = 539,
  SYMS_CvReg_X64_YMM5F4 = 540,
  SYMS_CvReg_X64_YMM5F5 = 541,
  SYMS_CvReg_X64_YMM5F6 = 542,
  SYMS_CvReg_X64_YMM5F7 = 543,
  SYMS_CvReg_X64_YMM6F0 = 544,
  SYMS_CvReg_X64_YMM6F1 = 545,
  SYMS_CvReg_X64_YMM6F2 = 546,
  SYMS_CvReg_X64_YMM6F3 = 547,
  SYMS_CvReg_X64_YMM6F4 = 548,
  SYMS_CvReg_X64_YMM6F5 = 549,
  SYMS_CvReg_X64_YMM6F6 = 550,
  SYMS_CvReg_X64_YMM6F7 = 551,
  SYMS_CvReg_X64_YMM7F0 = 552,
  SYMS_CvReg_X64_YMM7F1 = 553,
  SYMS_CvReg_X64_YMM7F2 = 554,
  SYMS_CvReg_X64_YMM7F3 = 555,
  SYMS_CvReg_X64_YMM7F4 = 556,
  SYMS_CvReg_X64_YMM7F5 = 557,
  SYMS_CvReg_X64_YMM7F6 = 558,
  SYMS_CvReg_X64_YMM7F7 = 559,
  
  SYMS_CvReg_X64_YMM8F0  = 560,
  SYMS_CvReg_X64_YMM8F1  = 561,
  SYMS_CvReg_X64_YMM8F2  = 562,
  SYMS_CvReg_X64_YMM8F3  = 563,
  SYMS_CvReg_X64_YMM8F4  = 564,
  SYMS_CvReg_X64_YMM8F5  = 565,
  SYMS_CvReg_X64_YMM8F6  = 566,
  SYMS_CvReg_X64_YMM8F7  = 567,
  SYMS_CvReg_X64_YMM9F0  = 568,
  SYMS_CvReg_X64_YMM9F1  = 569,
  SYMS_CvReg_X64_YMM9F2  = 570,
  SYMS_CvReg_X64_YMM9F3  = 571,
  SYMS_CvReg_X64_YMM9F4  = 572,
  SYMS_CvReg_X64_YMM9F5  = 573,
  SYMS_CvReg_X64_YMM9F6  = 574,
  SYMS_CvReg_X64_YMM9F7  = 575,
  SYMS_CvReg_X64_YMM10F0 = 576,
  SYMS_CvReg_X64_YMM10F1 = 577,
  SYMS_CvReg_X64_YMM10F2 = 578,
  SYMS_CvReg_X64_YMM10F3 = 579,
  SYMS_CvReg_X64_YMM10F4 = 580,
  SYMS_CvReg_X64_YMM10F5 = 581,
  SYMS_CvReg_X64_YMM10F6 = 582,
  SYMS_CvReg_X64_YMM10F7 = 583,
  SYMS_CvReg_X64_YMM11F0 = 584,
  SYMS_CvReg_X64_YMM11F1 = 585,
  SYMS_CvReg_X64_YMM11F2 = 586,
  SYMS_CvReg_X64_YMM11F3 = 587,
  SYMS_CvReg_X64_YMM11F4 = 588,
  SYMS_CvReg_X64_YMM11F5 = 589,
  SYMS_CvReg_X64_YMM11F6 = 590,
  SYMS_CvReg_X64_YMM11F7 = 591,
  
  SYMS_CvReg_X64_YMM12F0 = 592,
  SYMS_CvReg_X64_YMM12F1 = 593,
  SYMS_CvReg_X64_YMM12F2 = 594,
  SYMS_CvReg_X64_YMM12F3 = 595,
  SYMS_CvReg_X64_YMM12F4 = 596,
  SYMS_CvReg_X64_YMM12F5 = 597,
  SYMS_CvReg_X64_YMM12F6 = 598,
  SYMS_CvReg_X64_YMM12F7 = 599,
  SYMS_CvReg_X64_YMM13F0 = 600,
  SYMS_CvReg_X64_YMM13F1 = 601,
  SYMS_CvReg_X64_YMM13F2 = 602,
  SYMS_CvReg_X64_YMM13F3 = 603,
  SYMS_CvReg_X64_YMM13F4 = 604,
  SYMS_CvReg_X64_YMM13F5 = 605,
  SYMS_CvReg_X64_YMM13F6 = 606,
  SYMS_CvReg_X64_YMM13F7 = 607,
  SYMS_CvReg_X64_YMM14F0 = 608,
  SYMS_CvReg_X64_YMM14F1 = 609,
  SYMS_CvReg_X64_YMM14F2 = 610,
  SYMS_CvReg_X64_YMM14F3 = 611,
  SYMS_CvReg_X64_YMM14F4 = 612,
  SYMS_CvReg_X64_YMM14F5 = 613,
  SYMS_CvReg_X64_YMM14F6 = 614,
  SYMS_CvReg_X64_YMM14F7 = 615,
  SYMS_CvReg_X64_YMM15F0 = 616,
  SYMS_CvReg_X64_YMM15F1 = 617,
  SYMS_CvReg_X64_YMM15F2 = 618,
  SYMS_CvReg_X64_YMM15F3 = 619,
  SYMS_CvReg_X64_YMM15F4 = 620,
  SYMS_CvReg_X64_YMM15F5 = 621,
  SYMS_CvReg_X64_YMM15F6 = 622,
  SYMS_CvReg_X64_YMM15F7 = 623,
  
  
  // AVX floating-point double precise registers
  
  SYMS_CvReg_X64_YMM0D0 = 624,
  SYMS_CvReg_X64_YMM0D1 = 625,
  SYMS_CvReg_X64_YMM0D2 = 626,
  SYMS_CvReg_X64_YMM0D3 = 627,
  SYMS_CvReg_X64_YMM1D0 = 628,
  SYMS_CvReg_X64_YMM1D1 = 629,
  SYMS_CvReg_X64_YMM1D2 = 630,
  SYMS_CvReg_X64_YMM1D3 = 631,
  SYMS_CvReg_X64_YMM2D0 = 632,
  SYMS_CvReg_X64_YMM2D1 = 633,
  SYMS_CvReg_X64_YMM2D2 = 634,
  SYMS_CvReg_X64_YMM2D3 = 635,
  SYMS_CvReg_X64_YMM3D0 = 636,
  SYMS_CvReg_X64_YMM3D1 = 637,
  SYMS_CvReg_X64_YMM3D2 = 638,
  SYMS_CvReg_X64_YMM3D3 = 639,
  SYMS_CvReg_X64_YMM4D0 = 640,
  SYMS_CvReg_X64_YMM4D1 = 641,
  SYMS_CvReg_X64_YMM4D2 = 642,
  SYMS_CvReg_X64_YMM4D3 = 643,
  SYMS_CvReg_X64_YMM5D0 = 644,
  SYMS_CvReg_X64_YMM5D1 = 645,
  SYMS_CvReg_X64_YMM5D2 = 646,
  SYMS_CvReg_X64_YMM5D3 = 647,
  SYMS_CvReg_X64_YMM6D0 = 648,
  SYMS_CvReg_X64_YMM6D1 = 649,
  SYMS_CvReg_X64_YMM6D2 = 650,
  SYMS_CvReg_X64_YMM6D3 = 651,
  SYMS_CvReg_X64_YMM7D0 = 652,
  SYMS_CvReg_X64_YMM7D1 = 653,
  SYMS_CvReg_X64_YMM7D2 = 654,
  SYMS_CvReg_X64_YMM7D3 = 655,
  
  SYMS_CvReg_X64_YMM8D0  = 656,
  SYMS_CvReg_X64_YMM8D1  = 657,
  SYMS_CvReg_X64_YMM8D2  = 658,
  SYMS_CvReg_X64_YMM8D3  = 659,
  SYMS_CvReg_X64_YMM9D0  = 660,
  SYMS_CvReg_X64_YMM9D1  = 661,
  SYMS_CvReg_X64_YMM9D2  = 662,
  SYMS_CvReg_X64_YMM9D3  = 663,
  SYMS_CvReg_X64_YMM10D0 = 664,
  SYMS_CvReg_X64_YMM10D1 = 665,
  SYMS_CvReg_X64_YMM10D2 = 666,
  SYMS_CvReg_X64_YMM10D3 = 667,
  SYMS_CvReg_X64_YMM11D0 = 668,
  SYMS_CvReg_X64_YMM11D1 = 669,
  SYMS_CvReg_X64_YMM11D2 = 670,
  SYMS_CvReg_X64_YMM11D3 = 671,
  SYMS_CvReg_X64_YMM12D0 = 672,
  SYMS_CvReg_X64_YMM12D1 = 673,
  SYMS_CvReg_X64_YMM12D2 = 674,
  SYMS_CvReg_X64_YMM12D3 = 675,
  SYMS_CvReg_X64_YMM13D0 = 676,
  SYMS_CvReg_X64_YMM13D1 = 677,
  SYMS_CvReg_X64_YMM13D2 = 678,
  SYMS_CvReg_X64_YMM13D3 = 679,
  SYMS_CvReg_X64_YMM14D0 = 680,
  SYMS_CvReg_X64_YMM14D1 = 681,
  SYMS_CvReg_X64_YMM14D2 = 682,
  SYMS_CvReg_X64_YMM14D3 = 683,
  SYMS_CvReg_X64_YMM15D0 = 684,
  SYMS_CvReg_X64_YMM15D1 = 685,
  SYMS_CvReg_X64_YMM15D2 = 686,
  SYMS_CvReg_X64_YMM15D3 = 687,
  
  SYMS_CvReg_X64_COUNT,
  
  SYMS_CvReg_MAX_COUNT = SYMS_CvReg_X64_COUNT
};

////////////////////////////////
//~ allen: PDB Symbol Parser Helper

// TODO(allen): rename SYMS_CvElementHeader
//SYMS_CvSubSecKind_SYMBOLS
typedef struct SYMS_CvSymbolHelper{
  SYMS_U16 size;
  SYMS_U16 type;
} SYMS_CvSymbolHelper;


////////////////////////////////
//~ allen: PDB Misc. Types

typedef struct SYMS_CvMethod{
  SYMS_CvFieldAttribs attribs;
  SYMS_U16 pad;
  SYMS_CvTypeId itype;
  // unsigned long vbaseoff[0];
} SYMS_CvMethod;

////////////////////////////////
//~ allen: "C13" Lines

#define SYMS_CvSubSectionKind_IGNORE_FLAG 0x80000000

// TODO(allen): move to .mc file

typedef SYMS_U16 SYMS_CvSubSecLinesFlags;
enum{
  SYMS_CvSubSecLinesFlag_HasColumns = (1 << 0),
};

//SYMS_CvSubSecKind_LINES
typedef struct SYMS_CvSubSecLinesHeader{
  SYMS_U32 sec_off;
  SYMS_CvSectionIndex sec;
  SYMS_CvSubSecLinesFlags flags;
  SYMS_U32 len;
} SYMS_CvSubSecLinesHeader;

typedef struct SYMS_CvFile{
  SYMS_U32 file_off;
  SYMS_U32 num_lines;
  SYMS_U32 block_size;
  // SYMS_CvLine   lines[num_lines];
  // SYMS_CvColumn columns[num_columns]; // num_columns?
} SYMS_CvFile;

typedef SYMS_U32 SYMS_CvLineFlags;
enum{
  SYMS_CvLineFlag_LINE_NUMBER_MASK   = 0xffffff,
  SYMS_CvLineFlag_LINE_NUMBER_SHIFT  = 0,
  SYMS_CvLineFlag_DELTA_TO_END_MASK  = 0x7f,
  SYMS_CvLineFlag_DELTA_TO_END_SHIFT = 24,
  SYMS_CvLineFlag_STATEMENT          = (1 << 31)
};

typedef struct SYMS_CvLine{
  SYMS_U32 off;
  SYMS_CvLineFlags flags;
} SYMS_CvLine;

typedef struct SYMS_CvColumn{
  SYMS_U16 start;
  SYMS_U16 end;
} SYMS_CvColumn;

////////////////////////////////
// Data structs for C13 frame data sub-section

enum {
  SYMS_CvFrameDataFlag_HAS_SEH       = (1 << 0),
  SYMS_CvFrameDataFlag_HAS_EH        = (1 << 1),
  SYMS_CvFrameDataFlag_IS_FUNC_START = (1 << 2)
};
typedef SYMS_U32 SYMS_CvFrameDataFlags;

typedef struct SYMS_CvFrameData{
  SYMS_U32 start_voff;
  SYMS_U32 code_size;
  SYMS_U32 local_size;
  SYMS_U32 params_size;
  SYMS_U32 max_stack_size;
  SYMS_U32 frame_func;
  SYMS_U16 prolog_size;
  SYMS_U16 saved_reg_size;
  SYMS_CvFrameDataFlags flags;
} SYMS_CvFrameData;

////////////////////////////////
//~ allen: End Codeview Packed Types

#pragma pack(pop)

#endif //SYMS_CV_H
