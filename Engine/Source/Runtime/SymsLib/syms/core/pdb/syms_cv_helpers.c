// Copyright Epic Games, Inc. All Rights Reserved.

////////////////////////////////
//~ allen: CodeView Parser Functions

// cv parse helers

SYMS_API SYMS_CvElement
syms_cv_element(SYMS_String8 data, SYMS_MsfAccel *msf, SYMS_MsfRange range, SYMS_U32 off, SYMS_U32 align){
  SYMS_CvSymbolHelper sym = {0};
  syms_msf_read_struct_in_range(data, msf, range, off, &sym);
  SYMS_CvElement result = {0};
  if (sym.size > 0){
    SYMS_U32 end_unclamped = off + 2 + sym.size;
    SYMS_U32 end_clamped = SYMS_ClampTop(end_unclamped, range.size);
    SYMS_U32 next_off = SYMS_AlignPow2(end_unclamped, align);
    SYMS_U32 start = off + 4;
    result.range = syms_msf_sub_range(range, start, end_clamped - start);
    result.next_off = next_off;
    result.kind = sym.type;
  }
  return(result);
}

SYMS_API SYMS_U32
syms_cv_read_numeric(SYMS_String8 data, SYMS_MsfAccel *msf, SYMS_MsfRange range, SYMS_U32 off,
                     SYMS_CvNumeric *out){
  SYMS_U32 result = 0;
  SYMS_U16 leaf = 0;
  syms_memzero_struct(out);
  if (syms_msf_read_struct_in_range(data, msf, range, off, &leaf)){
    if (leaf < SYMS_CvLeaf_NUMERIC){
      out->kind = SYMS_TypeKind_UInt32;
      syms_memmove(out->data, &leaf, 2);
      result = sizeof(leaf);
    }
    else{
      SYMS_CvNumeric num = {SYMS_TypeKind_Null};
      SYMS_U32 size = 0;
      switch (leaf){
        case SYMS_CvLeaf_FLOAT16:   num.kind = SYMS_TypeKind_Float16;  size = 2;  break;
        case SYMS_CvLeaf_FLOAT32:   num.kind = SYMS_TypeKind_Float32;  size = 4;  break;
        case SYMS_CvLeaf_FLOAT48:   num.kind = SYMS_TypeKind_Float48;  size = 6;  break;
        case SYMS_CvLeaf_FLOAT64:   num.kind = SYMS_TypeKind_Float64;  size = 8;  break;
        case SYMS_CvLeaf_FLOAT80:   num.kind = SYMS_TypeKind_Float80;  size = 10; break;
        case SYMS_CvLeaf_FLOAT128:  num.kind = SYMS_TypeKind_Float128; size = 16; break;
        case SYMS_CvLeaf_CHAR:      num.kind = SYMS_TypeKind_Int8;     size = 1;  break;
        case SYMS_CvLeaf_SHORT:     num.kind = SYMS_TypeKind_Int16;    size = 2;  break;
        case SYMS_CvLeaf_USHORT:    num.kind = SYMS_TypeKind_UInt16;   size = 2;  break;
        case SYMS_CvLeaf_LONG:      num.kind = SYMS_TypeKind_Int32;    size = 4;  break;
        case SYMS_CvLeaf_ULONG:     num.kind = SYMS_TypeKind_UInt32;   size = 4;  break;
        case SYMS_CvLeaf_QUADWORD:  num.kind = SYMS_TypeKind_Int64;    size = 8; break;
        case SYMS_CvLeaf_UQUADWORD: num.kind = SYMS_TypeKind_UInt64;   size = 8; break;
        default:break;
      }
      SYMS_U32 number_off = off + sizeof(leaf);
      if (syms_msf_read_in_range(data, msf, range, number_off, size, num.data)){
        syms_memmove(out, &num, sizeof(num));
        result = number_off + size - off;
      }
    }
  }
  return(result);
}

SYMS_API SYMS_U32
syms_cv_u32_from_numeric(SYMS_CvNumeric num){
  SYMS_U32 result = 0;
  syms_memmove(&result, num.data, 4);
  return(result);
}

SYMS_API SYMS_CvStubRef*
syms_cv_alloc_ref(SYMS_Arena *arena, SYMS_CvStubRef **free_list){
  SYMS_CvStubRef *result = *free_list;
  if (result != 0){
    SYMS_StackPop(*free_list);
  }
  else{
    result = syms_push_array(arena, SYMS_CvStubRef, 1);
  }
  return(result);
}

SYMS_API SYMS_SecInfoArray
syms_cv_sec_info_array_from_bin(SYMS_Arena *arena, SYMS_String8 data, SYMS_MsfAccel *msf,
                                SYMS_MsfRange range){
  SYMS_ArenaTemp scratch = syms_get_scratch(&arena, 1);
  
  // size of section header / size of coff section
  SYMS_U64 count = range.size/sizeof(SYMS_CoffSectionHeader);
  
  // grab coff array
  SYMS_CoffSectionHeader *coff_secs = syms_push_array(scratch.arena, SYMS_CoffSectionHeader, count);
  syms_msf_read_in_range(data, msf, range, 0, count*sizeof(*coff_secs), coff_secs);
  
  // fill array
  SYMS_SecInfoArray result = {0};
  result.count = count;
  result.sec_info = syms_push_array(arena, SYMS_SecInfo, count);
  
  SYMS_SecInfo *sec_info = result.sec_info;
  SYMS_CoffSectionHeader *coff_sec = coff_secs;
  
  for (SYMS_U64 i = 0; i < result.count; i += 1, sec_info += 1, coff_sec += 1){
    // extract name
    SYMS_U8 *name = coff_sec->name;
    SYMS_U8 *name_ptr = name;
    SYMS_U8 *name_opl = name + 8;
    for (;name_ptr < name_opl && *name_ptr != 0; name_ptr += 1);
    
    // fill sec info
    sec_info->name = syms_push_string_copy(arena, syms_str8_range(name, name_ptr));
    sec_info->vrange.min = coff_sec->virt_off;
    sec_info->vrange.max = coff_sec->virt_off + coff_sec->virt_size;
    sec_info->frange.min = coff_sec->file_off;
    sec_info->frange.max = coff_sec->file_off + coff_sec->file_size;
  }
  
  syms_release_scratch(scratch);
  
  return(result);
}

SYMS_API void
syms_cv_c13_sub_sections_from_range(SYMS_Arena *arena, SYMS_String8 data, SYMS_MsfAccel *msf,
                                    SYMS_MsfRange range, SYMS_CvC13SubSectionList *list_out){
  SYMS_U32 off = 0;
  for (;;){
    // read header
    SYMS_CvSubSectionHeader header = {0};
    syms_msf_read_struct_in_range(data, msf, range, off, &header);
    
    // get sub section info
    SYMS_U32 sub_section_off = off + sizeof(header);
    SYMS_U32 sub_section_size = header.size;
    SYMS_U32 after_sub_section = sub_section_off + sub_section_size;
    
    // exit condition
    if (!syms_msf_bounds_check_in_range(range, after_sub_section)){
      break;
    }
    
    // emit sub section
    SYMS_CvC13SubSection *sub_section = syms_push_array_zero(arena, SYMS_CvC13SubSection, 1);
    SYMS_QueuePush(list_out->first, list_out->last, sub_section);
    list_out->count += 1;
    sub_section->kind = header.kind;
    sub_section->off = sub_section_off;
    sub_section->size = sub_section_size;
    
    // increment off
    SYMS_U64 unaligned_off = sub_section_off + sub_section_size;
    off = SYMS_AlignPow2(unaligned_off, 4);
  }
}


// cv line info

SYMS_API void
syms_cv_loose_push_file_id(SYMS_Arena *arena, SYMS_CvLineTableLoose *loose, SYMS_FileID id){
  // check if this id is new
  SYMS_B32 is_duplicate = syms_false;
  for (SYMS_CvFileNode *node = loose->first_file_node;
       node != 0;
       node = node->next){
    SYMS_U64 count = node->count;
    SYMS_FileID *id_ptr = node->file_ids;
    for (SYMS_U64 i = 0; i < count; i += 1, id_ptr += 1){
      if (*id_ptr == id){
        is_duplicate = syms_true;
        goto dblbreak;
      }
    }
  }
  dblbreak:;
  
  // insert new id
  if (!is_duplicate){
    SYMS_CvFileNode *last_node = loose->last_file_node;
    if (last_node == 0 || last_node->count == SYMS_ARRAY_SIZE(last_node->file_ids)){
      last_node = syms_push_array(arena, SYMS_CvFileNode, 1);
      SYMS_QueuePush(loose->first_file_node, loose->last_file_node, last_node);
      last_node->count = 0;
    }
    last_node->file_ids[last_node->count] = id;
    last_node->count += 1;
    loose->file_count += 1;
  }
}

SYMS_API SYMS_Line*
syms_cv_loose_push_sequence(SYMS_Arena *arena, SYMS_CvLineTableLoose *loose, SYMS_U64 line_count){
  SYMS_CvLineSequence *new_seq = syms_push_array(arena, SYMS_CvLineSequence, 1);
  SYMS_Line *result = syms_push_array(arena, SYMS_Line, line_count);
  
  new_seq->lines = result;
  new_seq->line_count = line_count;
  
  SYMS_QueuePush(loose->first_seq, loose->last_seq, new_seq);
  loose->seq_count += 1;
  loose->line_count += line_count;
  
  return(result);
}

SYMS_API SYMS_LineParseOut
syms_cv_line_parse_from_loose(SYMS_Arena *arena, SYMS_CvLineTableLoose *loose){
  // file table from list
  SYMS_FileID *id_array = syms_push_array(arena, SYMS_FileID, loose->file_count);
  {
    SYMS_FileID *file_id_ptr = id_array;
    for (SYMS_CvFileNode *node = loose->first_file_node;
         node != 0;
         node = node->next){
      syms_memmove(file_id_ptr, node->file_ids, sizeof(*node->file_ids)*node->count);
      file_id_ptr += node->count;
    }
  }
  
  // line table from lists
  SYMS_U64 *sequence_index_array = syms_push_array(arena, SYMS_U64, loose->seq_count + 1);
  SYMS_Line *line_array = syms_push_array(arena, SYMS_Line, loose->line_count);
  {
    SYMS_U64 *index_ptr = sequence_index_array;
    SYMS_Line *line_ptr = line_array;
    SYMS_U64 index = 0;
    for (SYMS_CvLineSequence *seq = loose->first_seq;
         seq != 0;
         seq = seq->next){
      SYMS_U64 seq_line_count = seq->line_count;
      // fill indices
      *index_ptr = index;
      index_ptr += 1;
      // fill lines
      syms_memmove(line_ptr, seq->lines, sizeof(*seq->lines)*seq_line_count);
      line_ptr += seq_line_count;
      index += seq_line_count;
    }
    *index_ptr = index;
  }
  
  // fill result
  SYMS_LineParseOut result = {0};
  result.file_id_array.count = loose->file_count;
  result.file_id_array.ids = id_array;
  result.line_table.sequence_count = loose->seq_count;
  result.line_table.sequence_index_array = sequence_index_array;
  result.line_table.line_count = loose->line_count;
  result.line_table.line_array = line_array;
  
  return(result);
}

SYMS_API void
syms_cv_loose_lines_from_c13(SYMS_Arena *arena, SYMS_String8 data,
                             SYMS_MsfAccel *msf, SYMS_MsfRange c13_range,
                             SYMS_CvC13SubSection *sub_sections,
                             SYMS_U64 *section_voffs, SYMS_U64 section_count,
                             SYMS_CvLineTableLoose *loose){
  // find the checksums section
  SYMS_CvC13SubSection*checksums = 0;
  for (SYMS_CvC13SubSection *node = sub_sections;
       node != 0;
       node = node->next){
    if (node->kind == SYMS_CvSubSectionKind_FILECHKSMS){
      checksums = node;
      break;
    }
  }
  
  // extract checksums range
  SYMS_MsfRange checksums_range = {0};
  if (checksums != 0){
    checksums_range = syms_msf_sub_range(c13_range, checksums->off, checksums->size);
  }
  
  // iterate lines sections
  for (SYMS_CvC13SubSection *sub_sec_node = sub_sections;
       sub_sec_node != 0;
       sub_sec_node = sub_sec_node->next){
    if (sub_sec_node->kind == SYMS_CvSubSectionKind_LINES){
      SYMS_MsfRange subsec_range = syms_msf_sub_range(c13_range, sub_sec_node->off, sub_sec_node->size);
      
      // read header
      SYMS_CvSubSecLinesHeader subsec_lines_header = {0};
      syms_msf_read_struct_in_range(data, msf, subsec_range, 0, &subsec_lines_header);
      
      // check for columns
      SYMS_B32 has_columns = ((subsec_lines_header.flags&SYMS_CvSubSecLinesFlag_HasColumns) != 0);
      
      // read file
      SYMS_U32 file_off = sizeof(subsec_lines_header);
      SYMS_CvFile file = {0};
      syms_msf_read_struct_in_range(data, msf, subsec_range, file_off, &file);
      
      SYMS_U32 line_array_off = file_off + sizeof(file);
      SYMS_U32 column_array_off = line_array_off + file.num_lines*sizeof(SYMS_CvLine);
      
      // track down this file info
      SYMS_CvChecksum chksum = {0};
      syms_msf_read_struct_in_range(data, msf, checksums_range, file.file_off, &chksum);
      
      // setup the section's file id
      SYMS_FileID file_id = SYMS_ID_u32_u32(SYMS_CvFileIDKind_StrTblOff, chksum.name_off);
      syms_cv_loose_push_file_id(arena, loose, file_id);
      
      // get the section's virtual offset
      SYMS_U64 section_voff = syms_1based_checked_lookup_u64(section_voffs, section_count,
                                                             subsec_lines_header.sec);
      SYMS_U64 seq_base_voff = section_voff + subsec_lines_header.sec_off;
      
      // start the sequence
      SYMS_Line *lines_out = syms_cv_loose_push_sequence(arena, loose, file.num_lines + 1);
      
      // parse info
      SYMS_Line *line_out_ptr = lines_out;
      SYMS_U32 line_cursor_off = line_array_off;
      SYMS_U32 column_cursor_off = column_array_off;
      for (SYMS_U32 i = 0; i < file.num_lines; i += 1, line_out_ptr += 1){
        
        // read line
        SYMS_CvLine line = {0};
        syms_msf_read_struct_in_range(data, msf, subsec_range, line_cursor_off, &line);
        
        // read column
        SYMS_CvColumn col = {0};
        if (has_columns){
          syms_msf_read_struct_in_range(data, msf, subsec_range, column_cursor_off, &col);
        }
        
        // calculate line data
        SYMS_U64 line_off = seq_base_voff + line.off;
        SYMS_U32 line_number = (line.flags>>SYMS_CvLineFlag_LINE_NUMBER_SHIFT)&SYMS_CvLineFlag_LINE_NUMBER_MASK;
        //SYMS_U32 delta_to_end = (line.flags>>SYMS_CvLineFlag_DELTA_TO_END_SHIFT)&SYMS_CvLineFlag_DELTA_TO_END_MASK;
        //SYMS_B32 statement = ((line.flags & SYMS_CvLineFlag_STATEMENT) != 0);
        
        // emit
        line_out_ptr->src_coord.file_id = file_id;
        line_out_ptr->src_coord.line = line_number;
        line_out_ptr->src_coord.col = col.start;
        line_out_ptr->voff = line_off;
        
        // increment 
        line_cursor_off += sizeof(line);
        column_cursor_off += sizeof(col);
      }
      
      // explicitly add an ender
      {
        line_out_ptr->src_coord.file_id = file_id;
        line_out_ptr->src_coord.line = 0;
        line_out_ptr->src_coord.col = 0;
        line_out_ptr->voff = seq_base_voff + subsec_lines_header.len;
      }
    }
  }
}

SYMS_API void
syms_cv_loose_lines_from_c11(SYMS_Arena *arena, SYMS_String8 data,
                             SYMS_MsfAccel *msf, SYMS_MsfRange c11_range,
                             SYMS_U64 *section_voffs, SYMS_U64 section_count,
                             SYMS_CvLineTableLoose *loose){
  // c11 layout:
  //  file_count: U16;
  //  range_count: U16;
  //  file_array: U32[file_count];
  //  range_array: U64[range_count];
  //  range_array2: U16[range_count];
  //  @align(4)
  //  ... more data ...
  struct C11Header{
    SYMS_U16 file_count;
    SYMS_U16 range_count;
  };
  struct C11Header header = {0};
  syms_msf_read_struct_in_range(data, msf, c11_range, 0, &header);
  SYMS_U32 unk_file_array_off   = sizeof(header);
  SYMS_U32 unk_range_array_off  = unk_file_array_off + header.file_count*4;
  SYMS_U32 unk_range_array2_off = unk_range_array_off + header.range_count*8;
  SYMS_U32 after_unk_range_array2_off = unk_range_array2_off + header.range_count*2;
  SYMS_U32 after_header_off = (after_unk_range_array2_off + 3)&~3;
  
  // iterate file sections
  SYMS_U32 file_section_cursor = after_header_off;
  for (;file_section_cursor < c11_range.size;){
    // file section layout:
    //  sec_count: U16;
    //  padding: U16;
    //  sec_array: U32[sec_count];
    //  range_array: U64[sec_count];
    //  path: length-prefixed with 1 byte;
    //  @align(4)
    //  ... more data ...
    SYMS_U32 file_section_off = file_section_cursor;
    SYMS_U16 sec_count = 0;
    syms_msf_read_struct_in_range(data, msf, c11_range, file_section_off, &sec_count);
    SYMS_U32 unk_sec_array_off = file_section_off + 4;
    SYMS_U32 range_array_off   = unk_sec_array_off + 4*sec_count;
    SYMS_U32 parse_path_off    = range_array_off + 8*sec_count;
    
    SYMS_U32 path_size = 0;
    syms_msf_read_in_range(data, msf, c11_range, parse_path_off, 1, &path_size);
    SYMS_U32 path_off = parse_path_off + 1; (void)path_off;
    SYMS_U32 after_path_off = parse_path_off + 1 + path_size;
    
    SYMS_U32 after_file_section_off = (after_path_off + 3)&~3;
    
    // setup section file id
    SYMS_FileID file_id = SYMS_ID_u32_u32(SYMS_CvFileIDKind_C11Off, parse_path_off);
    syms_cv_loose_push_file_id(arena, loose, file_id);
    
    // iterate sections
    SYMS_U32 section_cursor = after_file_section_off;
    for (SYMS_U16 i = 0; i < sec_count; i += 1){
      // section layout:
      //  sec: U16;
      //  line_count: U16;
      //  offs_array: U32[line_count];
      //  nums_array: U16[line_count];
      //  @align(4)
      //  ... more data ...
      SYMS_U32 section_off = section_cursor;
      struct C11Sec{
        SYMS_U16 sec;
        SYMS_U16 line_count;
      };
      struct C11Sec c11_sec = {0};
      syms_msf_read_struct_in_range(data, msf, c11_range, section_off, &c11_sec);
      SYMS_U32 offs_array_off = section_off + sizeof(c11_sec);
      SYMS_U32 nums_array_off = offs_array_off + c11_sec.line_count*4;
      SYMS_U32 after_nums_array_off = nums_array_off + c11_sec.line_count*2;
      SYMS_U32 after_section_off = (after_nums_array_off + 3)&~3;
      
      // get section range
      SYMS_U32Range sec_range = {0};
      syms_msf_read_struct_in_range(data, msf, c11_range, range_array_off + 8*i, &sec_range);
      
      // get the section's virtual offset
      SYMS_U32 seq_base_voff = syms_1based_checked_lookup_u64(section_voffs, section_count, c11_sec.sec);
      
      // start the sequence
      SYMS_Line *lines_out = syms_cv_loose_push_sequence(arena, loose, c11_sec.line_count + 1);
      
      // iterate line info
      SYMS_Line *line_out_ptr = lines_out;
      for (SYMS_U16 j = 0; j < c11_sec.line_count; j += 1, line_out_ptr += 1){
        SYMS_U32 line_off = 0;
        SYMS_U16 line_num = 0;
        syms_msf_read_struct_in_range(data, msf, c11_range, offs_array_off + 4*j, &line_off);
        syms_msf_read_struct_in_range(data, msf, c11_range, nums_array_off + 2*j, &line_num);
        
        line_out_ptr->src_coord.file_id = file_id;
        line_out_ptr->src_coord.line = line_num;
        line_out_ptr->src_coord.col = 0;
        line_out_ptr->voff = seq_base_voff + line_off;
      }
      
      // explicitly add an ender
      {
        line_out_ptr->src_coord.file_id = file_id;
        line_out_ptr->src_coord.line = 0;
        line_out_ptr->src_coord.col = 0;
        line_out_ptr->voff = seq_base_voff + sec_range.max;
      }
      
      // increment
      section_cursor = after_section_off;
    }
    
    // increment
    file_section_cursor = section_cursor;
  }
}


// cv parsers

SYMS_API SYMS_CvUnitAccel*
syms_cv_leaf_unit_from_range(SYMS_Arena *arena, SYMS_String8 data, SYMS_MsfAccel *msf,
                             SYMS_MsfStreamNumber sn, SYMS_U64Range raw_range,
                             SYMS_CvLeafConsParams *params){
  // setup msf
  SYMS_MsfRange range = {0};
  range.sn = sn;
  range.off = raw_range.min;
  range.size = raw_range.max - raw_range.min;
  
  // index stub list
  SYMS_CvStub *first_index_stub = 0;
  SYMS_CvStub *last_index_stub = 0;
  
  // parse loop
  SYMS_U64 ti = params->first_ti;
  SYMS_U64 off_stub_count = 0;
  SYMS_U32 align = params->align;
  SYMS_U32 cursor = 0;
  for (;;){
    // read element
    SYMS_CvElement element = syms_cv_element(data, msf, range, cursor, align);
    
    // exit condition
    if (element.next_off == 0){
      break;
    }
    
    // setup stub
    SYMS_CvStub *stub = syms_push_array_zero(arena, SYMS_CvStub, 1);
    stub->off = element.range.off - 4;
    stub->index = ti;
    
    // parse
    switch (element.kind){
      default:break;
      
      case SYMS_CvLeaf_ARRAY:
      {
        SYMS_U32 num_off = sizeof(SYMS_CvLeafArray);
        SYMS_CvNumeric num = {SYMS_TypeKind_Null};
        SYMS_U32 num_size = syms_cv_read_numeric(data, msf, element.range, num_off, &num);
        (void)num_size;
        stub->num = syms_cv_u32_from_numeric(num);
      }break;
      
      // udt
      case SYMS_CvLeaf_CLASS:
      case SYMS_CvLeaf_STRUCTURE:
      case SYMS_CvLeaf_INTERFACE:
      {
        SYMS_U32 num_off = sizeof(SYMS_CvLeafStruct);
        SYMS_CvNumeric num = {SYMS_TypeKind_Null};
        SYMS_U32 num_size = syms_cv_read_numeric(data, msf, element.range, num_off, &num);
        SYMS_U32 name_off = num_off + num_size;
        SYMS_String8 name = syms_msf_read_zstring_in_range(arena, data, msf, element.range, name_off);
        
        stub->num = syms_cv_u32_from_numeric(num);
        stub->name = name;
      }break;
      
      case SYMS_CvLeaf_UNION:
      {
        SYMS_U32 num_off = sizeof(SYMS_CvLeafUnion);
        SYMS_CvNumeric num = {SYMS_TypeKind_Null};
        SYMS_U32 num_size = syms_cv_read_numeric(data, msf, element.range, num_off, &num);
        SYMS_U32 name_off = num_off + num_size;
        SYMS_String8 name = syms_msf_read_zstring_in_range(arena, data, msf, element.range, name_off);
        
        stub->num = syms_cv_u32_from_numeric(num);
        stub->name = name;
      }break;
      
      case SYMS_CvLeaf_ENUM:
      {
        SYMS_U32 name_off = sizeof(SYMS_CvLeafEnum);
        SYMS_String8 name = syms_msf_read_zstring_in_range(arena, data, msf, element.range, name_off);
        stub->name = name;
      }break;
      
      case SYMS_CvLeaf_CLASSPTR:
      case SYMS_CvLeaf_CLASSPTR2:
      {
        SYMS_U32 num_off = sizeof(SYMS_CvLeafClassPtr);
        SYMS_CvNumeric num = {SYMS_TypeKind_Null};
        SYMS_U32 num_size = syms_cv_read_numeric(data, msf, element.range, num_off, &num);
        SYMS_U32 name_off = num_off + num_size;
        SYMS_String8 name = syms_msf_read_zstring_in_range(arena, data, msf, element.range, name_off);
        
        stub->num = syms_cv_u32_from_numeric(num);
        stub->name = name;
      }break;
      
      case SYMS_CvLeaf_ALIAS:
      {
        SYMS_U32 name_off = sizeof(SYMS_CvLeafAlias);
        SYMS_String8 name = syms_msf_read_zstring_in_range(arena, data, msf, element.range, name_off);
        
        stub->name = name;
      }break;
      
      // field list
      case SYMS_CvLeaf_FIELDLIST:
      {
        // parse loop
        SYMS_CvStub *parent = stub;
        SYMS_U32 fl_cursor = 0;
        for (;fl_cursor < element.range.size;){
          // read kind
          SYMS_CvLeaf lf_kind = 0;
          syms_msf_read_struct_in_range(data, msf, element.range, fl_cursor, &lf_kind);
          
          // insert new stub under parent
          SYMS_CvStub *fl_stub = syms_push_array_zero(arena, SYMS_CvStub, 1);
          SYMS_QueuePush_N(parent->first, parent->last, fl_stub, sibling_next);
          fl_stub->parent = parent;
          fl_stub->off = element.range.off + fl_cursor;
          
          // read internal leaf
          SYMS_U32 lf_data_off = fl_cursor + sizeof(lf_kind);
          SYMS_U32 lf_end_off = lf_data_off + 2;
          
          switch (lf_kind){
            default:break;
            
            case SYMS_CvLeaf_MEMBER:
            {
              SYMS_U32 num_off = lf_data_off + sizeof(SYMS_CvLeafMember);
              SYMS_CvNumeric num = {SYMS_TypeKind_Null};
              SYMS_U32 num_size = syms_cv_read_numeric(data, msf, element.range, num_off, &num);
              SYMS_U32 name_off = num_off + num_size;
              SYMS_String8 name = syms_msf_read_zstring_in_range(arena, data, msf, element.range, name_off);
              
              fl_stub->num = syms_cv_u32_from_numeric(num);
              fl_stub->name = name;
              lf_end_off = name_off + name.size + 1;
            }break;
            
            case SYMS_CvLeaf_STMEMBER:
            {
              SYMS_U32 name_off = lf_data_off + sizeof(SYMS_CvLeafStMember);
              SYMS_String8 name = syms_msf_read_zstring_in_range(arena, data, msf, element.range, name_off);
              
              fl_stub->name = name;
              lf_end_off = name_off + name.size + 1;
            }break;
            
            case SYMS_CvLeaf_METHOD:
            {
              SYMS_U32 name_off = lf_data_off + sizeof(SYMS_CvLeafMethod);
              SYMS_String8 name = syms_msf_read_zstring_in_range(arena, data, msf, element.range, name_off);
              
              fl_stub->name = name;
              lf_end_off = name_off + name.size + 1;
            }break;
            
            case SYMS_CvLeaf_ONEMETHOD:
            {
              SYMS_CvLeafOneMethod onemethod = {0};
              syms_msf_read_struct_in_range(data, msf, element.range, lf_data_off, &onemethod);
              SYMS_CvMethodProp prop = SYMS_CvFieldAttribs_Extract_MPROP(onemethod.attribs);
              
              SYMS_U32 name_off = lf_data_off + sizeof(onemethod);
              SYMS_U32 virtoff = 0;
              if (prop == SYMS_CvMethodProp_PUREINTRO || prop == SYMS_CvMethodProp_INTRO){
                syms_msf_read_struct_in_range(data, msf, range, name_off, &virtoff);
                name_off += sizeof(virtoff);
              }
              SYMS_String8 name = syms_msf_read_zstring_in_range(arena, data, msf, element.range, name_off);
              
              fl_stub->name = name;
              fl_stub->num = virtoff;
              lf_end_off = name_off + name.size + 1;
            }break;
            
            case SYMS_CvLeaf_ENUMERATE:
            {
              SYMS_U32 num_off = lf_data_off + sizeof(SYMS_CvLeafEnumerate);
              SYMS_CvNumeric num = {SYMS_TypeKind_Null};
              SYMS_U32 num_size = syms_cv_read_numeric(data, msf, element.range, num_off, &num);
              SYMS_U32 name_off = num_off + num_size;
              SYMS_String8 name = syms_msf_read_zstring_in_range(arena, data, msf, element.range, name_off);
              
              fl_stub->num = syms_cv_u32_from_numeric(num);
              fl_stub->name = name;
              lf_end_off = name_off + name.size + 1;
            }break;
            
            case SYMS_CvLeaf_NESTTYPE:
            case SYMS_CvLeaf_NESTTYPEEX:
            {
              SYMS_U32 name_off = lf_data_off + sizeof(SYMS_CvLeafNestTypeEx);
              SYMS_String8 name = syms_msf_read_zstring_in_range(arena, data, msf, element.range, name_off);
              
              fl_stub->name = name;
              lf_end_off = name_off + name.size + 1;
            }break;
            
            case SYMS_CvLeaf_BCLASS:
            {
              SYMS_U32 num_off = lf_data_off + sizeof(SYMS_CvLeafBClass);
              SYMS_CvNumeric num = {SYMS_TypeKind_Null};
              SYMS_U32 num_size = syms_cv_read_numeric(data, msf, element.range, num_off, &num);
              
              fl_stub->num = syms_cv_u32_from_numeric(num);
              lf_end_off = num_off + num_size;
            }break;
            
            case SYMS_CvLeaf_VBCLASS:
            case SYMS_CvLeaf_IVBCLASS:
            {
              SYMS_U32 num_off = lf_data_off + sizeof(SYMS_CvLeafVBClass);
              SYMS_CvNumeric num = {SYMS_TypeKind_Null};
              SYMS_U32 num_size = syms_cv_read_numeric(data, msf, element.range, num_off, &num);
              
              SYMS_U32 num2_off = num_off + num_size;
              SYMS_CvNumeric num2 = {SYMS_TypeKind_Null};
              SYMS_U32 num2_size = syms_cv_read_numeric(data, msf, element.range, num2_off, &num2);
              
              fl_stub->num = syms_cv_u32_from_numeric(num);
              fl_stub->num2 = syms_cv_u32_from_numeric(num2);
              lf_end_off = num2_off + num2_size;
            }break;
            
            case SYMS_CvLeaf_VFUNCTAB:
            {
              lf_end_off = lf_data_off + sizeof(SYMS_CvLeafVFuncTab);
            }break;
            
            case SYMS_CvLeaf_VFUNCOFF:
            {
              lf_end_off = lf_data_off + sizeof(SYMS_CvLeafVFuncOff);
            }break;
          }
          
          // increment
          SYMS_U32 next_lf_off = (lf_end_off + 3)&~3;
          fl_cursor = next_lf_off;
          off_stub_count += 1;
        }
        
      }break;
      
      // method list
      case SYMS_CvLeaf_METHODLIST:
      {
        // parse loop
        SYMS_CvStub *parent = stub;
        SYMS_U32 ml_cursor = 0;
        
        for (;ml_cursor < element.range.size;){
          // read method
          SYMS_CvMethod methodrec = {0};
          syms_msf_read_struct_in_range(data, msf, element.range, ml_cursor, &methodrec);
          SYMS_CvMethodProp mprop = SYMS_CvFieldAttribs_Extract_MPROP(methodrec.attribs);
          
          // get virtual offset
          SYMS_U32 virtual_offset = 0;
          SYMS_U32 next_off = ml_cursor + sizeof(methodrec);
          switch (mprop){
            default:break;
            case SYMS_CvMethodProp_INTRO:
            case SYMS_CvMethodProp_PUREINTRO:
            {
              syms_msf_read_struct_in_range(data, msf, element.range, next_off, &virtual_offset);
              next_off += sizeof(SYMS_U32);
            }break;
          }
          
          // insert new stub under parent
          SYMS_CvStub *ml_stub = syms_push_array(arena, SYMS_CvStub, 1);
          syms_memzero_struct(ml_stub);
          SYMS_QueuePush_N(parent->first, parent->last, ml_stub, sibling_next);
          ml_stub->parent = parent;
          ml_stub->off = element.range.off + ml_cursor;
          ml_stub->num = virtual_offset;
          
          // increment
          ml_cursor = next_off;
        }
      }break;
    }
    
    // save stub onto list
    SYMS_QueuePush_N(first_index_stub, last_index_stub, stub, sibling_next);
    ti += 1;
    
    // increment
    cursor = element.next_off;
  }
  
  // setup ti stubs array
  SYMS_U64 index_stub_count = (ti - params->first_ti);
  SYMS_CvStub **ti_stubs = syms_push_array(arena, SYMS_CvStub*, index_stub_count);
  {
    SYMS_CvStub **ti_stub_ptr = ti_stubs;
    for (SYMS_CvStub *stub = first_index_stub;
         stub != 0;
         stub = stub->sibling_next, ti_stub_ptr += 1){
      *ti_stub_ptr = stub;
    }
  }
  
  // build bucket table
  SYMS_U64 bucket_count = off_stub_count*5/4;
  SYMS_CvStub **buckets = syms_push_array_zero(arena, SYMS_CvStub*, bucket_count);
  if (bucket_count > 0){
    SYMS_CvStub **stub_ptr = ti_stubs;
    for (SYMS_U64 i = 0; i < index_stub_count; i += 1, stub_ptr += 1){
      SYMS_CvStub *stub = *stub_ptr;
      for (SYMS_CvStub *internal_stub = stub->first;
           internal_stub != 0;
           internal_stub = internal_stub->sibling_next){
        SYMS_U64 hash = syms_hash_u64(internal_stub->off);
        SYMS_U32 bucket_index = hash%bucket_count;
        SYMS_StackPush_N(buckets[bucket_index], internal_stub, bucket_next);
      }
    }
  }
  
  // filter top stubs
  SYMS_MsfRange whole_range = syms_msf_range_from_sn(msf, range.sn);
  SYMS_CvStub **udt_stubs = syms_push_array(arena, SYMS_CvStub*, index_stub_count);
  SYMS_U64 udt_count = 0;
  {
    SYMS_CvStub **udt_stub_ptr = udt_stubs;
    SYMS_CvStub **stub_ptr = ti_stubs;
    SYMS_CvStub **stub_opl = ti_stubs + index_stub_count;
    for (; stub_ptr < stub_opl; stub_ptr += 1){
      SYMS_CvStub *stub = *stub_ptr;
      SYMS_U16 type = {0};
      syms_msf_read_struct_in_range(data, msf, whole_range, stub->off + 2, &type);
      switch (type){
        // type kinds
        case SYMS_CvLeaf_CLASS:
        case SYMS_CvLeaf_STRUCTURE:
        case SYMS_CvLeaf_INTERFACE:
        case SYMS_CvLeaf_UNION:
        case SYMS_CvLeaf_ENUM:
        case SYMS_CvLeaf_CLASSPTR:
        case SYMS_CvLeaf_CLASSPTR2:
        case SYMS_CvLeaf_ALIAS:
        {
          *udt_stub_ptr = stub;
          udt_stub_ptr += 1;
        }break;
      }
    }
    
    udt_count = (SYMS_U64)(udt_stub_ptr - udt_stubs);
    syms_arena_put_back(arena, sizeof(SYMS_CvStub*)*(index_stub_count - udt_count));
  }
  
  // fill result
  SYMS_CvUnitAccel *result = syms_push_array(arena, SYMS_CvUnitAccel, 1);
  result->format = params->format;
  result->leaf_set = syms_true;
  result->sn = range.sn;
  result->top_stubs = udt_stubs;
  result->top_count = udt_count;
  result->top_min_index = params->first_ti;
  result->buckets = buckets;
  result->bucket_count = bucket_count;
  result->all_count = index_stub_count + off_stub_count;
  result->ti_indirect_stubs = ti_stubs;
  result->ti_count = index_stub_count;
  result->uid = params->uid;
  
  return(result);
}

SYMS_API SYMS_CvUnitAccel*
syms_cv_sym_unit_from_ranges(SYMS_Arena *arena, SYMS_String8 data, SYMS_MsfAccel *msf, 
                             SYMS_MsfStreamNumber sn, SYMS_U64RangeArray symbol_ranges,
                             SYMS_CvSymConsParams *params){
  // get scratch
  SYMS_ArenaTemp scratch = syms_get_scratch(&arena, 1);
  
  // root
  SYMS_CvStub root = {0};
  SYMS_U64 top_count = 0;
  
  // all list
  SYMS_CvStub *first = 0;
  SYMS_CvStub *last = 0;
  SYMS_U64 all_count = 0;
  
  // proc & var
  SYMS_CvStubRef *proc_first = 0;
  SYMS_CvStubRef *proc_last = 0;
  SYMS_U64 proc_count = 0;
  
  SYMS_CvStubRef *var_first = 0;
  SYMS_CvStubRef *var_last = 0;
  SYMS_U64 var_count = 0;
  
  // thread vars
  SYMS_CvStubRef *tls_var_first = 0;
  SYMS_CvStubRef *tls_var_last = 0;
  SYMS_U64 tls_var_count = 0;
  
  // thunk
  SYMS_CvStubRef *thunk_first = 0;
  SYMS_CvStubRef *thunk_last = 0;
  SYMS_U64 thunk_count = 0;
  
  // pub
  SYMS_CvStubRef *pub_first = 0;
  SYMS_CvStubRef *pub_last = 0;
  SYMS_U64 pub_count = 0;
  
  // free stub ref nodes
  SYMS_CvStubRef *stack_free = 0;
  
  // extract alignment
  SYMS_U32 align = params->align;
  
  // outer parse loop
  SYMS_U64Range *range_ptr = symbol_ranges.ranges;
  SYMS_U64Range *range_opl = symbol_ranges.ranges + symbol_ranges.count;
  for (; range_ptr < range_opl; range_ptr += 1){
    SYMS_MsfRange range = syms_msf_make_range(sn, range_ptr->min, range_ptr->max - range_ptr->min);
    
    // per-range state
    SYMS_CvStub *defrange_collector_stub = &root;
    SYMS_CvStubRef *stack = 0;
    
    // range parse loop
    SYMS_U32 cursor = 0;
    for (;;){
      // read element
      SYMS_CvElement element = syms_cv_element(data, msf, range, cursor, align);
      
      // exit condition
      if (element.next_off == 0){
        break;
      }
      
      // init stub
      SYMS_U32 symbol_off = element.range.off - 4;
      SYMS_CvStub *stub = syms_push_array_zero(arena, SYMS_CvStub, 1);
      SYMS_QueuePush_N(first, last, stub, bucket_next);
      all_count += 1;
      stub->off = symbol_off;
      
      // default parent
      SYMS_CvStub *parent_for_this_stub = &root;
      if (stack != 0){
        parent_for_this_stub = stack->stub;
      }
      
      // defrange collecting check
      SYMS_B32 preserve_defrange_collection = syms_false;
      
      // parse
      switch (element.kind){
        default:break;
        
        case SYMS_CvSymKind_LDATA32:
        case SYMS_CvSymKind_GDATA32:
        {
          SYMS_U32 name_off = sizeof(SYMS_CvData32);
          SYMS_String8 name = syms_msf_read_zstring_in_range(arena, data, msf, element.range, name_off);
          
          stub->name = name;
          
          // push onto var list
          SYMS_CvStubRef *ref = syms_cv_alloc_ref(scratch.arena, &stack_free);
          ref->stub = stub;
          SYMS_QueuePush(var_first, var_last, ref);
          var_count += 1;
        }break;
        
        case SYMS_CvSymKind_LTHREAD32:
        case SYMS_CvSymKind_GTHREAD32:
        {
          SYMS_U32 name_off = sizeof(SYMS_CvThread32);
          SYMS_String8 name = syms_msf_read_zstring_in_range(arena, data, msf, element.range, name_off);
          
          stub->name = name;
          
          // push onto var list
          SYMS_CvStubRef *ref = syms_cv_alloc_ref(scratch.arena, &stack_free);
          ref->stub = stub;
          SYMS_QueuePush(tls_var_first, tls_var_last, ref);
          tls_var_count += 1;
        }break;
        
        // general local variable
        case SYMS_CvSymKind_LOCAL:
        {
          SYMS_U32 name_off = sizeof(SYMS_CvLocal);
          SYMS_String8 name = syms_msf_read_zstring_in_range(arena, data, msf, element.range, name_off);
          
          stub->name = name;
          
          preserve_defrange_collection = syms_true;
          defrange_collector_stub = stub;
        }break;
        
        case SYMS_CvSymKind_FILESTATIC:
        {
          SYMS_U32 name_off = sizeof(SYMS_CvFileStatic);
          SYMS_String8 name = syms_msf_read_zstring_in_range(arena, data, msf, element.range, name_off);
          
          stub->name = name;
          
          preserve_defrange_collection = syms_true;
          defrange_collector_stub = stub;
        }break;
        
        case SYMS_CvSymKind_DEFRANGE_2005:
        case SYMS_CvSymKind_DEFRANGE2_2005:
        case SYMS_CvSymKind_DEFRANGE:
        case SYMS_CvSymKind_DEFRANGE_SUBFIELD:
        case SYMS_CvSymKind_DEFRANGE_REGISTER:
        case SYMS_CvSymKind_DEFRANGE_FRAMEPOINTER_REL:
        case SYMS_CvSymKind_DEFRANGE_SUBFIELD_REGISTER:
        case SYMS_CvSymKind_DEFRANGE_FRAMEPOINTER_REL_FULL_SCOPE:
        case SYMS_CvSymKind_DEFRANGE_REGISTER_REL:
        {
          preserve_defrange_collection = syms_true;
          parent_for_this_stub = defrange_collector_stub;
        }break;
        
        case SYMS_CvSymKind_REGREL32:
        {
          SYMS_U32 name_off = sizeof(SYMS_CvRegrel32);
          SYMS_String8 name = syms_msf_read_zstring_in_range(arena, data, msf, element.range, name_off);
          
          stub->name = name;
        }break;
        
        case SYMS_CvSymKind_CONSTANT:
        {
          SYMS_CvNumeric numeric_val = {SYMS_TypeKind_Null};
          SYMS_String8 name = {0};
          SYMS_U32 numeric_off = sizeof(SYMS_CvConstant);
          SYMS_U32 numeric_size = syms_cv_read_numeric(data, msf, element.range, numeric_off, &numeric_val);
          SYMS_U32 name_off = numeric_off + numeric_size;
          name = syms_msf_read_zstring_in_range(arena, data, msf, element.range, name_off);
          
          stub->num = syms_cv_u32_from_numeric(numeric_val);
          stub->name = name;
        }break;
        
        case SYMS_CvSymKind_PROCREF:
        case SYMS_CvSymKind_LPROCREF:
        case SYMS_CvSymKind_DATAREF:
        {
          SYMS_U32 name_off = sizeof(SYMS_CvRef2);
          SYMS_String8 name = syms_msf_read_zstring_in_range(arena, data, msf, element.range, name_off);
          
          stub->name = name;
        }break;
        
        case SYMS_CvSymKind_LPROC32_ID:
        case SYMS_CvSymKind_GPROC32_ID:
        case SYMS_CvSymKind_LPROC32:
        case SYMS_CvSymKind_GPROC32:
        {
          // TODO(allen): create a *PROC32 -> FRAMEPROC accelerator
          
          SYMS_U32 name_off = sizeof(SYMS_CvProc32);
          SYMS_String8 name = syms_msf_read_zstring_in_range(arena, data, msf, element.range, name_off);
          
          stub->name = name;
          
          // push stub stack
          SYMS_CvStubRef *stack_node = syms_cv_alloc_ref(scratch.arena, &stack_free);
          SYMS_StackPush(stack, stack_node);
          stack_node->stub = stub;
          
          // push onto proc list
          SYMS_CvStubRef *ref = syms_cv_alloc_ref(scratch.arena, &stack_free);
          ref->stub = stub;
          SYMS_QueuePush(proc_first, proc_last, ref);
          proc_count += 1;
        }break;
        
        case SYMS_CvSymKind_BLOCK32:
        {
          SYMS_U32 name_off = sizeof(SYMS_CvBlock32);
          SYMS_String8 name = syms_msf_read_zstring_in_range(arena, data, msf, element.range, name_off);
          
          stub->name = name;
          
          // push stub stack
          SYMS_CvStubRef *stack_node = syms_cv_alloc_ref(scratch.arena, &stack_free);
          SYMS_StackPush(stack, stack_node);
          stack_node->stub = stub;
        }break;
        
        case SYMS_CvSymKind_END:
        {
          if (stack != 0){
            SYMS_CvStubRef *bucket = stack;
            SYMS_StackPop(stack);
            SYMS_StackPush(stack_free, bucket);
          }
          parent_for_this_stub = &root;
        }break;
        
        case SYMS_CvSymKind_PUB32:
        {
          SYMS_U32 name_off = sizeof(SYMS_CvPubsym32);
          SYMS_String8 name = syms_msf_read_zstring_in_range(arena, data, msf, element.range, name_off);
          stub->name = name;
          
          // push onto stripped symbol list
          SYMS_CvStubRef *ref = syms_cv_alloc_ref(scratch.arena, &stack_free);
          ref->stub = stub;
          SYMS_QueuePush(pub_first, pub_last, ref);
          pub_count += 1;
        }break;
        
        case SYMS_CvSymKind_THUNK32:
        {
          SYMS_U32 name_off = sizeof(SYMS_CvThunk32);
          SYMS_String8 name = syms_msf_read_zstring_in_range(scratch.arena, data, msf, element.range, name_off);
          stub->name = name;
          
          // push onto thunk list
          SYMS_CvStubRef *ref = syms_cv_alloc_ref(scratch.arena, &stack_free);
          ref->stub = stub;
          SYMS_QueuePush(thunk_first, thunk_last, ref);
          thunk_count += 1;
        }break;
      }
      
      // clear defrange collector
      if (!preserve_defrange_collection){
        defrange_collector_stub = &root;
      }
      
      // insert into tree
      SYMS_ASSERT(parent_for_this_stub != 0);
      SYMS_QueuePush_N(parent_for_this_stub->first, parent_for_this_stub->last, stub, sibling_next);
      if (parent_for_this_stub != &root){
        stub->parent = parent_for_this_stub;
      }
      else{
        top_count += 1;
      }
      
      // increment
      cursor = element.next_off;
    }
  }
  
  // build top stubs pointer table
  SYMS_CvStub **top_stubs = syms_push_array(arena, SYMS_CvStub*, top_count);
  {
    SYMS_CvStub **ptr = top_stubs;
    for (SYMS_CvStub *node = root.first;
         node != 0;
         node = node->sibling_next, ptr += 1){
      *ptr = node;
    }
  }
  
  // build proc stubs pointer table
  SYMS_CvStub **proc_stubs = syms_push_array(arena, SYMS_CvStub*, proc_count);
  {
    SYMS_CvStub **ptr = proc_stubs;
    for (SYMS_CvStubRef *ref = proc_first;
         ref != 0;
         ref = ref->next, ptr += 1){
      *ptr = ref->stub;
    }
  }
  
  // build var stubs pointer table
  SYMS_CvStub **var_stubs = syms_push_array(arena, SYMS_CvStub*, var_count);
  {
    SYMS_CvStub **ptr = var_stubs;
    for (SYMS_CvStubRef *ref = var_first;
         ref != 0;
         ref = ref->next, ptr += 1){
      *ptr = ref->stub;
    }
  }
  
  // build thread stubs pointer table
  SYMS_CvStub **tls_var_stubs = syms_push_array(arena, SYMS_CvStub*, tls_var_count);
  {
    SYMS_CvStub **ptr = tls_var_stubs;
    for (SYMS_CvStubRef *ref = tls_var_first;
         ref != 0;
         ref = ref->next, ptr += 1){
      *ptr = ref->stub;
    }
  }
  
  // build thunk stubs pointer table
  SYMS_CvStub **thunk_stubs = syms_push_array(arena, SYMS_CvStub*, thunk_count);
  {
    SYMS_CvStub **ptr = thunk_stubs;
    for (SYMS_CvStubRef *ref = thunk_first;
         ref != 0;
         ref = ref->next, ptr += 1){
      *ptr = ref->stub;
    }
  }
  
  // build pub stubs pointer table
  SYMS_CvStub **pub_stubs = syms_push_array(arena, SYMS_CvStub*, pub_count);
  {
    SYMS_CvStub **ptr = pub_stubs;
    for (SYMS_CvStubRef *ref = pub_first;
         ref != 0;
         ref = ref->next, ptr += 1){
      *ptr = ref->stub;
    }
  }
  
  // build bucket table
  SYMS_U64 bucket_count = (all_count/2)*2 + 3;
  SYMS_CvStub **buckets = syms_push_array(arena, SYMS_CvStub*, bucket_count);
  if (bucket_count > 0){
    syms_memset(buckets, 0, sizeof(*buckets)*bucket_count);
    for (SYMS_CvStub *bucket = first, *next = 0;
         bucket != 0;
         bucket = next){
      next = bucket->bucket_next;
      SYMS_U64 hash = syms_hash_u64(bucket->off);
      SYMS_U32 bucket_index = hash%bucket_count;
      SYMS_StackPush_N(buckets[bucket_index], bucket, bucket_next);
    }
  }
  
  // release scratch
  syms_release_scratch(scratch);
  
  // fill result
  SYMS_CvUnitAccel *result = syms_push_array_zero(arena, SYMS_CvUnitAccel, 1);
  result->format = params->format;
  result->sn = sn;
  result->top_stubs = top_stubs;
  result->top_count = top_count;
  result->buckets = buckets;
  result->bucket_count = bucket_count;
  result->all_count = all_count;
  result->uid = params->uid;
  result->proc_stubs = proc_stubs;
  result->proc_count = proc_count;
  result->var_stubs = var_stubs;
  result->var_count = var_count;
  result->tls_var_stubs = tls_var_stubs;
  result->tls_var_count = tls_var_count;
  result->thunk_stubs = thunk_stubs;
  result->thunk_count = thunk_count;
  result->pub_stubs = pub_stubs;
  result->pub_count = pub_count;
  
  return(result);
}


// cv extract helpers

SYMS_API SYMS_CvStub*
syms_cv_stub_from_unit_off(SYMS_CvUnitAccel *unit, SYMS_U32 off){
  SYMS_CvStub *result = 0;
  if (unit->bucket_count > 0){
    SYMS_U64 hash = syms_hash_u64(off);
    SYMS_U32 bucket_index = hash%unit->bucket_count;
    for (SYMS_CvStub *stub = unit->buckets[bucket_index];
         stub != 0;
         stub = stub->bucket_next){
      if (stub->off == off){
        result = stub;
        break;
      }
    }
  }
  return(result);
}

SYMS_API SYMS_CvStub*
syms_cv_stub_from_unit_index(SYMS_CvUnitAccel *unit, SYMS_U32 index){
  SYMS_CvStub *result = 0;
  if (unit->top_min_index <= index){
    SYMS_U32 relative_index = index - unit->top_min_index;
    if (relative_index < unit->ti_count){
      result = unit->ti_indirect_stubs[relative_index];
    }
  }
  return(result);
}

SYMS_API SYMS_CvStub*
syms_cv_stub_from_unit_sid(SYMS_CvUnitAccel *unit, SYMS_SymbolID sid){
  SYMS_CvStub *result = 0;
  switch (SYMS_ID_u32_0(sid)){
    case SYMS_CvSymbolIDKind_Index:
    {
      result = syms_cv_stub_from_unit_index(unit, SYMS_ID_u32_1(sid));
    }break;
    case SYMS_CvSymbolIDKind_Off:
    {
      result = syms_cv_stub_from_unit_off(unit, SYMS_ID_u32_1(sid));
    }break;
  }
  return(result);
}

SYMS_API SYMS_CvResolvedElement
syms_cv_resolve_from_id(SYMS_String8 data, SYMS_MsfAccel *msf, SYMS_CvUnitAccel *unit,
                        SYMS_SymbolID id){
  // read id
  SYMS_MsfRange range = syms_msf_range_from_sn(msf, unit->sn);
  
  SYMS_CvResolvedElement result = {0};
  switch (SYMS_ID_u32_0(id)){
    case SYMS_CvSymbolIDKind_Index:
    {
      result.stub = syms_cv_stub_from_unit_index(unit, SYMS_ID_u32_1(id));
      if (result.stub != 0){
        SYMS_CvElement element = syms_cv_element(data, msf, range, result.stub->off, 1);
        result.kind = element.kind;
        result.range = element.range;
      }
      result.is_index = syms_true;
      result.is_leaf = unit->leaf_set;
    }break;
    
    case SYMS_CvSymbolIDKind_Off:
    {
      result.stub = syms_cv_stub_from_unit_off(unit, SYMS_ID_u32_1(id));
      if (result.stub != 0){
        if (unit->leaf_set){
          // NOTE(allen): for "leaf" data we don't have built in sizes, so we just
          // return a range to the end of the stream.
          SYMS_U32 lf_off = result.stub->off;
          syms_msf_read_struct_in_range(data, msf, range, lf_off, &result.kind);
          SYMS_U32 lf_data_off = lf_off + 2;
          result.range = syms_msf_sub_range(range, lf_data_off, range.size - lf_data_off);
          result.is_leaf = syms_true;
        }
        
        else{
          // NOTE(allen): for "sym" data we can get the regular cv element
          SYMS_CvElement element = syms_cv_element(data, msf, range, result.stub->off, 1);
          result.kind = element.kind;
          result.range = element.range;
        }
      }
    }break;
  }
  
  return(result);
}

SYMS_API SYMS_U64
syms_cv_type_index_first(SYMS_CvUnitAccel *unit){
  SYMS_U64 result = 0;
  if (unit->leaf_set){
    result = unit->top_min_index;
  }
  return(result);
}

SYMS_API SYMS_U64
syms_cv_type_index_count(SYMS_CvUnitAccel *unit){
  SYMS_U64 result = 0;
  if (unit->leaf_set){
    result = unit->ti_count;
  }
  return(result);
}

// api implementors

SYMS_API SYMS_UnitID
syms_cv_uid_from_accel(SYMS_CvUnitAccel *unit){
  return(unit->uid);
}

SYMS_API SYMS_SymbolKind
syms_cv_symbol_kind_from_sid(SYMS_String8 data, SYMS_MsfAccel *msf, SYMS_CvUnitAccel *unit,
                             SYMS_SymbolID sid){
  //- parse symbol
  SYMS_SymbolKind result = SYMS_SymbolKind_Null;
  
  //- resolve id
  SYMS_CvResolvedElement resolve = syms_cv_resolve_from_id(data, msf, unit, sid);
  
  //- Handle SYM
  if (!resolve.is_leaf){
    if (resolve.stub != 0){
      //- get kind
      switch (resolve.kind){
        default:break;
        case SYMS_CvSymKind_LDATA32:
        case SYMS_CvSymKind_GDATA32:
        {
          result = SYMS_SymbolKind_ImageRelativeVariable;
        }break;
        case SYMS_CvSymKind_LOCAL:
        case SYMS_CvSymKind_REGREL32:
        {
          result = SYMS_SymbolKind_LocalVariable;
        }break;
        case SYMS_CvSymKind_CONSTANT:
        {
          result = SYMS_SymbolKind_Const;
        }break;
        case SYMS_CvSymKind_LPROC32_ID:
        case SYMS_CvSymKind_GPROC32_ID:
        case SYMS_CvSymKind_LPROC32:
        case SYMS_CvSymKind_GPROC32:
        {
          result = SYMS_SymbolKind_Procedure;
        }break;
        case SYMS_CvSymKind_BLOCK32:
        {
          result = SYMS_SymbolKind_Scope;
        }break;
        case SYMS_CvSymKind_LTHREAD32:
        case SYMS_CvSymKind_GTHREAD32:
        {
          result = SYMS_SymbolKind_TLSVariable;
        }break;
      }
    }
  }
  
  //- Handle LEAF
  else{
    //- basic type info
    if (resolve.stub == 0 && resolve.is_index){
      SYMS_CvTypeIndex ti = SYMS_ID_u32_1(sid);
      if (ti == syms_cv_type_id_variadic){
        result = SYMS_SymbolKind_Type;
      }
      else if (ti < 0x1000){
        SYMS_CvBasicPointerKind basic_ptr_kind = SYMS_CvBasicPointerKindFromTypeId(ti);
        switch (basic_ptr_kind){
          case SYMS_CvBasicPointerKind_VALUE:
          case SYMS_CvBasicPointerKind_16BIT:
          case SYMS_CvBasicPointerKind_FAR_16BIT:
          case SYMS_CvBasicPointerKind_HUGE_16BIT:
          case SYMS_CvBasicPointerKind_32BIT:
          case SYMS_CvBasicPointerKind_16_32BIT:
          case SYMS_CvBasicPointerKind_64BIT:
          {
            SYMS_U32 itype_kind = SYMS_CvBasicTypeFromTypeId(ti);
            switch (itype_kind){
              case SYMS_CvBasicType_VOID:
              case SYMS_CvBasicType_HRESULT:
              case SYMS_CvBasicType_RCHAR:
              case SYMS_CvBasicType_CHAR:
              case SYMS_CvBasicType_UCHAR:
              case SYMS_CvBasicType_WCHAR:
              case SYMS_CvBasicType_SHORT:
              case SYMS_CvBasicType_USHORT:
              case SYMS_CvBasicType_LONG:
              case SYMS_CvBasicType_ULONG:
              case SYMS_CvBasicType_QUAD:
              case SYMS_CvBasicType_UQUAD:
              case SYMS_CvBasicType_OCT:
              case SYMS_CvBasicType_UOCT:
              case SYMS_CvBasicType_CHAR8:
              case SYMS_CvBasicType_CHAR16:
              case SYMS_CvBasicType_CHAR32:
              case SYMS_CvBasicType_BOOL8:
              case SYMS_CvBasicType_BOOL16:
              case SYMS_CvBasicType_BOOL32:
              case SYMS_CvBasicType_BOOL64:
              case SYMS_CvBasicType_INT8:
              case SYMS_CvBasicType_INT16:
              case SYMS_CvBasicType_INT32:
              case SYMS_CvBasicType_INT64:
              case SYMS_CvBasicType_INT128:
              case SYMS_CvBasicType_UINT8:
              case SYMS_CvBasicType_UINT16:
              case SYMS_CvBasicType_UINT32:
              case SYMS_CvBasicType_UINT64:
              case SYMS_CvBasicType_UINT128:
              case SYMS_CvBasicType_FLOAT16:
              case SYMS_CvBasicType_FLOAT32:
              case SYMS_CvBasicType_FLOAT64:
              case SYMS_CvBasicType_FLOAT32PP:
              case SYMS_CvBasicType_FLOAT80:
              case SYMS_CvBasicType_FLOAT128:
              case SYMS_CvBasicType_COMPLEX32:
              case SYMS_CvBasicType_COMPLEX64:
              case SYMS_CvBasicType_COMPLEX80:
              case SYMS_CvBasicType_COMPLEX128:
              {
                result = SYMS_SymbolKind_Type;
              }break;
            }
          }break;
        }
      }
    }
    
    //- recorded type info
    if (resolve.stub != 0){
      switch (resolve.kind){
        // type kinds
        case SYMS_CvLeaf_MODIFIER:
        case SYMS_CvLeaf_POINTER:
        case SYMS_CvLeaf_PROCEDURE:
        case SYMS_CvLeaf_MFUNCTION:
        case SYMS_CvLeaf_ARRAY:
        case SYMS_CvLeaf_BITFIELD:
        case SYMS_CvLeaf_CLASS:
        case SYMS_CvLeaf_STRUCTURE:
        case SYMS_CvLeaf_INTERFACE:
        case SYMS_CvLeaf_UNION:
        case SYMS_CvLeaf_ENUM:
        case SYMS_CvLeaf_CLASSPTR:
        case SYMS_CvLeaf_CLASSPTR2:
        case SYMS_CvLeaf_ALIAS:
        {
          result = SYMS_SymbolKind_Type;
        }break;
      }
    }
    
  }
  
  return(result);
}

SYMS_API SYMS_SymbolIDArray
syms_cv_proc_sid_array_from_unit(SYMS_Arena *arena, SYMS_CvUnitAccel *unit){
  SYMS_SymbolIDArray result = {0};
  if (!unit->leaf_set){
    
    //- allocate array
    SYMS_U64 count = unit->proc_count;
    SYMS_SymbolID *ids = syms_push_array(arena, SYMS_SymbolID, count);
    
    //- fill array
    SYMS_SymbolID *id_ptr = ids;
    SYMS_CvStub **stub_ptr = unit->proc_stubs;
    SYMS_CvStub **opl = stub_ptr + count;
    for (; stub_ptr < opl; id_ptr += 1, stub_ptr += 1){
      *id_ptr = SYMS_ID_u32_u32(SYMS_CvSymbolIDKind_Off, (**stub_ptr).off);
    }
    
    //- assemble result
    result.count = count;
    result.ids = ids;
  }
  return(result);
}

SYMS_API SYMS_SymbolIDArray
syms_cv_var_sid_array_from_unit(SYMS_Arena *arena, SYMS_CvUnitAccel *unit){
  SYMS_SymbolIDArray result = {0};
  if (!unit->leaf_set){
    
    //- allocate array
    SYMS_U64 count = unit->var_count;
    SYMS_SymbolID *ids = syms_push_array(arena, SYMS_SymbolID, count);
    
    //- fill array
    SYMS_SymbolID *id_ptr = ids;
    SYMS_CvStub **stub_ptr = unit->var_stubs;
    SYMS_CvStub **opl = stub_ptr + count;
    for (; stub_ptr < opl; id_ptr += 1, stub_ptr += 1){
      *id_ptr = SYMS_ID_u32_u32(SYMS_CvSymbolIDKind_Off, (**stub_ptr).off);
    }
    
    //- assemble result
    result.count = count;
    result.ids = ids;
  }
  return(result);
}

SYMS_API SYMS_SymbolIDArray
syms_cv_type_sid_array_from_unit(SYMS_Arena *arena, SYMS_CvUnitAccel *unit){
  SYMS_SymbolIDArray result = {0};
  if (unit->leaf_set){
    
    //- allocate array
    SYMS_U64 count = unit->top_count;
    SYMS_SymbolID *ids = syms_push_array(arena, SYMS_SymbolID, count);
    
    //- fill array
    SYMS_SymbolID *id_ptr = ids;
    SYMS_CvStub **stub_ptr = unit->top_stubs;
    SYMS_CvStub **opl = stub_ptr + count;
    for (; stub_ptr < opl; id_ptr += 1, stub_ptr += 1){
      SYMS_CvStub *stub = *stub_ptr;
      *id_ptr = SYMS_ID_u32_u32(SYMS_CvSymbolIDKind_Index, stub->index);
    }
    
    //- assemble result
    result.count = count;
    result.ids = ids;
  }
  return(result);
}

SYMS_API SYMS_String8
syms_cv_symbol_name_from_sid(SYMS_Arena *arena, SYMS_CvUnitAccel *unit, SYMS_SymbolID sid){
  //- zero result
  SYMS_String8 result = {0};
  
  //- resolve id
  SYMS_CvStub *stub = syms_cv_stub_from_unit_sid(unit, sid);
  if (stub != 0){
    result = syms_push_string_copy(arena, stub->name);
  }
  
  //- try basic type
  else if (unit->leaf_set && (SYMS_ID_u32_0(sid) == SYMS_CvSymbolIDKind_Index)){
    SYMS_CvTypeIndex ti = SYMS_ID_u32_1(sid);
    if (ti < 0x1000){
      SYMS_CvBasicPointerKind basic_ptr_kind = SYMS_CvBasicPointerKindFromTypeId(ti);
      if (basic_ptr_kind == SYMS_CvBasicPointerKind_VALUE){
        SYMS_U32 itype_kind = SYMS_CvBasicTypeFromTypeId(ti);
        result = syms_string_from_enum_value(SYMS_CvBasicType, itype_kind);
      }
    }
  }
  
  return(result);
}

SYMS_API SYMS_TypeInfo
syms_cv_type_info_from_sid(SYMS_String8 data, SYMS_MsfAccel *msf, SYMS_CvUnitAccel *unit,
                           SYMS_SymbolID sid){
  // zero clear result
  SYMS_TypeInfo result = {SYMS_TypeKind_Null};
  
  // setup uid
  SYMS_UnitID uid = unit->uid;
  
  // resolve id
  SYMS_CvResolvedElement resolve = syms_cv_resolve_from_id(data, msf, unit, sid);
  if (resolve.is_leaf){
    
    // basic type info
    if (resolve.stub == 0 && resolve.is_index){
      SYMS_CvTypeIndex ti = SYMS_ID_u32_1(sid);
      if (ti == syms_cv_type_id_variadic){
        result.kind = SYMS_TypeKind_Variadic;
      }
      else if (ti < 0x1000){
        SYMS_CvBasicPointerKind basic_ptr_kind = SYMS_CvBasicPointerKindFromTypeId(ti);
        SYMS_U32 itype_kind = SYMS_CvBasicTypeFromTypeId(ti);
        
        result.reported_size_interp = SYMS_SizeInterpretation_ByteCount;
        
        switch (basic_ptr_kind){
          default:
          {
            result.reported_size_interp = SYMS_SizeInterpretation_Null;
          }break;
          
          case SYMS_CvBasicPointerKind_VALUE:
          {
            switch (itype_kind){
              default:
              {
                result.reported_size_interp = SYMS_SizeInterpretation_Null;
              }break;
              
              case SYMS_CvBasicType_VOID:
              {
                result.kind = SYMS_TypeKind_Void;
                result.reported_size_interp = SYMS_SizeInterpretation_Null;
              }break;
              
              case SYMS_CvBasicType_HRESULT:
              {
                result.kind = SYMS_TypeKind_Void;
                result.reported_size = 4;
              }break;
              
              case SYMS_CvBasicType_RCHAR:
              case SYMS_CvBasicType_CHAR:
              {
                result.kind = SYMS_TypeKind_Int8;
                result.mods = SYMS_TypeModifier_Char;
                result.reported_size = 1;
              }break;
              
              case SYMS_CvBasicType_UCHAR:
              {
                result.kind = SYMS_TypeKind_UInt8;
                result.mods = SYMS_TypeModifier_Char;
                result.reported_size = 1;
              }break;
              
              case SYMS_CvBasicType_WCHAR:
              {
                result.kind = SYMS_TypeKind_UInt16;
                result.mods = SYMS_TypeModifier_Char;
                result.reported_size = 2;
              }break;
              
              case SYMS_CvBasicType_BOOL8:
              case SYMS_CvBasicType_CHAR8:
              case SYMS_CvBasicType_INT8:
              {
                result.kind = SYMS_TypeKind_Int8;
                result.reported_size = 1;
              }break;
              
              case SYMS_CvBasicType_BOOL16:
              case SYMS_CvBasicType_CHAR16:
              case SYMS_CvBasicType_SHORT:
              case SYMS_CvBasicType_INT16:
              {
                result.kind = SYMS_TypeKind_Int16;
                result.reported_size = 2;
              }break;
              
              case SYMS_CvBasicType_BOOL32:
              case SYMS_CvBasicType_CHAR32:
              case SYMS_CvBasicType_INT32:
              {
                result.kind = SYMS_TypeKind_Int32;
                result.reported_size = 4;
              }break;
              
              case SYMS_CvBasicType_BOOL64:
              case SYMS_CvBasicType_QUAD:
              case SYMS_CvBasicType_INT64:
              {
                result.kind = SYMS_TypeKind_Int64;
                result.reported_size = 8;
              }break;
              
              case SYMS_CvBasicType_OCT:
              case SYMS_CvBasicType_INT128:
              {
                result.kind = SYMS_TypeKind_Int128;
                result.reported_size = 16;
              }break;
              
              case SYMS_CvBasicType_UINT8:
              {
                result.kind = SYMS_TypeKind_UInt8;
                result.reported_size = 1;
              }break;
              
              case SYMS_CvBasicType_USHORT:
              case SYMS_CvBasicType_UINT16:
              {
                result.kind = SYMS_TypeKind_UInt16;
                result.reported_size = 2;
              }break;
              
              case SYMS_CvBasicType_LONG:
              {
                result.kind = SYMS_TypeKind_Int32;
                result.reported_size = 4;
              }break;
              
              case SYMS_CvBasicType_ULONG:
              {
                result.kind = SYMS_TypeKind_UInt32;
                result.reported_size = 4;
              }break;
              
              case SYMS_CvBasicType_UINT32:
              {
                result.kind = SYMS_TypeKind_UInt32;
                result.reported_size = 4;
              }break;
              
              case SYMS_CvBasicType_UQUAD:
              case SYMS_CvBasicType_UINT64:
              {
                result.kind = SYMS_TypeKind_UInt64;
                result.reported_size = 8;
              }break;
              
              case SYMS_CvBasicType_UOCT:
              case SYMS_CvBasicType_UINT128:
              {
                result.kind = SYMS_TypeKind_UInt128;
                result.reported_size = 16;
              }break;
              
              case SYMS_CvBasicType_FLOAT16:
              {
                result.kind = SYMS_TypeKind_Float16;
                result.reported_size = 2;
              }break;
              
              case SYMS_CvBasicType_FLOAT32:
              {
                result.kind = SYMS_TypeKind_Float32;
                result.reported_size = 4;
              }break;
              
              case SYMS_CvBasicType_FLOAT64:
              {
                result.kind = SYMS_TypeKind_Float64;
                result.reported_size = 8;
              }break;
              
              case SYMS_CvBasicType_FLOAT32PP:
              {
                result.kind = SYMS_TypeKind_Float32PP;
                result.reported_size = 4;
              }break;
              
              case SYMS_CvBasicType_FLOAT80:
              {
                result.kind = SYMS_TypeKind_Float80;
                result.reported_size = 10;
              }break;
              
              case SYMS_CvBasicType_FLOAT128:
              {
                result.kind = SYMS_TypeKind_Float128;
                result.reported_size = 16;
              }break;
              
              case SYMS_CvBasicType_COMPLEX32:
              {
                result.kind = SYMS_TypeKind_Complex32;
                result.reported_size = 8;
              }break;
              
              case SYMS_CvBasicType_COMPLEX64:
              {
                result.kind = SYMS_TypeKind_Complex64;
                result.reported_size = 16;
              }break;
              
              case SYMS_CvBasicType_COMPLEX80:
              {
                result.kind = SYMS_TypeKind_Complex80;
                result.reported_size = 20;
              }break;
              
              case SYMS_CvBasicType_COMPLEX128:
              {
                result.kind = SYMS_TypeKind_Complex128;
                result.reported_size = 32;
              }break;
            }
          }break;
          
          case SYMS_CvBasicPointerKind_16BIT:
          case SYMS_CvBasicPointerKind_FAR_16BIT:
          case SYMS_CvBasicPointerKind_HUGE_16BIT:
          {
            result.kind = SYMS_TypeKind_Ptr;
            result.reported_size = 2;
            result.direct_type.uid = uid;
            result.direct_type.sid = SYMS_ID_u32_u32(SYMS_CvSymbolIDKind_Index, itype_kind);
          }break;
          
          case SYMS_CvBasicPointerKind_32BIT:
          case SYMS_CvBasicPointerKind_16_32BIT:
          {
            result.kind = SYMS_TypeKind_Ptr;
            result.reported_size = 4;
            result.direct_type.uid = uid;
            result.direct_type.sid = SYMS_ID_u32_u32(SYMS_CvSymbolIDKind_Index, itype_kind);
          }break;
          
          case SYMS_CvBasicPointerKind_64BIT:
          {
            result.kind = SYMS_TypeKind_Ptr;
            result.reported_size  = 8;
            result.direct_type.uid = uid;
            result.direct_type.sid = SYMS_ID_u32_u32(SYMS_CvSymbolIDKind_Index, itype_kind);
          }break;
        }
      }
    }
    
    // recorded type info
    if (resolve.stub != 0){
      // shared fields for user data type paths
      SYMS_TypeKind type_kind = SYMS_TypeKind_Null;
      SYMS_CvTypeProps props = 0;
      
      switch (resolve.kind){
        default:break;
        
        case SYMS_CvLeaf_MODIFIER:
        {
          SYMS_CvLeafModifier modifier = {0};
          if (syms_msf_read_struct_in_range(data, msf, resolve.range, 0, &modifier)){
            result.kind = SYMS_TypeKind_Modifier;
            result.mods = syms_pdb_modifier_from_cv_modifier_flags(modifier.flags);
            result.direct_type.uid = uid;
            result.direct_type.sid = SYMS_ID_u32_u32(SYMS_CvSymbolIDKind_Index, modifier.itype);
            result.reported_size_interp = SYMS_SizeInterpretation_Multiplier;
            result.reported_size = 1;
          }
        }break;
        
        case SYMS_CvLeaf_POINTER:
        {
          SYMS_CvLeafPointer ptr = {0};
          if (syms_msf_read_struct_in_range(data, msf, resolve.range, 0, &ptr)){
            SYMS_U64 size = SYMS_CvPointerAttribs_Extract_SIZE(ptr.attr);
            //SYMS_CvPointerKind ptr_kind = SYMS_CvPointerAttribs_Extract_KIND(ptr.attr);
            SYMS_CvPointerMode ptr_mode = SYMS_CvPointerAttribs_Extract_MODE(ptr.attr);
            
            SYMS_TypeKind type_kind = syms_pdb_type_kind_from_cv_pointer_mode(ptr_mode);
            
            SYMS_CvTypeIndex containing_type = 0;
            if (type_kind == SYMS_TypeKind_MemberPtr){
              syms_msf_read_struct_in_range(data, msf, resolve.range, sizeof(ptr), &containing_type);
            }
            
            result.direct_type.uid = uid;
            result.direct_type.sid = SYMS_ID_u32_u32(SYMS_CvSymbolIDKind_Index, ptr.itype);
            result.kind = type_kind;
            result.mods = syms_pdb_modifier_from_cv_pointer_attribs(ptr.attr);
            result.reported_size_interp = SYMS_SizeInterpretation_ByteCount;
            result.reported_size = size;
            if (containing_type != 0){
              result.containing_type.uid = uid;
              result.containing_type.sid = SYMS_ID_u32_u32(SYMS_CvSymbolIDKind_Index, containing_type);
            }
          }
        }break;
        
        case SYMS_CvLeaf_PROCEDURE:
        {
          SYMS_CvLeafProcedure proc = {0};
          if (syms_msf_read_struct_in_range(data, msf, resolve.range, 0, &proc)){
            // skipped: funcattr, arg_count, arg_itype
            result.direct_type.uid = uid;
            result.direct_type.sid = SYMS_ID_u32_u32(SYMS_CvSymbolIDKind_Index, proc.ret_itype);
            result.kind = SYMS_TypeKind_Proc;
            result.call_convention = syms_pdb_call_convention_from_cv_call_kind(proc.call_kind);
          }
        }break;
        
        case SYMS_CvLeaf_MFUNCTION:
        {
          SYMS_CvLeafMFunction mfunc = {0};
          if (syms_msf_read_struct_in_range(data, msf, resolve.range, 0, &mfunc)){
            // skipped: class_itype, this_itype, funcattr, arg_count, arg_itype, thisadjust
            result.direct_type.uid = uid;
            result.direct_type.sid = SYMS_ID_u32_u32(SYMS_CvSymbolIDKind_Index, mfunc.ret_itype);
            result.kind = SYMS_TypeKind_Proc;
            result.call_convention = syms_pdb_call_convention_from_cv_call_kind(mfunc.call_kind);
          }
        }break;
        
        case SYMS_CvLeaf_ARRAY:
        {
          SYMS_CvLeafArray array = {0};
          if (syms_msf_read_struct_in_range(data, msf, resolve.range, 0, &array)){
            result.direct_type.uid = uid;
            result.direct_type.sid = SYMS_ID_u32_u32(SYMS_CvSymbolIDKind_Index, array.entry_itype);
            result.kind = SYMS_TypeKind_Array;
            result.reported_size_interp = SYMS_SizeInterpretation_ByteCount;
            result.reported_size = resolve.stub->num;
          }
        }break;
        
        case SYMS_CvLeaf_BITFIELD:
        {
          SYMS_CvLeafBitField bitfield = {0};
          if (syms_msf_read_struct_in_range(data, msf, resolve.range, 0, &bitfield)){
            result.direct_type.uid = uid;
            result.direct_type.sid = SYMS_ID_u32_u32(SYMS_CvSymbolIDKind_Index, bitfield.itype);
            result.kind = SYMS_TypeKind_Bitfield;
            result.reported_size_interp = SYMS_SizeInterpretation_Multiplier;
            result.reported_size = 1;
            // TODO(allen): bitfield reporting
          }
        }break;
        
        // udt
        {
          case SYMS_CvLeaf_STRUCTURE:
          {
            type_kind = SYMS_TypeKind_Struct;
            goto read_struct;
          }
          case SYMS_CvLeaf_CLASS:
          case SYMS_CvLeaf_INTERFACE:
          {
            type_kind = SYMS_TypeKind_Class;
            goto read_struct;
          }
          read_struct:
          {
            SYMS_CvLeafStruct struct_ = {0};
            syms_msf_read_struct_in_range(data, msf, resolve.range, 0, &struct_);
            props = struct_.props;
            goto fill_result;
          }
          
          case SYMS_CvLeaf_UNION:
          {
            type_kind = SYMS_TypeKind_Union;
            SYMS_CvLeafUnion union_ = {0};
            syms_msf_read_struct_in_range(data, msf, resolve.range, 0, &union_);
            props = union_.props;
            goto fill_result;
          }
          
          case SYMS_CvLeaf_ENUM:
          {
            type_kind = SYMS_TypeKind_Enum;
            SYMS_CvLeafEnum enum_ = {0};
            syms_msf_read_struct_in_range(data, msf, resolve.range, 0, &enum_);
            props = enum_.props;
            result.direct_type.uid = uid;
            result.direct_type.sid = SYMS_ID_u32_u32(SYMS_CvSymbolIDKind_Index, enum_.itype);
            goto fill_result;
          }
          
          case SYMS_CvLeaf_CLASSPTR:
          case SYMS_CvLeaf_CLASSPTR2:
          {
            type_kind = SYMS_TypeKind_Struct;
            SYMS_CvLeafClassPtr class_ptr = {0};
            syms_msf_read_struct_in_range(data, msf, resolve.range, 0, &class_ptr);
            props = class_ptr.props;
            goto fill_result;
          }
          
          fill_result:
          {
            SYMS_B32 is_fwdref = !!(props & SYMS_CvTypeProp_FWDREF);
            if (is_fwdref){
              result.kind = syms_type_kind_fwd_from_main(type_kind);
              result.reported_size_interp = SYMS_SizeInterpretation_ResolveForwardReference;
              result.reported_size = 0;
            }
            else{
              result.kind = type_kind;
              if (type_kind == SYMS_TypeKind_Enum){
                result.reported_size_interp = SYMS_SizeInterpretation_Multiplier;
                result.reported_size = 1;
              }
              else{
                result.reported_size_interp = SYMS_SizeInterpretation_ByteCount;
                result.reported_size = resolve.stub->num;
              }
            }
          }
        }break;
        
        case SYMS_CvLeaf_ALIAS:
        {
          SYMS_CvLeafAlias alias = {0};
          if (syms_msf_read_struct_in_range(data, msf, resolve.range, 0, &alias)){
            result.kind = SYMS_TypeKind_Typedef;
            result.direct_type.uid = uid;
            result.direct_type.sid = SYMS_ID_u32_u32(SYMS_CvSymbolIDKind_Index, alias.itype);
            result.reported_size_interp = SYMS_SizeInterpretation_Multiplier;
            result.reported_size = 1;
          }
        }break;
      }
    }
  }
  
  return(result);
}

SYMS_API SYMS_ConstInfo
syms_cv_const_info_from_sid(SYMS_String8 data, SYMS_MsfAccel *msf, SYMS_CvUnitAccel *unit,
                            SYMS_SymbolID sid){
  // zero clear result
  SYMS_ConstInfo result = {SYMS_TypeKind_Null};
  
  // read id
  SYMS_CvResolvedElement resolve = syms_cv_resolve_from_id(data, msf, unit, sid);
  if (resolve.is_leaf && resolve.stub != 0 &&
      resolve.kind == SYMS_CvLeaf_ENUMERATE){
    SYMS_CvLeafEnumerate enumerate  = {0};
    if (syms_msf_read_struct_in_range(data, msf, resolve.range, 0, &enumerate)){
      // TODO(allen): attribs: SYMS_CvFieldAttribs;
      
      SYMS_U32 num_off = sizeof(SYMS_CvLeafEnumerate);
      SYMS_CvNumeric num = {SYMS_TypeKind_Null};
      SYMS_U32 num_size = syms_cv_read_numeric(data, msf, resolve.range, num_off, &num);
      (void)num_size;
      
      result.kind = num.kind;
      syms_memmove(&result.val, num.data, sizeof(result.val));
    }
  }
  
  return(result);
}


