// Copyright Epic Games, Inc. All Rights Reserved.
/* date = May 24th 2022 4:16 pm */

#ifndef SYMS_CV_HELPERS_H
#define SYMS_CV_HELPERS_H

// TODO(allen): simplifying & optimizing pass

////////////////////////////////
//~ allen: CodeView Fundamental

typedef enum SYMS_CvSymbolIDKind{
  SYMS_CvSymbolIDKind_Off,
  SYMS_CvSymbolIDKind_Index,
} SYMS_CvSymbolIDKind;

typedef enum SYMS_CvFileIDKind{
  SYMS_CvFileIDKind_Null,
  SYMS_CvFileIDKind_IPIOff,
  SYMS_CvFileIDKind_IPIStringID,
  SYMS_CvFileIDKind_StrTblOff,
  SYMS_CvFileIDKind_C11Off,
} SYMS_CvFileIDKind;

////////////////////////////////
//~ allen: CodeView Parsing

typedef struct SYMS_CvElement{
  SYMS_MsfRange range;
  SYMS_U32 next_off;
  SYMS_U16 kind;
} SYMS_CvElement;

typedef struct SYMS_CvNumeric{
  SYMS_TypeKind kind;
  SYMS_U8 data[32];
} SYMS_CvNumeric;

////////////////////////////////
//~ allen: CodeView C13

typedef struct SYMS_CvC13SubSection{
  struct SYMS_CvC13SubSection *next;
  SYMS_CvSubSectionKind kind;
  SYMS_U32 off;
  SYMS_U32 size;
} SYMS_CvC13SubSection;

typedef struct SYMS_CvC13SubSectionList{
  SYMS_CvC13SubSection *first;
  SYMS_CvC13SubSection *last;
  SYMS_U64 count;
} SYMS_CvC13SubSectionList;

////////////////////////////////
//~ allen: CodeView Line Info

typedef struct SYMS_CvFileNode{
  struct SYMS_CvFileNode *next;
  SYMS_U64 count;
  SYMS_FileID file_ids[6];
} SYMS_CvFileNode;

typedef struct SYMS_CvLineSequence{
  struct SYMS_CvLineSequence *next;
  SYMS_Line *lines;
  SYMS_U64 line_count;
} SYMS_CvLineSequence;

typedef struct SYMS_CvLineTableLoose{
  SYMS_CvFileNode *first_file_node;
  SYMS_CvFileNode *last_file_node;
  SYMS_U64 file_count;
  
  SYMS_CvLineSequence *first_seq;
  SYMS_CvLineSequence *last_seq;
  SYMS_U64 seq_count;
  SYMS_U64 line_count;
} SYMS_CvLineTableLoose;

////////////////////////////////
//~ allen: CodeView Units

typedef struct SYMS_CvStub{
  struct SYMS_CvStub *bucket_next;
  struct SYMS_CvStub *parent;
  struct SYMS_CvStub *sibling_next;
  struct SYMS_CvStub *first;
  struct SYMS_CvStub *last;
  SYMS_U32 off;
  SYMS_U32 index;
  SYMS_U32 num;
  SYMS_U32 num2;
  union{
    SYMS_String8 name;
  };
} SYMS_CvStub;

typedef struct SYMS_CvStubRef{
  struct SYMS_CvStubRef *next;
  SYMS_CvStub *stub;
} SYMS_CvStubRef;

typedef struct SYMS_CvUnitAccel{
  SYMS_FileFormat format;
  SYMS_B32 leaf_set;
  SYMS_MsfStreamNumber sn;
  SYMS_CvStub **top_stubs;
  SYMS_U64 top_count;
  SYMS_U64 top_min_index;
  SYMS_CvStub **buckets;
  SYMS_U64 bucket_count;
  SYMS_U64 all_count;
  SYMS_CvStub **ti_indirect_stubs;
  SYMS_U64 ti_count;
  SYMS_UnitID uid;
  
  SYMS_CvStub **proc_stubs;
  SYMS_U64 proc_count;
  SYMS_CvStub **var_stubs;
  SYMS_U64 var_count;
  SYMS_CvStub **tls_var_stubs;
  SYMS_U64 tls_var_count;
  SYMS_CvStub **thunk_stubs;
  SYMS_U64 thunk_count;
  SYMS_CvStub **pub_stubs;
  SYMS_U64 pub_count;
} SYMS_CvUnitAccel;

typedef struct SYMS_CvResolvedElement{
  SYMS_CvStub *stub;
  SYMS_U16 kind;
  SYMS_B8 is_leaf;
  SYMS_B8 is_index;
  SYMS_MsfRange range;
} SYMS_CvResolvedElement;

////////////////////////////////
//~ allen: CodeView Members

typedef struct SYMS_CvMemStubNode{
  struct SYMS_CvMemStubNode *next;
  SYMS_String8 name;
  SYMS_U32 num;
  SYMS_U32 off;
  SYMS_U32 off2;
} SYMS_CvMemStubNode;

typedef struct SYMS_PdbMemStubList{
  SYMS_CvMemStubNode *first;
  SYMS_CvMemStubNode *last;
  SYMS_U64 mem_count;
} SYMS_PdbMemStubList;

typedef struct SYMS_CvMemsAccel{
  SYMS_FileFormat format;
  SYMS_String8 type_name;
  SYMS_U64 count;
  SYMS_CvMemStubNode **members;
} SYMS_CvMemsAccel;

////////////////////////////////
//~ allen: CodeView Parser Parameters

typedef struct SYMS_CvLeafConsParams{
  SYMS_FileFormat format;
  SYMS_UnitID uid;
  SYMS_CvTypeIndex first_ti;
  SYMS_U32 align;
} SYMS_CvLeafConsParams;

typedef struct SYMS_CvSymConsParams{
  SYMS_FileFormat format;
  SYMS_U32 align;
  SYMS_UnitID uid;
} SYMS_CvSymConsParams;

////////////////////////////////
//~ allen: CodeView Parser Functions

// cv parse helers
SYMS_API SYMS_CvElement syms_cv_element(SYMS_String8 data, SYMS_MsfAccel *msf, SYMS_MsfRange range,
                                        SYMS_U32 off, SYMS_U32 align);

SYMS_API SYMS_U32 syms_cv_read_numeric(SYMS_String8 data, SYMS_MsfAccel *msf, SYMS_MsfRange range,
                                       SYMS_U32 off, SYMS_CvNumeric *out);
SYMS_API SYMS_U32 syms_cv_u32_from_numeric(SYMS_CvNumeric num);

SYMS_API SYMS_CvStubRef*syms_cv_alloc_ref(SYMS_Arena *arena, SYMS_CvStubRef **free_list);

SYMS_API void syms_cv_c13_sub_sections_from_range(SYMS_Arena *arena, SYMS_String8 data, 
                                                  SYMS_MsfAccel *msf, SYMS_MsfRange range,
                                                  SYMS_CvC13SubSectionList *list_out);

SYMS_API SYMS_SecInfoArray syms_cv_sec_info_array_from_bin(SYMS_Arena *arena, SYMS_String8 data,
                                                           SYMS_MsfAccel *msf, SYMS_MsfRange range);

// cv line info
SYMS_API void       sym_cv_loose_push_file_id(SYMS_Arena *arena, SYMS_CvLineTableLoose *loose,
                                              SYMS_FileID id);
SYMS_API SYMS_Line* sym_cv_loose_push_sequence(SYMS_Arena *arena, SYMS_CvLineTableLoose *loose,
                                               SYMS_U64 line_count);
SYMS_API SYMS_LineParseOut sym_cv_line_parse_from_loose(SYMS_Arena *arena,
                                                        SYMS_CvLineTableLoose *loose);

SYMS_API void sym_cv_loose_lines_from_c13(SYMS_Arena *arena, SYMS_String8 data,
                                          SYMS_MsfAccel *msf, SYMS_MsfRange c13_range,
                                          SYMS_CvC13SubSection *sub_sections,
                                          SYMS_U64 *section_voffs, SYMS_U64 section_count,
                                          SYMS_CvLineTableLoose *loose);
SYMS_API void sym_cv_loose_lines_from_c11(SYMS_Arena *arena, SYMS_String8 data,
                                          SYMS_MsfAccel *msf, SYMS_MsfRange c11_range,
                                          SYMS_U64 *section_voffs, SYMS_U64 section_count,
                                          SYMS_CvLineTableLoose *loose);


// cv parsers
SYMS_API SYMS_CvUnitAccel*syms_cv_leaf_unit_from_range(SYMS_Arena *arena, SYMS_String8 data,
                                                       SYMS_MsfAccel *msf,
                                                       SYMS_MsfStreamNumber sn, SYMS_U64Range range,
                                                       SYMS_CvLeafConsParams *params);

SYMS_API SYMS_CvUnitAccel*syms_cv_sym_unit_from_ranges(SYMS_Arena *arena, SYMS_String8 data,
                                                       SYMS_MsfAccel *msf, 
                                                       SYMS_MsfStreamNumber sn,
                                                       SYMS_U64RangeArray ranges,
                                                       SYMS_CvSymConsParams *params);

// cv extract helpers
SYMS_API SYMS_CvStub* syms_cv_stub_from_unit_off(SYMS_CvUnitAccel *unit, SYMS_U32 off);
SYMS_API SYMS_CvStub* syms_cv_stub_from_unit_index(SYMS_CvUnitAccel *unit, SYMS_U32 index);
SYMS_API SYMS_CvStub* syms_cv_stub_from_unit_sid(SYMS_CvUnitAccel *unit, SYMS_SymbolID sid);
SYMS_API SYMS_CvResolvedElement syms_cv_resolve_from_id(SYMS_String8 data, SYMS_MsfAccel *msf,
                                                        SYMS_CvUnitAccel *unit, SYMS_SymbolID id);

SYMS_API SYMS_U64 syms_cv_type_index_first(SYMS_CvUnitAccel *unit);
SYMS_API SYMS_U64 syms_cv_type_index_count(SYMS_CvUnitAccel *unit);

#define syms_cv_sid_from_type_index(idx) SYMS_ID_u32_u32(SYMS_CvSymbolIDKind_Index, idx)

// main api
SYMS_API SYMS_UnitID     syms_cv_uid_from_accel(SYMS_CvUnitAccel *unit);

SYMS_API SYMS_SymbolIDArray syms_cv_proc_sid_array_from_unit(SYMS_Arena *arena, SYMS_CvUnitAccel *unit);
SYMS_API SYMS_SymbolIDArray syms_cv_var_sid_array_from_unit(SYMS_Arena *arena, SYMS_CvUnitAccel *unit);
SYMS_API SYMS_SymbolIDArray syms_cv_type_sid_array_from_unit(SYMS_Arena *arena, SYMS_CvUnitAccel *unit);

SYMS_API SYMS_SymbolKind syms_cv_symbol_kind_from_sid(SYMS_String8 data, SYMS_MsfAccel *msf,
                                                      SYMS_CvUnitAccel *unit, SYMS_SymbolID sid);
SYMS_API SYMS_String8    syms_cv_symbol_name_from_sid(SYMS_Arena *arena, SYMS_CvUnitAccel *unit,
                                                      SYMS_SymbolID sid);

SYMS_API SYMS_TypeInfo   syms_cv_type_info_from_sid(SYMS_String8 data, SYMS_MsfAccel *msf,
                                                    SYMS_CvUnitAccel *unit, SYMS_SymbolID sid);

SYMS_API SYMS_ConstInfo  syms_cv_const_info_from_sid(SYMS_String8 data, SYMS_MsfAccel *msf,
                                                     SYMS_CvUnitAccel *unit, SYMS_SymbolID sid);


#endif //SYMS_CV_HELPERS_H
