// Copyright Epic Games, Inc. All Rights Reserved.
/* date = May 4th 2021 1:54 pm */

#ifndef SYMS_MSF_H
#define SYMS_MSF_H

////////////////////////////////
// NOTE(allen): MSF Format Types

#define SYMS_MSF_INVALID_STREAM_NUMBER SYMS_U16_MAX
typedef SYMS_U16 SYMS_MsfStreamNumber; 

SYMS_READ_ONLY SYMS_GLOBAL char syms_msf20_magic[] = "Microsoft C/C++ program database 2.00\r\n\x1aJG\0\0";
SYMS_READ_ONLY SYMS_GLOBAL char syms_msf70_magic[] = "Microsoft C/C++ MSF 7.00\r\n\032DS\0\0";

#define SYMS_MSF20_MAGIC_SIZE 44
#define SYMS_MSF70_MAGIC_SIZE 32
#define SYMS_MSF_MAX_MAGIC_SIZE 44

typedef struct SYMS_MsfHeader20{
  SYMS_U32 block_size;
  SYMS_U16 free_block_map_block;
  SYMS_U16 block_count;
  SYMS_U32 directory_size;
  SYMS_U32 unknown;
  SYMS_U16 directory_map;
} SYMS_MsfHeader20;

typedef struct SYMS_MsfHeader70{
  SYMS_U32 block_size;
  SYMS_U32 free_block_map_block;
  SYMS_U32 block_count;
  SYMS_U32 directory_size;
  SYMS_U32 unknown;
  SYMS_U32 directory_super_map;
} SYMS_MsfHeader70;

#endif //SYMS_MSF_H
