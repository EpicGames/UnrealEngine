// Copyright Epic Games, Inc. All Rights Reserved.

#ifndef SYMS_MSF_PARSER_C
#define SYMS_MSF_PARSER_C

////////////////////////////////
//~ allen: MSF Reader Fundamentals Without Accelerator

SYMS_API SYMS_MsfHeaderInfo
syms_msf_header_info_from_data_slow(SYMS_String8 data){
  void *base = data.str;
  SYMS_U64Range range = syms_make_u64_range(0, data.size);
  
  // determine msf type
  SYMS_U32 index_size = 0;
  char magic_buffer[SYMS_MSF_MAX_MAGIC_SIZE];
  if (syms_based_range_read(base, range, 0, SYMS_MSF20_MAGIC_SIZE, magic_buffer) &&
      syms_memcmp(syms_msf20_magic, magic_buffer, SYMS_MSF20_MAGIC_SIZE) == 0){
    index_size = 2;
  }
  else if (syms_based_range_read(base, range, 0, SYMS_MSF70_MAGIC_SIZE, magic_buffer) &&
           syms_memcmp(syms_msf70_magic, magic_buffer, SYMS_MSF70_MAGIC_SIZE) == 0){
    index_size = 4;
  }
  
  // grab parts of header we will use in syms
  SYMS_MsfHeaderInfo result = {0};
  
  if (index_size == 2){
    SYMS_MsfHeader20 header = {0};
    if (syms_based_range_read_struct(base, range, SYMS_MSF20_MAGIC_SIZE, &header)){
      result.index_size = index_size;
      result.block_size = header.block_size;
      result.block_count = header.block_count;
      result.directory_size = header.directory_size;
    }
  }
  else if (index_size == 4){
    SYMS_MsfHeader70 header = {0};
    if (syms_based_range_read_struct(base, range, SYMS_MSF70_MAGIC_SIZE, &header)){
      result.index_size = index_size;
      result.block_size = header.block_size;
      result.block_count = header.block_count;
      result.directory_size = header.directory_size;
      result.directory_super_map = header.directory_super_map;
    }
  }
  
  return(result);
}

////////////////////////////////
//~ allen: MSF Reader Accelerator Constructor

SYMS_API SYMS_MsfAccel*
syms_msf_accel_from_data(SYMS_Arena *arena, SYMS_String8 data){
  // NOTE(allen): Layout of directory
  //
  // PDB20:
  // struct Pdb20StreamSize{
  //  U32 size;
  //  U32 unknown; // looks like kind codes or revision counters or something
  // }
  // struct{
  //  U32 stream_count;
  //  Pdb20StreamSize stream_sizes[stream_count];
  //  U16 stream_indices[stream_count][...];
  // }
  //
  // PDB70:
  // struct{
  //  U32 stream_count;
  //  U32 stream_sizes[stream_count];
  //  U32 stream_indices[stream_count][...];
  // }
  
  //- setup result
  SYMS_MsfAccel *result = 0;
  
  //- header
  SYMS_MsfHeaderInfo header = syms_msf_header_info_from_data_slow(data);
  if (header.index_size > 0){
    
    //- directory
    SYMS_B32 got_directory = syms_true;
    SYMS_U8 *directory = 0;
    {
      SYMS_U32 directory_size = header.directory_size;
      directory = syms_push_array(arena, SYMS_U8, directory_size);
      
      // setup important sizes and counts
      SYMS_U32 size_of_block_index = header.index_size;
      
      SYMS_U64 file_size = data.size;
      SYMS_U32 block_size = SYMS_ClampTop(header.block_size, file_size);
      
      SYMS_U32 block_count_in_directory = SYMS_CeilIntegerDiv(directory_size, block_size);
      SYMS_U32 directory_map_size = block_count_in_directory*size_of_block_index;
      
      SYMS_U32 block_count_in_directory_map = SYMS_CeilIntegerDiv(directory_map_size, block_size);
      
      SYMS_U32 block_count_in_whole_file_max = SYMS_CeilIntegerDiv(file_size, block_size);
      SYMS_U32 block_count_in_whole_file = SYMS_ClampTop(header.block_count, block_count_in_whole_file_max);
      
      // setup the index blocks
      SYMS_U32 directory_super_map_dummy = 0;
      SYMS_U32 *directory_super_map = 0;
      SYMS_U32 directory_map_block_skip_size = 0;
      
      if (size_of_block_index == 2){
        directory_super_map = &directory_super_map_dummy;
        directory_map_block_skip_size = SYMS_MSF20_MAGIC_SIZE + SYMS_MEMBER_OFFSET(SYMS_MsfHeader20, directory_map);
      }
      else{
        SYMS_U64 super_map_off = SYMS_MSF70_MAGIC_SIZE + SYMS_MEMBER_OFFSET(SYMS_MsfHeader70, directory_super_map);
        directory_super_map = (SYMS_U32*)(data.str + super_map_off);
      }
      
      SYMS_U32 max_index_count_in_map_block = (block_size - directory_map_block_skip_size)/size_of_block_index;
      
      // super map: [s1, s2, s3, ...]
      //       map: s1 -> [i1, i2, i3, ...]; s2 -> [...]; s3 -> [...]; ...
      // directory: i1 -> [data]; i2 -> [data]; i3 -> [data]; ... i1 -> [data]; ...
      
      // for each index in super map ...
      SYMS_U8 *out_ptr = directory;
      SYMS_U32 *super_map_ptr = directory_super_map;
      for (SYMS_U32 i = 0;
           i < block_count_in_directory_map;
           i += 1, super_map_ptr += 1){
        SYMS_U32 directory_map_block_index = *super_map_ptr;
        if (directory_map_block_index >= block_count_in_whole_file){
          // TODO(allen): File Defect: (Block Index, In Super Directory, Too Large)
          got_directory = syms_false;
          goto read_directory_done;
        }
        
        SYMS_U64 directory_map_block_off = (SYMS_U64)(directory_map_block_index)*block_size;
        SYMS_U8 *directory_map_block_base = data.str + directory_map_block_off;
        
        // clamp index count by end of directory
        SYMS_U32 index_count = 0;
        {
          SYMS_U32 directory_pos = (SYMS_U32)(out_ptr - directory);
          SYMS_U32 remaining_size = directory_size - directory_pos;
          SYMS_U32 remaining_map_block_count = SYMS_CeilIntegerDiv(remaining_size, block_size);
          index_count = SYMS_ClampTop(max_index_count_in_map_block, remaining_map_block_count);
        }
        
        // for each index in map ...
        SYMS_U8 *map_ptr = directory_map_block_base + directory_map_block_skip_size;
        for (SYMS_U32 j = 0;
             j < index_count;
             j += 1, map_ptr += size_of_block_index){
          SYMS_U32 directory_block_index = 0;
          syms_memmove(&directory_block_index, map_ptr, size_of_block_index);
          if (directory_block_index >= block_count_in_whole_file){
            // TODO(allen): File Defect: (Block Index, In Directory, Too Large)
            got_directory = syms_false;
            goto read_directory_done;
          }
          
          SYMS_U64 directory_block_off = (SYMS_U64)(directory_block_index)*block_size;
          SYMS_U8 *directory_block_base = data.str + directory_block_off;
          
          // clamp copy size by end of directory
          SYMS_U32 copy_size = 0;
          {
            SYMS_U32 directory_pos = (SYMS_U32)(out_ptr - directory);
            SYMS_U32 remaining_size = directory_size - directory_pos;
            copy_size = SYMS_ClampTop(block_size, remaining_size);
          }
          
          // copy block data
          syms_memmove(out_ptr, directory_block_base, copy_size);
          out_ptr += copy_size;
        }
      }
      
      read_directory_done:;
    }
    SYMS_U64Range directory_range = {0, header.directory_size};
    
    //- stream info
    SYMS_U32 stream_count = 0;
    SYMS_MsfAccelStreamInfo *stream_info = 0;
    {
      // read count
      syms_based_range_read(directory, directory_range, 0, 4, &stream_count);
      
      // allocate info array
      stream_info = syms_push_array_zero(arena, SYMS_MsfAccelStreamInfo, stream_count);
      
      // setup counts, sizes, and offsets for the directory data
      SYMS_U32 block_size = header.block_size;
      SYMS_U32 size_of_block_index = header.index_size;
      SYMS_U32 size_of_stream_size_entry = 4;
      if (size_of_block_index == 2){
        size_of_stream_size_entry = 8;
      }
      SYMS_U32 all_sizes_off = 4;
      SYMS_U32 all_indices_off = all_sizes_off + stream_count*size_of_stream_size_entry;
      
      // iterate sizes and indices in lock step
      SYMS_U32 size_cursor = all_sizes_off;
      SYMS_U32 index_cursor = all_indices_off;
      SYMS_MsfAccelStreamInfo *stream_info_ptr = stream_info;
      for (SYMS_U32 i = 0; i < stream_count; i += 1){
        // read stream size
        SYMS_U32 stream_size = 0;
        syms_based_range_read(directory, directory_range, size_cursor, 4, &stream_size);
        if (stream_size == 0xffffffff){
          stream_size = 0;
        }
        
        // compute block count
        SYMS_U32 stream_block_count = SYMS_CeilIntegerDiv(stream_size, block_size);
        
        // save stream info
        stream_info_ptr->stream_indices = directory + index_cursor;
        stream_info_ptr->size = stream_size;
        
        // advance cursors
        size_cursor += size_of_stream_size_entry;
        index_cursor += stream_block_count*size_of_block_index;
        stream_info_ptr += 1;
      }
    }
    
    // fill result
    result = syms_push_array(arena, SYMS_MsfAccel, 1);
    result->header = header;
    result->stream_count = stream_count;
    result->stream_info = stream_info;
  }
  
  return(result);
}

SYMS_API SYMS_MsfAccel*
syms_msf_deep_copy(SYMS_Arena *arena, SYMS_MsfAccel *msf){
  SYMS_ProfBegin("syms_msf_deep_copy");
  SYMS_MsfAccel *result = syms_push_array(arena, SYMS_MsfAccel, 1);
  syms_memmove(result, msf, sizeof(*result));
  result->stream_info = syms_push_array(arena, SYMS_MsfAccelStreamInfo, result->stream_count);
  syms_memmove(result->stream_info, msf->stream_info, sizeof(SYMS_MsfAccelStreamInfo)*result->stream_count);
  SYMS_ProfEnd();
  return(result);
}

SYMS_API SYMS_MsfAccel*
syms_msf_accel_dummy_from_raw_data(SYMS_Arena *arena, SYMS_String8 data){
  // setup header data
  SYMS_MsfHeaderInfo header = {0};
  header.index_size = 4;
  header.block_size = data.size;
  header.block_count = 1;
  header.directory_size = 0;
  header.directory_super_map = 0;
  
  // setup stream info
  SYMS_MsfAccelStreamInfo *stream_info = syms_push_array(arena, SYMS_MsfAccelStreamInfo, 1);
  stream_info->stream_indices = syms_push_array_zero(arena, SYMS_U8, 4);
  stream_info->size = data.size;
  
  // fill result
  SYMS_MsfAccel *result = syms_push_array(arena, SYMS_MsfAccel, 1);
  result->header = header;
  result->stream_count = 1;
  result->stream_info = stream_info;
  
  return(result);
}


////////////////////////////////
//~ allen: MSF Reader Fundamentals With Accelerator

SYMS_API SYMS_MsfHeaderInfo
syms_msf_header_info_from_msf(SYMS_MsfAccel *msf){
  return(msf->header);
}

SYMS_API SYMS_U32
syms_msf_get_stream_count(SYMS_MsfAccel *msf){
  return(msf->stream_count);
}

SYMS_API SYMS_MsfStreamInfo
syms_msf_stream_info_from_sn(SYMS_MsfAccel *msf, SYMS_MsfStreamNumber sn){
  // scan for stream in directory
  SYMS_MsfStreamInfo result = {0};
  if (sn < msf->stream_count){
    SYMS_MsfAccelStreamInfo *accel_stream_info = &msf->stream_info[sn];
    result.sn = sn;
    result.stream_indices = accel_stream_info->stream_indices;
    result.size = accel_stream_info->size;
  }
  return(result);
}

SYMS_API SYMS_B32
syms_msf_bounds_check(SYMS_MsfAccel *msf, SYMS_MsfStreamNumber sn, SYMS_U32 off){
  SYMS_B32 result = syms_false;
  SYMS_MsfStreamInfo stream_info = syms_msf_stream_info_from_sn(msf, sn);
  if (off <= stream_info.size){
    result = syms_true;
  }
  return(result);
}

SYMS_API SYMS_B32
syms_msf_read(SYMS_String8 data, SYMS_MsfAccel *msf,
              SYMS_MsfStreamNumber sn, SYMS_U32 off, SYMS_U32 size, void *out){
  SYMS_B32 result = syms_false;
  
  // stream info
  SYMS_MsfStreamInfo stream_info = syms_msf_stream_info_from_sn(msf, sn);
  if (size > 0 && off + size <= stream_info.size){
    // copy block-by-block
    SYMS_U64 file_size = data.size;
    SYMS_U32 block_size = msf->header.block_size;
    SYMS_U32 size_of_block_index = msf->header.index_size;
    SYMS_U32 block_count_in_whole_file_max = SYMS_CeilIntegerDiv(file_size, (SYMS_U64)block_size);
    SYMS_U32 block_count_in_whole_file = SYMS_ClampTop(msf->header.block_count, block_count_in_whole_file_max);
    SYMS_U32 block_count_in_stream = SYMS_CeilIntegerDiv((SYMS_U64)stream_info.size, (SYMS_U64)block_size);
    
    SYMS_U32 completed_amount = 0;
    for (;;){
      // offset-in-stream -> part-index
      SYMS_U32 remaining_off = off + completed_amount;
      SYMS_U32 src_part_index = remaining_off/block_size;
      SYMS_ASSERT(src_part_index < block_count_in_stream);
      
      // part-index -> block-index
      SYMS_U32 src_block_index_off = src_part_index*size_of_block_index;
      SYMS_U32 src_block_index = 0;
      syms_memmove(&src_block_index, stream_info.stream_indices + src_block_index_off, size_of_block_index);
      if (src_block_index >= block_count_in_whole_file){
        // TODO(allen): File Defect: (Block Index, In Stream, Too Large)
        break;
      }
      
      //  block-index -> range-in-file
      SYMS_U64 src_block_base = (SYMS_U64)(src_block_index)*block_size;
      SYMS_U32 relative_off = remaining_off%block_size;
      SYMS_U32 relative_opl = SYMS_ClampTop(relative_off + size - completed_amount, block_size);
      SYMS_U32 contiguous_size = relative_opl - relative_off;
      SYMS_ASSERT(src_block_base + relative_opl <= file_size);
      
      // copy & advance
      syms_memmove((SYMS_U8*)out + completed_amount, data.str + src_block_base + relative_off, contiguous_size);
      completed_amount += contiguous_size;
      if (completed_amount >= size){
        result = syms_true;
        break;
      }
    }
  }
  
  return(result);
}

SYMS_API SYMS_MsfRange
syms_msf_make_range(SYMS_MsfStreamNumber sn, SYMS_U32 off, SYMS_U32 len){
  SYMS_MsfRange result = {sn, off, len};
  return(result);
}

SYMS_API SYMS_MsfRange
syms_msf_range_from_sn(SYMS_MsfAccel *msf, SYMS_MsfStreamNumber sn){
  SYMS_MsfStreamInfo info = syms_msf_stream_info_from_sn(msf, sn);
  SYMS_MsfRange result = {0};
  if (info.size != 0){
    result.sn = info.sn;
    result.size = info.size;
  }
  return(result);
}

SYMS_API SYMS_MsfRange
syms_msf_sub_range(SYMS_MsfRange range, SYMS_U32 off, SYMS_U32 size){
  SYMS_MsfRange result = {0};
  if (off + size <= range.size){
    result.sn = range.sn;
    result.off = range.off + off;
    result.size = size;
  }
  return(result);
}

SYMS_API SYMS_MsfRange
syms_msf_sub_range_from_off_range(SYMS_MsfRange range, SYMS_U32Range off_range){
  SYMS_MsfRange result = syms_msf_sub_range(range, off_range.min, off_range.max - off_range.min);
  return(result);
}

////////////////////////////////
//~ allen: MSF Reader Range Helper Functions

SYMS_API SYMS_B32
syms_msf_bounds_check_in_range(SYMS_MsfRange range, SYMS_U32 off){
  SYMS_B32 result = (off <= range.size);
  return(result);
}

SYMS_API SYMS_B32
syms_msf_read_in_range(SYMS_String8 data, SYMS_MsfAccel *msf, SYMS_MsfRange range,
                       SYMS_U32 off, SYMS_U32 size, void *out){
  SYMS_B32 result = syms_false;
  if (off + size <= range.size){
    result = syms_msf_read(data, msf, range.sn, range.off + off, size, out);
  }
  return(result);
}

SYMS_API SYMS_String8
syms_msf_read_whole_range(SYMS_Arena *arena, SYMS_String8 data, SYMS_MsfAccel *msf, SYMS_MsfRange range){
  SYMS_String8 result = {0};
  result.str = syms_push_array(arena, SYMS_U8, range.size);
  result.size = range.size;
  syms_msf_read(data, msf, range.sn, range.off, range.size, result.str);
  return(result);
}

SYMS_API SYMS_String8
syms_msf_read_zstring_in_range(SYMS_Arena *arena, SYMS_String8 data, SYMS_MsfAccel *msf,
                               SYMS_MsfRange range, SYMS_U32 r_off){
  SYMS_ArenaTemp scratch = syms_get_scratch(&arena, 1);
  
  // build a list of string chunks on scratch
  SYMS_String8List list = {0};
  
  SYMS_U32 off = range.off + r_off;
  SYMS_U32 max_off = range.off + range.size;
  
  SYMS_MsfStreamInfo stream_info = syms_msf_stream_info_from_sn(msf, range.sn);
  if (off < stream_info.size && off < max_off){
    // scan block-by-block
    SYMS_U64 file_size = data.size;
    SYMS_U32 block_size = msf->header.block_size;
    SYMS_U32 size_of_block_index = msf->header.index_size;
    SYMS_U32 block_count_in_whole_file_max = SYMS_CeilIntegerDiv(file_size, block_size);
    SYMS_U32 block_count_in_whole_file = SYMS_ClampTop(msf->header.block_count, block_count_in_whole_file_max);
    SYMS_U32 block_count_in_stream = SYMS_CeilIntegerDiv(stream_info.size, block_size);
    
    SYMS_U32 max_scan_amount = max_off - off;
    SYMS_U32 scanned_amount = 0;
    for (;;){
      // offset-in-stream -> part-index
      SYMS_U32 remaining_off = off + scanned_amount;
      SYMS_U32 src_part_index = remaining_off/block_size;
      if (src_part_index >= block_count_in_stream){
        break;
      }
      
      // part-index -> block-index
      SYMS_U32 src_block_index_off = src_part_index*size_of_block_index;
      SYMS_U32 src_block_index = 0;
      syms_memmove(&src_block_index, stream_info.stream_indices + src_block_index_off, size_of_block_index);
      if (src_block_index >= block_count_in_whole_file){
        // TODO(allen): File Defect: (Block Index, In Stream, Too Large)
        break;
      }
      if (src_block_index >= block_count_in_whole_file){
        // TODO(allen): Detected defect in the file data.
        //  (Block Index, In Stream, Too Large)
        break;
      }
      
      // block-index -> range-in-file
      SYMS_U64 src_block_base = (SYMS_U64)(src_block_index)*block_size;
      SYMS_U32 relative_off = remaining_off%block_size;
      SYMS_U32 remaining_max_size = max_off - remaining_off;
      SYMS_U32 relative_opl = SYMS_ClampTop(relative_off + remaining_max_size, block_size);
      SYMS_ASSERT(src_block_base + block_size <= file_size);
      
      // scan
      SYMS_U8 *start = data.str + src_block_base + relative_off;
      SYMS_U8 *opl = start + relative_opl - relative_off;
      SYMS_U8 *ptr = start;
      for (;ptr < opl && *ptr != 0; ptr += 1);
      scanned_amount += (SYMS_U64)(ptr - start);
      
      // emit chunk
      if (start < ptr){
        SYMS_String8 str_range = syms_str8_range(start, ptr);
        syms_string_list_push(scratch.arena, &list, str_range);
      }
      
      // check end of scan conditions
      if (ptr < opl){
        break;
      }
      if (scanned_amount >= max_scan_amount){
        break;
      }
    }
  }
  
  
  // join into single string
  SYMS_String8 result = syms_string_list_join(arena, &list, 0);
  syms_release_scratch(scratch);
  return(result);
}

#endif //SYMS_MSF_PARSER_C
