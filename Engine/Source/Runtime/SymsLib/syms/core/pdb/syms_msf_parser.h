// Copyright Epic Games, Inc. All Rights Reserved.
/* date = May 4th 2021 1:55 pm */

#ifndef SYMS_MSF_PARSER_H
#define SYMS_MSF_PARSER_H

////////////////////////////////
//~ allen: MSF Parser Helper Types

typedef struct SYMS_MsfHeaderInfo{
  SYMS_U32 index_size;
  SYMS_U32 block_size;
  SYMS_U32 block_count;
  SYMS_U32 directory_size;
  SYMS_U32 directory_super_map;
} SYMS_MsfHeaderInfo;

typedef struct SYMS_MsfStreamInfo{
  SYMS_MsfStreamNumber sn;
  SYMS_U8 *stream_indices;
  SYMS_U32 size;
} SYMS_MsfStreamInfo;

typedef struct SYMS_MsfRange{
  SYMS_MsfStreamNumber sn;
  SYMS_U32 off;
  SYMS_U32 size;
}  SYMS_MsfRange;

////////////////////////////////
//~ allen: MSF Parser Accelerator

typedef struct SYMS_MsfAccelStreamInfo{
  SYMS_U8 *stream_indices;
  SYMS_U32 size;
} SYMS_MsfAccelStreamInfo;

typedef struct SYMS_MsfAccel{
  SYMS_MsfHeaderInfo header;
  SYMS_U32 stream_count;
  SYMS_MsfAccelStreamInfo *stream_info;
} SYMS_MsfAccel;

////////////////////////////////
//~ allen: MSF Reader Fundamentals Without Accelerator

SYMS_API SYMS_MsfHeaderInfo syms_msf_header_info_from_data_slow(SYMS_String8 data);

////////////////////////////////
//~ allen: MSF Reader Accelerator Constructor

SYMS_API SYMS_MsfAccel* syms_msf_accel_from_data(SYMS_Arena *arena, SYMS_String8 data);
SYMS_API SYMS_MsfAccel* syms_msf_deep_copy(SYMS_Arena *arena, SYMS_MsfAccel *msf);

SYMS_API SYMS_MsfAccel* syms_msf_accel_dummy_from_raw_data(SYMS_Arena *arena, SYMS_String8 data);

////////////////////////////////
//~ allen: MSF Reader Fundamentals With Accelerator

SYMS_API SYMS_MsfHeaderInfo syms_msf_header_info_from_msf(SYMS_MsfAccel *msf);

SYMS_API SYMS_U32 syms_msf_get_stream_count(SYMS_MsfAccel *msf);

SYMS_API SYMS_MsfStreamInfo syms_msf_stream_info_from_sn(SYMS_MsfAccel *msf, SYMS_MsfStreamNumber sn);

SYMS_API SYMS_B32 syms_msf_bounds_check(SYMS_MsfAccel *msf, SYMS_MsfStreamNumber sn, SYMS_U32 off);
SYMS_API SYMS_B32 syms_msf_read(SYMS_String8 data, SYMS_MsfAccel *msf,
                                SYMS_MsfStreamNumber sn, SYMS_U32 off, SYMS_U32 size, void *out);

#define syms_msf_read_struct(d,a,sn,off,p) syms_msf_read((d),(a),(sn),(off),sizeof(*(p)),(p))

SYMS_API SYMS_MsfRange syms_msf_make_range(SYMS_MsfStreamNumber sn, SYMS_U32 off, SYMS_U32 len);
SYMS_API SYMS_MsfRange syms_msf_range_from_sn(SYMS_MsfAccel *msf, SYMS_MsfStreamNumber sn);
SYMS_API SYMS_MsfRange syms_msf_sub_range(SYMS_MsfRange range, SYMS_U32 off, SYMS_U32 size);
SYMS_API SYMS_MsfRange syms_msf_sub_range_from_off_range(SYMS_MsfRange range, SYMS_U32Range off_range);

////////////////////////////////
//~ allen: MSF Reader Range Helper Functions

SYMS_API SYMS_B32 syms_msf_bounds_check_in_range(SYMS_MsfRange range, SYMS_U32 off);
SYMS_API SYMS_B32 syms_msf_read_in_range(SYMS_String8 data, SYMS_MsfAccel *msf,
                                         SYMS_MsfRange range, SYMS_U32 off, SYMS_U32 size, void *out);

SYMS_API SYMS_String8 syms_msf_read_whole_range(SYMS_Arena *arena, SYMS_String8 data, SYMS_MsfAccel *msf,
                                                SYMS_MsfRange range);

#define syms_msf_read_struct_in_range(d,a,rn,off,p) syms_msf_read_in_range((d),(a),(rn),(off),sizeof(*(p)),(p))

SYMS_API SYMS_String8 syms_msf_read_zstring_in_range(SYMS_Arena *arena, SYMS_String8 data, SYMS_MsfAccel *msf,
                                                     SYMS_MsfRange range, SYMS_U32 off);

#endif //SYMS_MSF_PARSER_H
