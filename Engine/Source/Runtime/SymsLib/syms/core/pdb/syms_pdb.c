// Copyright Epic Games, Inc. All Rights Reserved.

#ifndef SYMS_PDB_C
#define SYMS_PDB_C

////////////////////////////////
//~ allen: PDB Hash Function

SYMS_API SYMS_U32 syms_pdb_hashV1(SYMS_String8 string);

SYMS_API SYMS_U32
syms_pdb_hashV1(SYMS_String8 string){
  SYMS_U32 result = 0;
  SYMS_U8 *ptr = string.str;
  SYMS_U8 *opl = ptr + (string.size&(~3));
  for (; ptr < opl; ptr += 4){ result ^= *(SYMS_U32*)ptr; }
  if ((string.size&2) != 0){ result ^= *(SYMS_U16*)ptr; ptr += 2; }
  if ((string.size&1) != 0){ result ^= *           ptr; }
  result |= 0x20202020; result ^= (result >> 11); result ^= (result >> 16);
  return(result);
}

#endif // SYMS_PDB_C
