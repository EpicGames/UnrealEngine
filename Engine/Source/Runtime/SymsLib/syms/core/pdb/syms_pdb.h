// Copyright Epic Games, Inc. All Rights Reserved.
/* date = May 4th 2021 1:54 pm */

#ifndef SYMS_PDB_H
#define SYMS_PDB_H

// NOTE(allen): Definition of PDB format types.
// PDB includes DBI - Debug Information; and CV - Code View; formats
// As best as possible we have tried to name and organize our
// definitions to clearly show where they come from in the original
// definitions of PDB given by:
//  https://github.com/microsoft/microsoft-pdb

// NOTE(allen): When you see: 'This is not "literally" defined by the format.'
// What we are saying is there is no equivalent type in the original definition.
// But it is a useful helper to us that is not able to be rearranged because it
// reflects some kind of structure in the format.

////////////////////////////////
//~ allen: PDB Format Types

typedef SYMS_U32 SYMS_PdbVersion;
enum{
  SYMS_PdbVersion_VC2      = 19941610,
  SYMS_PdbVersion_VC4      = 19950623,
  SYMS_PdbVersion_VC41     = 19950814,
  SYMS_PdbVersion_VC50     = 19960307,
  SYMS_PdbVersion_VC98     = 19970604,
  SYMS_PdbVersion_VC70_DEP = 19990604,
  SYMS_PdbVersion_VC70     = 20000404,
  SYMS_PdbVersion_VC80     = 20030901,
  SYMS_PdbVersion_VC110    = 20091201,
  SYMS_PdbVersion_VC140    = 20140508
};

typedef SYMS_U16 SYMS_PdbModIndex;
typedef SYMS_U32 SYMS_PdbStringIndex;

typedef struct SYMS_PdbOffAndSize{
  SYMS_U32 off;
  SYMS_U32 size;
} SYMS_PdbOffAndSize;

typedef enum SYMS_PdbFixedStream{
  SYMS_PdbFixedStream_PDB = 1,
  SYMS_PdbFixedStream_TPI = 2,
  SYMS_PdbFixedStream_DBI = 3,
  SYMS_PdbFixedStream_IPI = 4
} SYMS_PdbFixedStream;

//- "PDB" fixed stream contains named streams

typedef enum SYMS_PdbNamedStream{
  SYMS_PdbNamedStream_HEADER_BLOCK,
  SYMS_PdbNamedStream_STRTABLE,
  SYMS_PdbNamedStream_LINK_INFO,
  SYMS_PdbNamedStream_COUNT
} SYMS_PdbNamedStream;

typedef struct SYMS_PdbInfoHeader{
  SYMS_PdbVersion version;
  SYMS_U32 time;
  SYMS_U32 age;
} SYMS_PdbInfoHeader;

////////////////////////////////
//~ allen: PDB DBI Format Types

//- "STRTABLE" named stream contains a string table

enum{
  SYMS_PdbDbiStrTableHeader_MAGIC = 0xEFFEEFFE
};

typedef struct SYMS_PdbDbiStrTableHeader{
  SYMS_U32 magic;
  SYMS_U32 version;
} SYMS_PdbDbiStrTableHeader;

//- "DBI" fixed stream contains all of this 'main dbi' stuff

typedef SYMS_U32 SYMS_PdbDbiStream;
enum{
  SYMS_PdbDbiStream_FPO,
  SYMS_PdbDbiStream_EXCEPTION,
  SYMS_PdbDbiStream_FIXUP,
  SYMS_PdbDbiStream_OMAP_TO_SRC,
  SYMS_PdbDbiStream_OMAP_FROM_SRC,
  SYMS_PdbDbiStream_SECTION_HEADER,
  SYMS_PdbDbiStream_TOKEN_RDI_MAP,
  SYMS_PdbDbiStream_XDATA,
  SYMS_PdbDbiStream_PDATA,
  SYMS_PdbDbiStream_NEW_FPO,
  SYMS_PdbDbiStream_SECTION_HEADER_ORIG,
  SYMS_PdbDbiStream_COUNT
};

typedef SYMS_U32 SYMS_PdbDbiHeaderSig;
enum{
  SYMS_PdbDbiHeaderSignature_V1 = 0xFFFFFFFF
};

typedef SYMS_U32 SYMS_PdbDbiVersion;
enum{
  SYMS_PdbDbiVersion_41  =   930803,
  SYMS_PdbDbiVersion_50  = 19960307,
  SYMS_PdbDbiVersion_60  = 19970606,
  SYMS_PdbDbiVersion_70  = 19990903,
  SYMS_PdbDbiVersion_110 = 20091201,
};

typedef SYMS_U16 SYMS_PdbDbiBuildNumber;
#define SYMS_PdbDbiBuildNumberNewFormatFlag 0x8000
#define SYMS_PdbDbiBuildNumberMinor(bn) ((bn)&0xFF)
#define SYMS_PdbDbiBuildNumberMajor(bn) (((bn) >> 8)&0x7F)
#define SYMS_PdbDbiBuildNumberNewFormat(bn) (!!((bn)&SYMS_PdbDbiBuildNumberNewFormatFlag))
#define SYMS_PdbDbiBuildNumber(maj, min) \
((SYMS_PdbDbiBuildNumber)(SYMS_PdbDbiBuildNumberNewFormatFlag | ((min)&0xFF) | (((maj)&0x7F) << 16)))

typedef SYMS_U16 SYMS_PdbDbiHeaderFlags;
enum{
  SYMS_PdbDbiHeaderFlag_INCREMENTAL = 0x1,
  SYMS_PdbDbiHeaderFlag_STRIPPED    = 0x2,
  SYMS_PdbDbiHeaderFlag_CTYPES      = 0x4
};

typedef struct SYMS_PdbDbiHeader{
  SYMS_PdbDbiHeaderSig sig;
  SYMS_PdbDbiVersion version;
  SYMS_U32 age;
  SYMS_MsfStreamNumber gsi_sn;
  SYMS_PdbDbiBuildNumber build_number;
  
  SYMS_MsfStreamNumber psi_sn;
  SYMS_U16 pdb_version;
  
  SYMS_MsfStreamNumber sym_sn;
  SYMS_U16 pdb_version2;
  
  SYMS_U32 module_info_size;
  SYMS_U32 sec_con_size;
  SYMS_U32 sec_map_size;
  SYMS_U32 file_info_size;
  
  SYMS_U32 tsm_size;
  SYMS_U32 mfc_index;
  SYMS_U32 dbg_header_size;
  SYMS_U32 ec_info_size;
  
  SYMS_PdbDbiHeaderFlags flags;
  SYMS_U16 machine;
  
  SYMS_U32 reserved;
} SYMS_PdbDbiHeader;

// NOTE(allen): This is not "literally" defined by the format.
typedef enum SYMS_PdbDbiRange{
  SYMS_PdbDbiRange_ModuleInfo,
  SYMS_PdbDbiRange_SecCon,
  SYMS_PdbDbiRange_SecMap,
  SYMS_PdbDbiRange_FileInfo,
  SYMS_PdbDbiRange_TSM,
  SYMS_PdbDbiRange_EcInfo,
  SYMS_PdbDbiRange_DbgHeader,
  SYMS_PdbDbiRange_COUNT
} SYMS_PdbDbiRange;

//- "ModuleInfo" dbi range contains this compilation unit information

typedef SYMS_U32 SYMS_PdbDbiSectionContribVersion;
// NOTE(allen): these are too big for an enum on some compilers (they are interpreted as signed)
#define SYMS_PdbDbiSectionContribVersion_1 (0xeffe0000u + 19970605u)
#define SYMS_PdbDbiSectionContribVersion_2 (0xeffe0000u + 20140516u)

// any other version number
typedef struct SYMS_PdbDbiSectionContrib40{
  SYMS_CvSectionIndex sec;
  SYMS_U32 sec_off;
  SYMS_U32 size;
  SYMS_U32 flags;
  SYMS_PdbModIndex mod;
} SYMS_PdbDbiSectionContrib40;

// SYMS_PdbDbiPdbDbiSectionContribVersion_1
typedef struct SYMS_PdbDbiSectionContrib{
  SYMS_PdbDbiSectionContrib40 base;
  SYMS_U32 data_crc;
  SYMS_U32 reloc_crc;
} SYMS_PdbDbiSectionContrib;

// SYMS_PdbDbiPdbDbiSectionContribVersion_2
typedef struct SYMS_PdbDbiSectionContrib2{
  SYMS_PdbDbiSectionContrib40 base;
  SYMS_U32 data_crc;
  SYMS_U32 reloc_crc;
  SYMS_U32 sec_coff;
} SYMS_PdbDbiSectionContrib2;

// TODO(allen): changes when sc version changes to 40. (see MODI50)
// (MODI_60_Persist)
typedef struct SYMS_PdbDbiCompUnitHeader{
  SYMS_U8 unused[4];
  SYMS_PdbDbiSectionContrib section_contribution;
  // TODO(allen): type info on these flags.
  SYMS_U16 flags;
  
  SYMS_MsfStreamNumber sn;
  SYMS_U32 symbol_bytes;
  SYMS_U32 c11_lines_size;
  SYMS_U32 c13_lines_size;
  
  SYMS_U16 num_contrib_files;
  SYMS_U8 padding1[2];
  SYMS_U32 file_names_offset;
  
  SYMS_PdbStringIndex src_file;
  SYMS_PdbStringIndex pdb_file;
  
  // char module_name[]; - null terminated
  // char obj_name[]; - null terminated
} SYMS_PdbDbiCompUnitHeader;

// NOTE(allen): This is not "literally" defined by the format.
typedef enum SYMS_PdbCompUnitRange{
  SYMS_PdbCompUnitRange_Symbols,
  SYMS_PdbCompUnitRange_C11,
  SYMS_PdbCompUnitRange_C13,
  SYMS_PdbCompUnitRange_COUNT
} SYMS_PdbCompUnitRange;

////////////////////////////////
//~ allen: PDB TPI Format Types

// "TPI" and "IPI" fixed streams contain type/information in these structures

typedef SYMS_U32 SYMS_PdbTpiVersion;
enum{
  SYMS_PdbTpiVersion_INTV_VC2 = 920924,
  SYMS_PdbTpiVersion_IMPV40 = 19950410,
  SYMS_PdbTpiVersion_IMPV41 = 19951122,
  SYMS_PdbTpiVersion_IMPV50_INTERIM = 19960307,
  SYMS_PdbTpiVersion_IMPV50 = 19961031,
  SYMS_PdbTpiVersion_IMPV70 = 19990903,
  SYMS_PdbTpiVersion_IMPV80 = 20040203,
};

typedef struct SYMS_PdbTpiHeader{
  // (PDB/dbi/tpi.h: HDR)
  SYMS_PdbTpiVersion version;
  SYMS_U32 header_size;
  SYMS_U32 ti_lo;
  SYMS_U32 ti_hi;
  SYMS_U32 types_size;
  
  // (PDB/dbi/tpi.h: PdbTpiHash)
  SYMS_MsfStreamNumber hash_sn;
  SYMS_MsfStreamNumber hash_sn_aux;
  SYMS_U32 hash_key_size;
  SYMS_U32 hash_bucket_count;
  SYMS_PdbOffAndSize hash_vals;
  SYMS_PdbOffAndSize ti_off;
  SYMS_PdbOffAndSize hash_adj;
} SYMS_PdbTpiHeader;

typedef struct SYMS_PdbTpiOffHint{
  SYMS_CvTypeIndex ti;
  SYMS_U32 off;
} SYMS_PdbTpiOffHint;

////////////////////////////////
//~ allen: PDB GSI Format Types

// "gsi" and "psi" streams contain symbol lookup tables in these types

typedef SYMS_U32 SYMS_PdbGsiSignature;
enum{
  SYMS_PdbGsiSignature_Basic = 0xffffffff,
};

typedef SYMS_U32 SYMS_PdbGsiVersion;
enum{
  SYMS_PdbGsiVersion_V70 = 0xeffe0000 + 19990810,
};

typedef struct SYMS_PdbGsiHeader{
  SYMS_PdbGsiSignature sig;
  SYMS_PdbGsiVersion ver;
  SYMS_U32 hr_len;
  SYMS_U32 num_buckets;
} SYMS_PdbGsiHeader;

typedef struct SYMS_PdbGsiHashRecord{
  SYMS_U32 off; // Offset in the symbol record stream
  SYMS_U32 cref;
} SYMS_PdbGsiHashRecord;

// TODO(allen): ?
// NOTE(nick): This is a crutch that helps serialize in-memory GSI buckets that use 64bit pointers for next HR.
typedef struct SYMS_PdbGsiHrOffsetCalc{
  SYMS_U32 next;
  SYMS_U32 off;
  SYMS_U32 cref;
} SYMS_PdbGsiHrOffsetCalc;

typedef struct SYMS_PdbPsiHeader{
  SYMS_U32 sym_hash_size;
  SYMS_U32 addr_map_size;
  SYMS_U32 thunk_count;
  SYMS_U32 thunk_size;
  SYMS_CvSectionIndex isec_thunk_table;
  SYMS_U8 padding[2];
  SYMS_U32 sec_thunk_tabl_off;
  SYMS_U32 sec_count;
} SYMS_PdbPsiHeader;

////////////////////////////////
//~ allen: PDB Hash Function

SYMS_API SYMS_U32 syms_pdb_hashV1(SYMS_String8 string);

#endif //SYMS_PDB_H
