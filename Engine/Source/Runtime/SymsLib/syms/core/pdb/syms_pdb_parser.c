// Copyright Epic Games, Inc. All Rights Reserved.

#ifndef SYMS_PDB_PARSER_C
#define SYMS_PDB_PARSER_C

////////////////////////////////
//~ allen: PDB TPI Functions

SYMS_API SYMS_PdbTpiOffRange
syms_pdb_tpi__hint_from_index(SYMS_PdbTpiAccel *tpi, SYMS_CvTypeIndex ti){
  // b.s. rule: largest 'index' so that 'tpi->hints[index].ti <= ti'
  //            or, when that's impossible, 'index = tpi->count'
  SYMS_PdbTpiOffHint *hints = tpi->hints;
  SYMS_U32 index = tpi->count;
  SYMS_U32 min = 0;
  SYMS_U32 max = tpi->count;
  for (;;){
    SYMS_U32 mid = (min + max)/2;
    SYMS_PdbTpiOffHint *h = hints + mid;
    if (h->ti > ti){
      max = mid;
      if (max == 0){
        break;
      }
    }
    else{
      min = mid;
      if (min + 1 >= max){
        index = min;
        break;
      }
    }
  }
  
  // return result
  SYMS_PdbTpiOffRange result = {0};
  if (index < tpi->count){
    result.first_off = tpi->base_off + tpi->hints[index].off;
    result.first_ti = tpi->hints[index].ti;
    result.opl_ti = SYMS_U32_MAX;
    if (index + 1 < tpi->count){
      result.opl_ti = tpi->hints[index + 1].ti;
    }
  }
  return(result);
}

SYMS_API void
syms_pdb_tpi__fill_off_range(SYMS_String8 data, SYMS_MsfAccel *msf, SYMS_PdbTpiAccel *tpi,
                             SYMS_PdbTpiOffRange *fill){
  // fill params
  SYMS_U32         first_off = fill->first_off;
  SYMS_CvTypeIndex first_ti  = SYMS_ClampBot(tpi->first_ti, fill->first_ti);
  SYMS_CvTypeIndex opl_ti    = SYMS_ClampTop(fill->opl_ti, tpi->opl_ti);
  SYMS_MsfStreamNumber sn    = tpi->type_sn;
  
  // default fill to non-zero invalid off (all bytes 0xFF)
  //  we do this because we use zero to indicate "cache not filled"
  //  so "filled by invalid" needs another value.
  SYMS_U32 *type_off_ptr = tpi->off + first_ti - tpi->first_ti;
  syms_memset(type_off_ptr, 0xFF, sizeof(*type_off_ptr)*(opl_ti - first_ti));
  
  // fill
  SYMS_U32 cursor = first_off;
  for (SYMS_CvTypeIndex ti = first_ti;
       ti < opl_ti;
       ti += 1, type_off_ptr += 1){
    // save off
    *type_off_ptr = cursor;
    // read size
    SYMS_U16 size = 0;
    syms_msf_read_struct(data, msf, sn, cursor, &size);
    // advance
    cursor += 2 + size;
    if (size == 0 || !syms_msf_bounds_check(msf, sn, cursor)){
      break;
    }
  }
}

SYMS_API SYMS_PdbTpiAccel
syms_pdb_tpi_accel_from_sn(SYMS_Arena *arena, SYMS_String8 data, SYMS_MsfAccel *msf, SYMS_MsfStreamNumber sn){
  SYMS_ProfBegin("syms_pdb_tpi_accel_from_sn");
  
  // init data structures
  SYMS_PdbTpiAccel result = {0};
  
  SYMS_PdbTpiHeader hdr = {0};
  syms_msf_read_struct(data, msf, sn, 0, &hdr);
  if (hdr.version == SYMS_PdbTpiVersion_IMPV80){ // NOTE(allen): the only version we support so far.
    
    // parse hash table
    SYMS_PdbChain **buckets = 0;
    SYMS_U32 bucket_count = 0;
    
    if (hdr.hash_bucket_count != 0){
      // grab scratch
      SYMS_ArenaTemp scratch = syms_get_scratch(&arena, 1);
      
      // setup buckets
      bucket_count = hdr.hash_bucket_count;
      buckets = syms_push_array(arena, SYMS_PdbChain*, bucket_count);
      syms_memset(buckets, 0, sizeof(*buckets)*bucket_count);
      
      // grab hash array
      SYMS_MsfRange hash_range = syms_msf_range_from_sn(msf, hdr.hash_sn);
      SYMS_MsfRange range = syms_msf_sub_range(hash_range, hdr.hash_vals.off, hdr.hash_vals.size);
      SYMS_String8 hash_array = syms_msf_read_whole_range(scratch.arena, data, msf, range);
      
      // iterate hash vals
      SYMS_U32 stride = hdr.hash_key_size;
      SYMS_U8 *cursor = hash_array.str;
      SYMS_U8 *opl_cursor = hash_array.str + hash_array.size;
      SYMS_CvTypeIndex ti = hdr.ti_lo;
      for (;;){
        // read index
        SYMS_U32 bucket_index = 0;
        syms_memmove(&bucket_index, cursor, stride);
        
        // save to typemap
        SYMS_PdbChain *entry = 0;
        if (bucket_index < bucket_count){
          entry = syms_push_array(arena, SYMS_PdbChain, 1);
          syms_memzero_struct(entry);
          SYMS_StackPush(buckets[bucket_index], entry);
          entry->v = ti;
        }
        
        // advance cursor
        cursor += stride;
        ti += 1;
        
        // exit condition
        if (cursor >= opl_cursor){
          break;
        }
      }
      
      // release scratch
      syms_release_scratch(scratch);
    }
    
    // parse hint table
    SYMS_PdbTpiOffHint *hints = 0;
    SYMS_U32 hint_count = 0;
    {
      SYMS_MsfRange hash_range = syms_msf_range_from_sn(msf, hdr.hash_sn);
      SYMS_MsfRange range = syms_msf_sub_range(hash_range, hdr.ti_off.off, hdr.ti_off.size);
      SYMS_String8 hint_memory = syms_msf_read_whole_range(arena, data, msf, range);
      hints = (SYMS_PdbTpiOffHint*)hint_memory.str;
      hint_count = (hint_memory.size)/sizeof(*hints);
    }
    
    // offset array
    SYMS_CvTypeIndex first_ti = 0;
    SYMS_CvTypeIndex opl_ti = 0;
    SYMS_U32 *off = 0;
    {
      first_ti = hdr.ti_lo;
      opl_ti   = SYMS_ClampBot(first_ti, hdr.ti_hi);
      
      SYMS_U64 count = opl_ti - first_ti;
      off = syms_push_array_zero(arena, SYMS_U32, count);
    }
    
    // fill result
    result.type_sn  = sn;
    result.buckets  = buckets;
    result.bucket_count = bucket_count;
    result.count    = hint_count;
    result.hints    = hints;
    result.base_off = hdr.header_size;
    result.first_ti = first_ti;
    result.opl_ti   = opl_ti;
    result.off      = off;
  }
  
  SYMS_ProfEnd();
  
  return(result);
}

SYMS_API SYMS_U32
syms_pdb_tpi_off_from_ti(SYMS_String8 data, SYMS_MsfAccel *msf, SYMS_PdbTpiAccel *tpi,
                         SYMS_CvTypeIndex ti){
  SYMS_U32 result = 0;
  if (tpi->first_ti <= ti && ti < tpi->opl_ti){
    // read off
    SYMS_U32 relative_index = ti - tpi->first_ti;
    SYMS_U32 off = tpi->off[relative_index];
    
    // if not zero this is the result
    if (off != 0){
      result = off;
    }
    
    // otherwise compute and cache this off range
    else{
      SYMS_PdbTpiOffRange hint = syms_pdb_tpi__hint_from_index(tpi, ti);
      syms_pdb_tpi__fill_off_range(data, msf, tpi, &hint);
      result = tpi->off[relative_index];
    }
    
    // collapse "error cases" down to null
    if (result == SYMS_U32_MAX){
      result = 0;
    }
  }
  return(result);
}

SYMS_API SYMS_U32
syms_pdb_tpi_base_off(SYMS_PdbTpiAccel *tpi){
  return(tpi->base_off);
}

SYMS_API SYMS_MsfRange
syms_pdb_tpi_range(SYMS_MsfAccel *msf, SYMS_PdbTpiAccel *map){
  SYMS_MsfRange range = syms_msf_range_from_sn(msf, map->type_sn);
  SYMS_MsfRange result = syms_msf_sub_range(range, map->base_off, range.size - map->base_off);
  return(result);
}

SYMS_API SYMS_USIDList
syms_pdb_types_from_name(SYMS_Arena *arena, SYMS_String8 data, SYMS_PdbDbgAccel *dbg,
                         SYMS_CvUnitAccel *unit, SYMS_String8 name){
  // setup accel
  SYMS_PdbTpiAccel *tpi = &dbg->tpi;
  
  SYMS_USIDList result = {0};
  if (tpi->bucket_count > 0){
    
    // get uid
    SYMS_UnitID uid = unit->uid;
    
    // get bucket
    SYMS_U32 name_hash = syms_pdb_hashV1(name);
    SYMS_U32 bucket_index = name_hash%tpi->bucket_count;
    
    // iterate bucket
    for (SYMS_PdbChain *bucket = tpi->buckets[bucket_index];
         bucket != 0;
         bucket = bucket->next){
      SYMS_CvTypeIndex ti = bucket->v;
      if (unit->top_min_index <= ti){
        SYMS_U64 index = ti - unit->top_min_index;
        if (index < unit->ti_count){
          SYMS_CvStub *stub = unit->ti_indirect_stubs[index];
          if (stub != 0 && syms_string_match(stub->name, name, 0)){
            SYMS_USIDNode *usid_node = syms_push_array(arena, SYMS_USIDNode, 1);
            SYMS_QueuePush(result.first, result.last, usid_node);
            result.count += 1;
            usid_node->usid.uid = uid;
            usid_node->usid.sid = SYMS_ID_u32_u32(SYMS_CvSymbolIDKind_Index, ti);
          }
        }
      }
    }
  }
  
  return(result);
}

////////////////////////////////
//~ allen: PDB GSI

SYMS_API SYMS_PdbGsiAccel
syms_pdb_gsi_accel_from_range(SYMS_Arena *arena, SYMS_String8 data, SYMS_MsfAccel *msf, SYMS_MsfRange range){
  SYMS_ProfBegin("syms_pdb_parse_info");
  
  // get scratch
  SYMS_ArenaTemp scratch = syms_get_scratch(&arena, 1);
  // check gsi info
  SYMS_B32 has_gsi = syms_false;
  SYMS_PdbGsiHeader header = {0};
  if (syms_msf_read_struct_in_range(data, msf, range, 0, &header)){
    if (header.sig == SYMS_PdbGsiSignature_Basic &&
        header.ver == SYMS_PdbGsiVersion_V70 &&
        header.num_buckets != 0){
      has_gsi = syms_true;
    }
  }
  
  // hash offset
  SYMS_U32 hash_record_array_off = sizeof(header);
  
  // bucket count
  SYMS_U32 bucket_count = 4096;
  SYMS_U32 slot_count = bucket_count + 1;
  
  // get unpacked offsets
  SYMS_U32 *unpacked_offsets = 0;
  if (has_gsi){
    unpacked_offsets = syms_push_array(scratch.arena, SYMS_U32, slot_count);
    
    SYMS_ArenaTemp temp = syms_arena_temp_begin(scratch.arena);
    
    // array offsets
    SYMS_U32 bitmask_count = ((slot_count + 31)/32);
    SYMS_U32 bitmask_size = bitmask_count*4;
    SYMS_U32 bitmask_off = hash_record_array_off + header.hr_len;
    SYMS_U32 offsets_off = bitmask_off + bitmask_size;
    
    // read bitmask
    SYMS_MsfRange bitmask_range = syms_msf_sub_range(range, bitmask_off, bitmask_size);
    SYMS_String8 bitmask = syms_msf_read_whole_range(scratch.arena, data, msf, bitmask_range);
    SYMS_U32 bitmask_count_clamped = bitmask.size/4;
    
    // read packed offsets
    SYMS_MsfRange offsets_range = syms_msf_sub_range(range, offsets_off, range.size - offsets_off);
    SYMS_String8 packed_offsets = syms_msf_read_whole_range(scratch.arena, data, msf, offsets_range);
    SYMS_U32 packed_offsets_count = packed_offsets.size/4;
    
    // unpack offsets
    SYMS_U32 *bitmask_ptr = (SYMS_U32*)(bitmask.str);
    SYMS_U32 *bitmask_opl = bitmask_ptr + bitmask_count_clamped;
    SYMS_U32 *dst_ptr = unpacked_offsets;
    SYMS_U32 *dst_opl = dst_ptr + slot_count;
    SYMS_U32 *src_ptr = (SYMS_U32*)packed_offsets.str;
    SYMS_U32 *src_opl = src_ptr + packed_offsets_count;
    for (; bitmask_ptr < bitmask_opl && src_ptr < src_opl; bitmask_ptr += 1){
      SYMS_U32 bits = *bitmask_ptr;
      SYMS_U32 dst_max = (SYMS_U32)(dst_opl - dst_ptr);
      SYMS_U32 src_max = (SYMS_U32)(src_opl - src_ptr);
      SYMS_U32 k_max0 = SYMS_ClampTop(32, dst_max);
      SYMS_U32 k_max = SYMS_ClampTop(k_max0, src_max);
      for (SYMS_U32 k = 0; k < k_max; k += 1){
        if ((bits & 1) == 1){
          *dst_ptr = *src_ptr;
          src_ptr += 1;
        }
        else{
          *dst_ptr = SYMS_U32_MAX;
        }
        dst_ptr += 1;
        bits >>= 1;
      }
    }
    for (; dst_ptr < dst_opl; dst_ptr += 1){
      *dst_ptr = SYMS_U32_MAX;
    }
    
    syms_arena_temp_end(temp);
  }
  
  // construct table
  SYMS_PdbChain **table = 0;
  SYMS_B32 table_build_success = syms_true;
  if (unpacked_offsets != 0){
    table = syms_push_array(arena, SYMS_PdbChain*, bucket_count + 1);
    
    // read hash records
    SYMS_MsfRange hash_record_range = syms_msf_sub_range(range, hash_record_array_off, header.hr_len);
    SYMS_String8 hash_records = syms_msf_read_whole_range(scratch.arena, data, msf, hash_record_range);
    SYMS_U32 entry_count = hash_records.size/sizeof(SYMS_PdbGsiHashRecord);
    
    // build bucket chains
    // NOTE(allen):  we have to do this backwards to make the singly linked
    // stacks end up in the right order, and because of the way groups of
    // offsets are stored by PDB (it is specified this way on purpose)
    SYMS_U32 last_index = entry_count - 1;
    SYMS_PdbGsiHashRecord *in_ptr  = (SYMS_PdbGsiHashRecord*)(hash_records.str) + last_index;
    SYMS_U32 prev_n = entry_count;
    for (SYMS_U32 i = bucket_count + 1; i > 0;){
      i -= 1;
      SYMS_PdbChain *table_ptr = 0;
      if (unpacked_offsets[i] != SYMS_U32_MAX){
        SYMS_U32 n = unpacked_offsets[i]/sizeof(SYMS_PdbGsiHrOffsetCalc);
        if (n > prev_n){
          table_build_success = syms_false;
          break;
        }
        SYMS_U32 num_steps = prev_n - n;
        for (SYMS_U32 j = 0; j < num_steps; j += 1){
          SYMS_PdbChain *bucket = syms_push_array(arena, SYMS_PdbChain, 1);
          bucket->next = 0;
          bucket->v = in_ptr->off - 1; 
          SYMS_StackPush(table_ptr, bucket);
          in_ptr -= 1;
        }
        prev_n = n;
      }
      table[i] = table_ptr;
    }
  }
  
  // build the result
  SYMS_PdbGsiAccel result = {0};
  if (table_build_success){
    result.buckets = table;
    result.bucket_count = bucket_count;
  }
  
  // release scratch
  syms_release_scratch(scratch);
  
  SYMS_ProfEnd();
  
  return(result);
}

SYMS_API SYMS_MsfRange
syms_pdb_gsi_part_from_psi_range(SYMS_MsfRange psi_range){
  // NOTE(allen): We don't actually use the PSI specific information right now,
  // so it's basically just a wrapper around GSI. We didn't collapse it down to
  // one system because there might be reasons to start using these extra features
  // and then this will need to be a wrapper.
#if 0
  // psi header
  SYMS_PdbPsiHeader header = {0};
  syms_msf_read_struct_in_range(data, accel, psi_range, 0, &header);
#endif
  
  SYMS_MsfRange result = syms_msf_sub_range(psi_range, sizeof(SYMS_PdbPsiHeader),
                                            psi_range.size - sizeof(SYMS_PdbPsiHeader));
  return(result);
}

SYMS_API SYMS_USIDList
syms_pdb_symbols_from_name(SYMS_Arena *arena, SYMS_String8 data, SYMS_MsfAccel *msf,
                           SYMS_PdbGsiAccel *gsi, SYMS_CvUnitAccel *unit, SYMS_String8 name){
  SYMS_USIDList result = {0};
  
  if (gsi->bucket_count > 0){
    // get unit range
    SYMS_MsfRange range = syms_msf_range_from_sn(msf, unit->sn);
    
    // grab scratch
    SYMS_ArenaTemp scratch = syms_get_scratch(&arena, 1);
    
    // get bucket
    SYMS_U32 name_hash = syms_pdb_hashV1(name);
    SYMS_U32 bucket_index = name_hash%gsi->bucket_count;
    
    // iterate bucket
    for (SYMS_PdbChain *bucket = gsi->buckets[bucket_index];
         bucket != 0;
         bucket = bucket->next){
      SYMS_U32 off = bucket->v;
      SYMS_CvStub *stub = syms_cv_stub_from_unit_off(unit, off);
      if (stub != 0 && syms_string_match(stub->name, name, 0)){
        SYMS_CvElement element = syms_cv_element(data, msf, range, stub->off, 1);
        
        SYMS_UnitID   uid = 0;
        SYMS_SymbolID sid = 0;
        switch (element.kind){
          case SYMS_CvSymKind_PROCREF:
          case SYMS_CvSymKind_LPROCREF:
          case SYMS_CvSymKind_DATAREF:
          {
            SYMS_CvRef2 ref2 = {0};
            syms_msf_read_struct_in_range(data, msf, element.range, 0, &ref2);
            uid = SYMS_PdbPseudoUnit_COUNT + ref2.imod;
            sid = SYMS_ID_u32_u32(SYMS_CvSymbolIDKind_Off, ref2.sym_off);
          }break;
        }
        
        if (uid != 0){
          SYMS_USIDNode *chain = syms_push_array(arena, SYMS_USIDNode, 1);
          SYMS_QueuePush(result.first, result.last, chain);
          result.count += 1;
          chain->usid.uid = uid;
          chain->usid.sid = sid;
        }
      }
    }
    
    // release scratch
    syms_release_scratch(scratch);
  }
  
  return(result);
}

////////////////////////////////
//~ allen: PDB Accel Functions

// pdb specific parsing

SYMS_API SYMS_PdbDbiAccel
syms_pdb_dbi_from_msf(SYMS_String8 data, SYMS_MsfAccel *msf, SYMS_MsfStreamNumber sn){
  SYMS_ProfBegin("syms_pdb_dbi_from_msf");
  
  // grab the dbi stream
  SYMS_MsfRange range = syms_msf_range_from_sn(msf, sn);
  
  // read dbi header
  SYMS_PdbDbiHeader header = {0};
  syms_msf_read_struct_in_range(data, msf, range, 0, &header);
  
  // extract dbi info
  SYMS_PdbDbiAccel result = {0};
  if (header.sig == SYMS_PdbDbiHeaderSignature_V1){
    // info directly from the header
    result.valid = 1;
    result.machine_type = header.machine;
    result.gsi_sn = header.gsi_sn;
    result.psi_sn = header.psi_sn;
    result.sym_sn = header.sym_sn;
    
    //- organize the sizes of the ranges in dbi
    SYMS_U64 range_size[SYMS_PdbDbiRange_COUNT];
    range_size[SYMS_PdbDbiRange_ModuleInfo] = header.module_info_size;
    range_size[SYMS_PdbDbiRange_SecCon]     = header.sec_con_size;
    range_size[SYMS_PdbDbiRange_SecMap]     = header.sec_map_size;
    range_size[SYMS_PdbDbiRange_FileInfo]   = header.file_info_size;
    range_size[SYMS_PdbDbiRange_TSM]        = header.tsm_size;
    range_size[SYMS_PdbDbiRange_EcInfo]     = header.ec_info_size;
    range_size[SYMS_PdbDbiRange_DbgHeader]  = header.dbg_header_size;
    
    //- fill range offset array
    {
      SYMS_U64 cursor = sizeof(header);
      SYMS_U64 i = 0;
      for (; i < (SYMS_U64)(SYMS_PdbDbiRange_COUNT); i += 1){
        result.range_off[i] = cursor;
        cursor += range_size[i];
        cursor = SYMS_ClampTop(cursor, range.size);
      }
      // allen: one last value past the end so that off[i + 1] - off[i] can get us the sizes back.
      result.range_off[i] = cursor;
    }
    
    //- read debug streams
    {
      // allen: zero is a valid stream [ it sucks I know :( ]
      // so we explicitly invalidate sn by clearing to 0xFF instead.
      syms_memset(result.dbg_sn, 0xFF, sizeof(result.dbg_sn));
      SYMS_MsfRange dbg_sub_range = syms_pdb_dbi_sub_range(&result, sn, SYMS_PdbDbiRange_DbgHeader);
      SYMS_U64 read_size = SYMS_ClampTop(sizeof(result.dbg_sn), dbg_sub_range.size);
      syms_msf_read_in_range(data, msf, dbg_sub_range, 0, read_size, result.dbg_sn);
    }
  }
  
  SYMS_ProfEnd();
  
  return(result);
}

SYMS_API SYMS_MsfRange
syms_pdb_dbi_sub_range(SYMS_PdbDbiAccel *dbi, SYMS_MsfStreamNumber sn, SYMS_PdbDbiRange n){
  SYMS_MsfRange result = {0};
  result.sn = sn;
  result.off = dbi->range_off[n];
  result.size = dbi->range_off[n + 1] - result.off;
  return(result);
}

SYMS_API SYMS_MsfRange
syms_pdb_dbi_stream(SYMS_MsfAccel *msf, SYMS_PdbDbiAccel *dbi, SYMS_PdbDbiStream n){
  SYMS_MsfRange result = syms_msf_range_from_sn(msf, dbi->dbg_sn[n]);
  return(result);
}

SYMS_API SYMS_PdbInfoTable
syms_pdb_parse_info(SYMS_Arena *arena, SYMS_String8 data, SYMS_MsfAccel *msf){
  SYMS_ProfBegin("syms_pdb_parse_info");
  
  // grab the info stream
  SYMS_MsfRange range = syms_msf_range_from_sn(msf, SYMS_PdbFixedStream_PDB);
  
  // read info stream's header
  SYMS_PdbInfoHeader header = {0};
  syms_msf_read_struct_in_range(data, msf, range, 0, &header);
  SYMS_U32 after_header_off = sizeof(header);
  
  // read auth_guid (only in certain recent pdb versions)
  SYMS_PeGuid auth_guid = {0};
  SYMS_U32 after_auth_guid_off = after_header_off;
  switch (header.version){
    case SYMS_PdbVersion_VC70_DEP:
    case SYMS_PdbVersion_VC70:
    case SYMS_PdbVersion_VC80:
    case SYMS_PdbVersion_VC110:
    case SYMS_PdbVersion_VC140:
    {
      syms_msf_read_struct_in_range(data, msf, range, after_header_off, &auth_guid);
      after_auth_guid_off = after_header_off + sizeof(auth_guid);
    }break;
    
    default:
    {}break;
  }
  
  // read table layout data
  SYMS_U32 names_base_off = 0;
  SYMS_U32 hash_table_count = 0;
  SYMS_U32 hash_table_max = 0;
  SYMS_U32 epilogue_base_off = 0;
  
  if (header.version != 0){
    SYMS_U32 names_len_off = after_auth_guid_off;
    
    SYMS_U32 names_len = 0;
    syms_msf_read_struct_in_range(data, msf, range, names_len_off, &names_len);
    
    names_base_off = names_len_off + 4;
    
    SYMS_U32 hash_table_count_off = names_base_off + names_len;
    syms_msf_read_struct_in_range(data, msf, range, hash_table_count_off, &hash_table_count);
    
    SYMS_U32 hash_table_max_off = hash_table_count_off + 4;
    syms_msf_read_struct_in_range(data, msf, range, hash_table_max_off, &hash_table_max);
    
    SYMS_U32 num_present_words_off = hash_table_max_off + 4;
    SYMS_U32 num_present_words = 0;
    syms_msf_read_struct_in_range(data, msf, range, num_present_words_off, &num_present_words);
    
    SYMS_U32 present_words_array_off = num_present_words_off + 4;
    
    SYMS_U32 num_deleted_words_off = present_words_array_off + num_present_words*sizeof(SYMS_U32);
    SYMS_U32 num_deleted_words = 0;
    syms_msf_read_struct_in_range(data, msf, range, num_deleted_words_off, &num_deleted_words);
    
    SYMS_U32 deleted_words_array_off = num_deleted_words_off + 4;
    
    epilogue_base_off = deleted_words_array_off + num_deleted_words*sizeof(SYMS_U32);
  }
  
  // read table
  SYMS_PdbInfoTable result = {0};
  if (hash_table_count > 0 && syms_msf_bounds_check_in_range(range, epilogue_base_off)){
    SYMS_U32 record_off = epilogue_base_off;
    for (SYMS_U32 i = 0;
         i < hash_table_count;
         i += 1, record_off += 8){
      // read record
      SYMS_U32 record[2];
      syms_msf_read_in_range(data, msf, range, record_off, 8, record);
      SYMS_U32 relative_name_off = record[0];
      SYMS_MsfStreamNumber sn = (SYMS_MsfStreamNumber)record[1];
      
      // read name
      SYMS_U32 name_off = names_base_off + relative_name_off;
      SYMS_String8 name = syms_msf_read_zstring_in_range(arena, data, msf, range, name_off);
      
      // push new slot
      SYMS_PdbInfoSlot *slot = syms_push_array(arena, SYMS_PdbInfoSlot, 1);
      SYMS_QueuePush(result.first, result.last, slot);
      slot->string = name;
      slot->sn = sn;
    }
  }
  result.auth_guid = auth_guid;
  
  SYMS_ProfEnd();
  
  return(result);
}

SYMS_API SYMS_PdbNamedStreamArray
syms_pdb_named_stream_array(SYMS_PdbInfoTable *table){
  SYMS_ProfBegin("syms_pdb_named_stream_array");
  
  // mapping "PdbDbiNamedStream" indexes to strings
  struct StreamNameIndexPair{
    SYMS_PdbNamedStream index;
    SYMS_String8 name;
  };
  struct StreamNameIndexPair pairs[] = {
    {SYMS_PdbNamedStream_HEADER_BLOCK, syms_str8_lit("/src/headerblock")},
    {SYMS_PdbNamedStream_STRTABLE    , syms_str8_lit("/names")},
    {SYMS_PdbNamedStream_LINK_INFO   , syms_str8_lit("/LinkInfo")},
  };
  
  // build baked array of stream indices
  SYMS_PdbNamedStreamArray result = {0};
  struct StreamNameIndexPair *p = pairs;
  for (SYMS_U64 i = 0; i < SYMS_ARRAY_SIZE(pairs); i += 1, p += 1){
    SYMS_String8 name = p->name;
    // slot from name
    SYMS_PdbInfoSlot *matching_slot = 0;
    for (SYMS_PdbInfoSlot *slot = table->first;
         slot != 0;
         slot = slot->next){
      if (syms_string_match(name, slot->string, 0)){
        matching_slot = slot;
        break;
      }
    }
    // store this slot's stream number into the array
    if (matching_slot != 0){
      result.sn[p->index] = matching_slot->sn;
    }
  }
  
  SYMS_ProfEnd();
  
  return(result);
}

SYMS_API SYMS_PdbStrtblAccel
syms_pdb_dbi_parse_strtbl(SYMS_Arena *arena, SYMS_String8 data, SYMS_MsfAccel *msf, SYMS_MsfStreamNumber sn){
  SYMS_ProfBegin("syms_pdb_dbi_parse_strtbl");
  
  // grab the strtable stream
  SYMS_MsfRange range = syms_msf_range_from_sn(msf, sn);
  
  // read strtable's header
  SYMS_PdbDbiStrTableHeader header = {0};
  syms_msf_read_struct_in_range(data, msf, range, 0, &header);
  
  // read strtable's layout
  SYMS_PdbStrtblAccel result = {0};
  if (header.magic == SYMS_PdbDbiStrTableHeader_MAGIC && header.version == 1){
    SYMS_U32 strblock_size_off = sizeof(header);
    SYMS_U32 strblock_size = 0;
    syms_msf_read_struct_in_range(data, msf, range, strblock_size_off, &strblock_size);
    SYMS_U32 strblock_off = strblock_size_off + 4;
    
    SYMS_U32 bucket_count_off = strblock_off + strblock_size;
    SYMS_U32 bucket_count = 0;
    syms_msf_read_struct_in_range(data, msf, range, bucket_count_off, &bucket_count);
    
    SYMS_U32 bucket_array_off = bucket_count_off + 4;
    SYMS_U32 bucket_array_size = bucket_count*sizeof(SYMS_PdbStringIndex);
    
    if (syms_msf_bounds_check_in_range(range, bucket_array_off + bucket_array_size)){
      result.bucket_count = bucket_count;
      result.sn = sn;
      result.strblock.min = range.off + strblock_off;
      result.strblock.max = result.strblock.min + strblock_size;
      result.buckets.min  = range.off + bucket_array_off;
      result.buckets.max  = result.buckets.min + bucket_array_size;
    }
  }
  
  SYMS_ProfEnd();
  
  return(result);
}


// pdb specific api

SYMS_API SYMS_String8
syms_pdb_strtbl_string_from_off(SYMS_Arena *arena, SYMS_String8 data, SYMS_PdbDbgAccel *dbg, SYMS_U32 off){
  // setup accel
  SYMS_MsfAccel *msf = dbg->msf;
  SYMS_PdbStrtblAccel *strtbl = &dbg->strtbl;
  
  // get string range
  SYMS_MsfRange range = syms_msf_range_from_sn(msf, strtbl->sn);
  SYMS_MsfRange str_range = syms_msf_sub_range_from_off_range(range, strtbl->strblock);
  
  // read string
  SYMS_String8 result = syms_msf_read_zstring_in_range(arena, data, msf, str_range, off);
  return(result);
}

SYMS_API SYMS_String8
syms_pdb_strtbl_string_from_index(SYMS_Arena *arena, SYMS_String8 data, SYMS_PdbDbgAccel *dbg,
                                  SYMS_PdbStringIndex n){
  // setup accel
  SYMS_MsfAccel *msf = dbg->msf;
  SYMS_PdbStrtblAccel *strtbl = &dbg->strtbl;
  
  SYMS_String8 result = {0};
  if (n < strtbl->bucket_count){
    // get bucket range
    SYMS_MsfRange range = syms_msf_range_from_sn(msf, strtbl->sn);
    SYMS_MsfRange bkt_range = syms_msf_sub_range_from_off_range(range, strtbl->buckets);
    
    // get offset
    SYMS_U32 offset_off = n*4;
    SYMS_U32 offset = 0;
    if (syms_msf_read_struct_in_range(data, msf, bkt_range, offset_off, &offset)){
      
      // get string range
      SYMS_MsfRange str_range = syms_msf_sub_range_from_off_range(range, strtbl->strblock);
      
      // read string
      result = syms_msf_read_zstring_in_range(arena, data, msf, str_range, offset);
    }
  }
  
  return(result);
}


// main api

SYMS_API SYMS_PdbFileAccel*
syms_pdb_file_accel_from_data(SYMS_Arena *arena, SYMS_String8 data){
  SYMS_PdbFileAccel *result = (SYMS_PdbFileAccel*)&syms_format_nil;
  SYMS_MsfAccel *msf = syms_msf_accel_from_data(arena, data);
  if (msf != 0){
    result = syms_push_array(arena, SYMS_PdbFileAccel, 1);
    result->format = SYMS_FileFormat_PDB;
    result->msf = msf;
  }
  return(result);
}

SYMS_API SYMS_PdbDbgAccel*
syms_pdb_dbg_accel_from_file(SYMS_Arena *arena, SYMS_String8 data, SYMS_PdbFileAccel *file){
  // setup accel
  SYMS_MsfAccel *msf = file->msf;
  
  // setup result
  SYMS_PdbDbgAccel *result = (SYMS_PdbDbgAccel*)&syms_format_nil;
  
  // parse dbi header
  SYMS_PdbDbiAccel dbi = syms_pdb_dbi_from_msf(data, msf, SYMS_PdbFixedStream_DBI);
  if (dbi.valid){
    // copy msf if it's not a prior on this arena.
    // TODO(allen): re-apply deep copy optimizations using handle checks
    SYMS_MsfAccel *dbg_msf = syms_msf_deep_copy(arena, msf);
    
    // parse info table
    SYMS_ArenaTemp scratch = syms_get_scratch(&arena, 1);
    SYMS_PdbInfoTable info_table = syms_pdb_parse_info(scratch.arena, data, msf);
    
    // get gsi range
    SYMS_MsfRange gsi_range = syms_msf_range_from_sn(msf, dbi.gsi_sn);
    SYMS_MsfRange psi_range = syms_msf_range_from_sn(msf, dbi.psi_sn);
    SYMS_MsfRange gsi_part_psi_range = syms_pdb_gsi_part_from_psi_range(psi_range);
    
    // get an msf range for section data
    SYMS_MsfRange section_data_range = syms_pdb_dbi_stream(msf, &dbi, SYMS_PdbDbiStream_SECTION_HEADER);
    
    // setup voff acceleration for coff sections
    SYMS_U64Array section_voff_array = {0};
    {
      // grab coff array
      SYMS_U64 sec_count = section_data_range.size/sizeof(SYMS_CoffSectionHeader);
      SYMS_CoffSectionHeader *sec_hdrs = syms_push_array(scratch.arena, SYMS_CoffSectionHeader, sec_count);
      syms_msf_read_in_range(data, msf, section_data_range, 0, sec_count*sizeof(*sec_hdrs), sec_hdrs);
      
      // convert to voff array
      section_voff_array = syms_pecoff_voff_array_from_coff_hdr_array(arena, sec_hdrs, sec_count);
    }
    
    // fill accelerator
    {
      result = syms_push_array(arena, SYMS_PdbDbgAccel, 1);
      result->format = SYMS_FileFormat_PDB;
      
      // msf
      result->msf = dbg_msf;
      
      // pdb
      result->dbi = dbi;
      result->named = syms_pdb_named_stream_array(&info_table);
      result->strtbl = syms_pdb_dbi_parse_strtbl(arena, data, msf, result->named.sn[SYMS_PdbNamedStream_STRTABLE]);
      result->tpi = syms_pdb_tpi_accel_from_sn(arena, data, msf, SYMS_PdbFixedStream_TPI);
      result->ipi = syms_pdb_tpi_accel_from_sn(arena, data, msf, SYMS_PdbFixedStream_IPI);
      result->gsi = syms_pdb_gsi_accel_from_range(arena, data, msf, gsi_range);
      result->psi = syms_pdb_gsi_accel_from_range(arena, data, msf, gsi_part_psi_range);
      result->auth_guid = info_table.auth_guid;
      
      // arch
      result->arch = syms_arch_from_coff_machine_type(dbi.machine_type);
      
      // type uid
      result->type_uid = SYMS_PdbPseudoUnit_TPI;
      
      // section data
      result->section_data_range = section_data_range;
      result->section_voffs = section_voff_array.u64;
      result->section_count = section_voff_array.count;
    }
    
    // release scratch
    syms_release_scratch(scratch);
  }
  return(result);
}


////////////////////////////////
//~ allen: PDB Architecture

SYMS_API SYMS_Arch
syms_pdb_arch_from_dbg(SYMS_PdbDbgAccel *dbg){
  SYMS_Arch result = dbg->arch;
  return(result);
}


////////////////////////////////
//~ allen: PDB Match Keys

SYMS_API SYMS_ExtMatchKey
syms_pdb_ext_match_key_from_dbg(SYMS_String8 data, SYMS_PdbDbgAccel *dbg){
  SYMS_ExtMatchKey result = {0};
  syms_memmove(&result, &dbg->auth_guid, sizeof(dbg->auth_guid));
  return(result);
}


////////////////////////////////
//~ allen: PDB Sections

// pdb specific

SYMS_API SYMS_CoffSectionHeader
syms_pdb_coff_section(SYMS_String8 data, SYMS_PdbDbgAccel *dbg, SYMS_U64 n){
  SYMS_CoffSectionHeader result = {0};
  if (1 <= n){
    SYMS_MsfRange range = dbg->section_data_range;
    SYMS_U64 off = (n - 1)*sizeof(SYMS_CoffSectionHeader);
    syms_msf_read_struct_in_range(data, dbg->msf, range, off, &result);
  }
  return(result);
}

SYMS_API SYMS_U64
syms_pdb_voff_from_section_n(SYMS_PdbDbgAccel *dbg, SYMS_U64 n){
  SYMS_U64 result = 0;
  if (1 <= n && n <= dbg->section_count){
    result = dbg->section_voffs[n - 1];
  }
  return(result);
}

// main api

SYMS_API SYMS_SecInfoArray
syms_pdb_sec_info_array_from_dbg(SYMS_Arena *arena, SYMS_String8 data, SYMS_PdbDbgAccel *dbg){
  // setup accels
  SYMS_MsfAccel *msf = dbg->msf;
  SYMS_MsfRange range = dbg->section_data_range;
  SYMS_SecInfoArray result = syms_cv_sec_info_array_from_bin(arena, data, msf, range);
  return(result);
}

////////////////////////////////
//~ allen: PDB Compilation Units

// pdb helpers

SYMS_API SYMS_PdbCompUnit*
syms_pdb_comp_unit_from_id(SYMS_PdbUnitSetAccel *accel, SYMS_UnitID id){
  SYMS_PdbCompUnit *result = 0;
  if (SYMS_PdbPseudoUnit_FIRST_COMP_UNIT <= id){
    SYMS_U64 index = id - SYMS_PdbPseudoUnit_FIRST_COMP_UNIT;
    if (index < accel->comp_count){
      result = accel->comp_units[index];
    }
  }
  return(result);
}

SYMS_API SYMS_MsfRange
syms_pdb_msf_range_from_comp_unit(SYMS_PdbCompUnit *unit, SYMS_PdbCompUnitRange n){
  SYMS_MsfRange result = {0};
  result.sn = unit->sn;
  result.off = unit->range_off[n];
  result.size = unit->range_off[n + 1] - result.off;
  return(result);
}

// main api

SYMS_API SYMS_PdbUnitSetAccel*
syms_pdb_unit_set_accel_from_dbg(SYMS_Arena *arena, SYMS_String8 data, SYMS_PdbDbgAccel *dbg){
  // setup accels
  SYMS_MsfAccel *msf = dbg->msf;
  SYMS_PdbDbiAccel *dbi = &dbg->dbi;
  
  // parse list of modules
  SYMS_PdbCompUnitNode *first = 0;
  SYMS_PdbCompUnitNode *last = 0;
  SYMS_U64 count = 0;
  
  {
    // grab module info range
    SYMS_MsfRange range = syms_pdb_dbi_sub_range(dbi, SYMS_PdbFixedStream_DBI, SYMS_PdbDbiRange_ModuleInfo);
    
    // parse loop
    SYMS_U64 off = 0;
    for (;;){
      // exit condition - past end
      if (off >= range.size){
        break;
      }
      
      // extract comp unit info
      SYMS_B32 got_comp_unit = syms_false;
      SYMS_PdbCompUnit comp_unit = {0};
      SYMS_U32 next_comp_unit_off = 0;
      
      SYMS_PdbDbiCompUnitHeader header = {0};
      if (syms_msf_read_struct_in_range(data, msf, range, off, &header)){
        got_comp_unit = syms_true;
        
        // extract names from after header
        SYMS_U32 name_off = off + sizeof(header);
        SYMS_String8 name = syms_msf_read_zstring_in_range(arena, data, msf, range, name_off);
        
        SYMS_U32 name2_off = name_off + name.size + 1;
        SYMS_String8 name2 = syms_msf_read_zstring_in_range(arena, data, msf, range, name2_off);
        
        SYMS_U32 after_name2_off = name2_off + name2.size + 1;
        next_comp_unit_off = (after_name2_off + 3)&~3;
        
        // get the size of this mod's stream
        SYMS_MsfStreamInfo stream_info = syms_msf_stream_info_from_sn(msf, header.sn);
        SYMS_U64 stream_size = stream_info.size;
        
        // organize size information
        SYMS_U64 range_size[SYMS_PdbCompUnitRange_COUNT];
        range_size[SYMS_PdbCompUnitRange_Symbols] = header.symbol_bytes;
        range_size[SYMS_PdbCompUnitRange_C11] = header.c11_lines_size;
        range_size[SYMS_PdbCompUnitRange_C13] = header.c13_lines_size;
        
        // fill comp unit range offs
        {
          SYMS_U64 cursor = 0;
          SYMS_U64 i = 0;
          for (; i < (SYMS_U64)(SYMS_PdbCompUnitRange_COUNT); i += 1){
            comp_unit.range_off[i] = cursor;
            cursor += range_size[i];
            cursor = SYMS_ClampTop(cursor, stream_size);
          }
          //~ allen: one last value past the end so that off[i + 1] - off[i] can get us the sizes back.
          comp_unit.range_off[i] = cursor;
        }
        
        // skip annoying signature in first range
        if (range_size[SYMS_PdbCompUnitRange_Symbols] > 4){
          comp_unit.range_off[0] += 4;
        }
        
        // fill comp unit direct fields
        comp_unit.sn = header.sn;
        comp_unit.src_file = header.src_file;
        comp_unit.pdb_file = header.pdb_file;
        
        comp_unit.group_name = name;
        comp_unit.obj_name = name2;
      }
      
      // exit condition - failed read
      if (!got_comp_unit){
        break;
      }
      
      // save to list
      SYMS_PdbCompUnitNode *node = syms_push_array(arena, SYMS_PdbCompUnitNode, 1);
      SYMS_QueuePush(first, last, node);
      count += 1;
      syms_memmove(&node->comp_unit, &comp_unit, sizeof(comp_unit));
      
      // increment offset
      off = next_comp_unit_off;
    }
  }
  
  // linearize for quick index
  SYMS_PdbCompUnit **comp_units = syms_push_array(arena, SYMS_PdbCompUnit*, count);
  {
    SYMS_PdbCompUnit **comp_unit_ptr = comp_units;
    for (SYMS_PdbCompUnitNode *node = first;
         node != 0;
         node = node->next, comp_unit_ptr += 1){
      *comp_unit_ptr = &node->comp_unit;
    }
  }
  
  // fill result
  SYMS_PdbUnitSetAccel *result = syms_push_array(arena, SYMS_PdbUnitSetAccel, 1);
  result->format = SYMS_FileFormat_PDB;
  result->comp_units = comp_units;
  result->comp_count = count;
  
  return(result);
}

SYMS_API SYMS_U64
syms_pdb_unit_count_from_set(SYMS_PdbUnitSetAccel *unit_set){
  SYMS_U64 result = unit_set->comp_count + SYMS_PdbPseudoUnit_COUNT;
  return(result);
}

SYMS_API SYMS_UnitInfo
syms_pdb_unit_info_from_uid(SYMS_PdbUnitSetAccel *unit_set, SYMS_UnitID uid){
  SYMS_UnitInfo result = {0};
  
  // TODO(allen): language
  
  switch (uid){
    case SYMS_PdbPseudoUnit_SYM:
    {
      result.features = (SYMS_UnitFeature_StaticVariables|
                         SYMS_UnitFeature_ExternVariables|
                         SYMS_UnitFeature_FunctionStubs);
    }break;
    
    case SYMS_PdbPseudoUnit_TPI:
    {
      result.features = SYMS_UnitFeature_Types;
    }break;
    
    default:
    {
      SYMS_PdbCompUnit *unit = syms_pdb_comp_unit_from_id(unit_set, uid);
      if (unit != 0){
        result.features = (SYMS_UnitFeature_CompilationUnit|
                           SYMS_UnitFeature_StaticVariables|
                           SYMS_UnitFeature_Functions);
      }
    }break;
  }
  
  return(result);
}

SYMS_API SYMS_UnitNames
syms_pdb_unit_names_from_uid(SYMS_Arena *arena, SYMS_PdbUnitSetAccel *unit_set, SYMS_UnitID uid){
  SYMS_UnitNames result = {0};
  // try compilation unit
  SYMS_PdbCompUnit *unit = syms_pdb_comp_unit_from_id(unit_set, uid);
  if (unit != 0){
    result.object_file  = syms_push_string_copy(arena, unit->group_name);
    result.archive_file = syms_push_string_copy(arena, unit->obj_name);
  }
  return(result);
}

SYMS_API SYMS_UnitRangeArray
syms_pdb_unit_ranges_from_set(SYMS_Arena *arena, SYMS_String8 data, SYMS_PdbDbgAccel *dbg,
                              SYMS_PdbUnitSetAccel *unit_set){
  SYMS_ArenaTemp scratch = syms_get_scratch(&arena, 1);
  
  //- setup accels
  SYMS_MsfAccel *msf = dbg->msf;
  SYMS_PdbDbiAccel *dbi = &dbg->dbi;
  
  //- parse contributions
  SYMS_U64 range_count = 0;
  SYMS_UnitRange *ranges = 0;
  {
    // grab module info range
    SYMS_MsfRange range = syms_pdb_dbi_sub_range(dbi, SYMS_PdbFixedStream_DBI, SYMS_PdbDbiRange_SecCon);
    
    // read version
    SYMS_PdbDbiSectionContribVersion version = 0;
    syms_msf_read_struct_in_range(data, msf, range, 0, &version);
    
    // read array off/size info
    SYMS_U32 item_size = 0;
    SYMS_U32 array_off = 0;
    switch (version){
      default:
      {
        // TODO(allen): do we have a test case for this?
        version = 0;
        item_size = sizeof(SYMS_PdbDbiSectionContrib40);
      }break;
      case SYMS_PdbDbiSectionContribVersion_1:
      {
        item_size = sizeof(SYMS_PdbDbiSectionContrib);
        array_off = sizeof(version);
      }break;
      case SYMS_PdbDbiSectionContribVersion_2:
      {
        item_size = sizeof(SYMS_PdbDbiSectionContrib2);
        array_off = sizeof(version);
      }break;
    }
    
    // allocate ranges array
    SYMS_U64 max_count = (range.size - array_off)/item_size;
    ranges = syms_push_array(arena, SYMS_UnitRange, max_count);
    
    // read contribution data
    SYMS_MsfRange sub_range = syms_msf_sub_range(range, array_off, range.size - array_off);
    SYMS_String8 contrib_data = syms_msf_read_whole_range(scratch.arena, data, msf, sub_range);
    
    // grab section -> voff accel
    SYMS_U64 section_count = dbg->section_count;
    SYMS_U64 *section_voffs = dbg->section_voffs;
    
    // fill array
    {
      SYMS_UnitRange *unit_range = ranges;
      SYMS_UnitRange *opl = ranges + max_count;
      SYMS_U8 *ptr = contrib_data.str;
      SYMS_U8 *ptr_opl = contrib_data.str + contrib_data.size;
      for (; ptr < ptr_opl; ptr += item_size){
        SYMS_PdbDbiSectionContrib40 *sc = (SYMS_PdbDbiSectionContrib40*)ptr;
        if (sc->size > 0 && 1 <= sc->sec && sc->sec <= section_count){
          // calculate range
          SYMS_U64 section_voff = section_voffs[sc->sec - 1];
          SYMS_U64 min = section_voff + sc->sec_off;
          
          // fill unit range
          unit_range->vrange.min = min;
          unit_range->vrange.max = min + sc->size;
          unit_range->uid = sc->mod + SYMS_PdbPseudoUnit_FIRST_COMP_UNIT;
          unit_range += 1;
        }
      }
      
      range_count = (SYMS_U64)(unit_range - ranges);
      
      syms_arena_put_back(arena, sizeof(SYMS_UnitRange)*(max_count - range_count));
    }
  }
  
  syms_release_scratch(scratch);
  
  //- assemble result
  SYMS_UnitRangeArray result = {ranges, range_count};
  return(result);
}

////////////////////////////////
//~ allen: PDB Symbol Parsing

// cv sym parse

SYMS_API SYMS_CvUnitAccel*
syms_pdb_pub_sym_accel_from_dbg(SYMS_Arena *arena, SYMS_String8 data, SYMS_PdbDbgAccel *dbg){
  SYMS_MsfAccel *msf = dbg->msf;
  SYMS_MsfRange range = syms_msf_range_from_sn(msf, dbg->dbi.sym_sn);
  SYMS_U64Range r = {range.off, range.off + range.size};
  SYMS_U64RangeArray range_array = {&r, 1};
  
  SYMS_CvSymConsParams params = {SYMS_FileFormat_Null};
  params.format = SYMS_FileFormat_PDB;
  params.uid = SYMS_PdbPseudoUnit_SYM;
  params.align = SYMS_CV_SYMBOL_ALIGN_IN_PDB;
  
  SYMS_CvUnitAccel *result = syms_cv_sym_unit_from_ranges(arena, data, msf, range.sn, range_array,
                                                          &params);
  return(result);
}

// cv leaf parse

SYMS_API SYMS_CvUnitAccel*
syms_pdb_leaf_accel_from_dbg(SYMS_Arena *arena, SYMS_String8 data, SYMS_PdbDbgAccel *dbg,
                             SYMS_UnitID uid){
  // get range from tpi
  SYMS_MsfAccel *msf = dbg->msf;
  SYMS_PdbTpiAccel *tpi = &dbg->tpi;
  SYMS_MsfRange msf_range = syms_pdb_tpi_range(msf, tpi);
  SYMS_U64Range range = {msf_range.off, msf_range.off + msf_range.size};
  
  // setup cons params
  SYMS_CvLeafConsParams params = {SYMS_FileFormat_Null};
  params.format = dbg->format;
  params.uid = uid;
  params.first_ti = tpi->first_ti;
  params.align = 4;
  
  // parse
  SYMS_CvUnitAccel *result = syms_cv_leaf_unit_from_range(arena, data, msf,
                                                          msf_range.sn, range, &params);
  
  return(result);
}

// main api

SYMS_API SYMS_CvUnitAccel*
syms_pdb_unit_accel_from_uid(SYMS_Arena *arena, SYMS_String8 data, SYMS_PdbDbgAccel *dbg,
                             SYMS_PdbUnitSetAccel *unit_set, SYMS_UnitID uid){
  // setup result
  SYMS_CvUnitAccel *result = (SYMS_CvUnitAccel*)&syms_format_nil;
  
  switch (uid){
    case SYMS_PdbPseudoUnit_SYM:
    {
      result = syms_pdb_pub_sym_accel_from_dbg(arena, data, dbg);
    }break;
    
    case SYMS_PdbPseudoUnit_TPI:
    {
      result = syms_pdb_leaf_accel_from_dbg(arena, data, dbg, uid);
    }break;
    
    default:
    {
      SYMS_PdbCompUnit *unit = syms_pdb_comp_unit_from_id(unit_set, uid);
      if (unit != 0){
        SYMS_MsfRange range = syms_pdb_msf_range_from_comp_unit(unit, SYMS_PdbCompUnitRange_Symbols);
        SYMS_U64Range r = {range.off, range.off + range.size};
        SYMS_U64RangeArray range_array = {&r, 1};
        
        SYMS_CvSymConsParams params = {SYMS_FileFormat_Null};
        params.format = SYMS_FileFormat_PDB;
        params.uid = SYMS_PdbPseudoUnit_SYM;
        params.align = SYMS_CV_SYMBOL_ALIGN_IN_PDB;
        
        result = syms_cv_sym_unit_from_ranges(arena, data, dbg->msf, range.sn, range_array, &params);
      }
    }break;
  }
  
  return(result);
}

SYMS_API SYMS_UnitID
syms_pdb_tls_var_uid_from_dbg(SYMS_PdbDbgAccel *dbg){
  // NOTE(nick): Thread var symbols are stored in global symbol stream
  // which doesn't abstract well with DWARF model where thread vars
  // are stored on per compilation units basis.
  return(SYMS_PdbPseudoUnit_SYM);
}

SYMS_API SYMS_SymbolIDArray
syms_pdb_tls_var_sid_array_from_unit(SYMS_Arena *arena, SYMS_CvUnitAccel *thread_unit){
  SYMS_SymbolIDArray result = {0};
  if (!thread_unit->leaf_set){
    
    //- allocate array
    SYMS_U64 count = thread_unit->tls_var_count;
    SYMS_SymbolID *ids = syms_push_array(arena, SYMS_SymbolID, count);
    
    //- fill array
    SYMS_SymbolID *id_ptr = ids;
    SYMS_CvStub **stub_ptr = thread_unit->tls_var_stubs;
    SYMS_CvStub **opl = stub_ptr + count;
    for (; stub_ptr < opl; id_ptr += 1, stub_ptr += 1){
      *id_ptr = SYMS_ID_u32_u32(SYMS_CvSymbolIDKind_Off, (**stub_ptr).off);
    }
    
    //- assemble result
    result.count = count;
    result.ids = ids;
  }
  
  return(result);
}

SYMS_API SYMS_U64
syms_pdb_symbol_count_from_unit(SYMS_CvUnitAccel *unit){
  SYMS_U64 result = unit->top_count;
  return(result);
}

SYMS_API SYMS_SymbolKind
syms_pdb_symbol_kind_from_sid(SYMS_String8 data, SYMS_PdbDbgAccel *dbg, SYMS_CvUnitAccel *unit,
                              SYMS_SymbolID sid){
  SYMS_SymbolKind result = syms_cv_symbol_kind_from_sid(data, dbg->msf, unit, sid);
  return(result);
}

SYMS_API SYMS_TypeInfo
syms_pdb_type_info_from_sid(SYMS_String8 data, SYMS_PdbDbgAccel *dbg, SYMS_CvUnitAccel *unit,
                            SYMS_SymbolID sid){
  SYMS_TypeInfo result = {SYMS_TypeKind_Null};
  if (unit->leaf_set){
    // setup
    SYMS_MsfAccel *msf = dbg->msf;
    SYMS_UnitID uid = unit->uid;
    
    // get type info
    result = syms_cv_type_info_from_sid(data, msf, unit, sid);
    
    // ipi information lookup
    if (SYMS_ID_u32_0(sid) == SYMS_CvSymbolIDKind_Index){
      SYMS_CvTypeIndex ti = SYMS_ID_u32_1(sid);
      
      SYMS_PdbTpiAccel *ipi = &dbg->ipi;
      SYMS_MsfRange ipi_range = syms_msf_range_from_sn(msf, ipi->type_sn);
      
      if (ipi->bucket_count > 0){
        
        // get bucket
        SYMS_String8 id_data = {(SYMS_U8*)&ti, sizeof(ti)};
        SYMS_U32 ti_hash = syms_pdb_hashV1(id_data);
        SYMS_U32 bucket_index = ti_hash%ipi->bucket_count;
        
        // iterate bucket
        for (SYMS_PdbChain *bucket = ipi->buckets[bucket_index];
             bucket != 0;
             bucket = bucket->next){
          SYMS_CvItemId item_id  = bucket->v;
          SYMS_U32 item_off = syms_pdb_tpi_off_from_ti(data, msf, ipi, item_id);
          
          SYMS_CvElement element = syms_cv_element(data, msf, ipi_range, item_off, 1);
          switch (element.kind){
            default:break;
            
            case SYMS_CvLeaf_UDT_MOD_SRC_LINE:
            {
              // TODO(allen): we have never hit this case before; we're not sure how it works,
              // and have no way to test it.
              SYMS_ASSERT_PARANOID(!"not implemented");
#if 0
              SYMS_CvLeafModSrcLine mod_src_line = {0};
              if (syms_msf_read_struct_in_range(data, msf, element.range, 0, &mod_src_line)){
              }
#endif
            }break;
            
            case SYMS_CvLeaf_UDT_SRC_LINE:
            {
              SYMS_CvLeafUDTSrcLine udt_src_line = {0};
              if (syms_msf_read_struct_in_range(data, msf, element.range, 0, &udt_src_line)){
                if (udt_src_line.udt_itype == ti){
                  result.src_coord.file_id = SYMS_ID_u32_u32(SYMS_CvFileIDKind_IPIStringID, udt_src_line.src);
                  result.src_coord.line = udt_src_line.ln;
                  goto dbl_break;
                }
              }
            }break;
          }
        }
        
        dbl_break:;
      }
    }
    
  }
  return(result);
}

SYMS_API SYMS_ConstInfo
syms_pdb_const_info_from_id(SYMS_String8 data, SYMS_PdbDbgAccel *dbg, SYMS_CvUnitAccel *unit,
                            SYMS_SymbolID sid){
  SYMS_ConstInfo result = syms_cv_const_info_from_sid(data, dbg->msf, unit, sid);
  return(result);
}


////////////////////////////////
//~ allen: PDB Variable Info

// cv parse

SYMS_API SYMS_USID
syms_pdb_sym_type_from_var_id(SYMS_String8 data, SYMS_PdbDbgAccel *dbg, SYMS_CvUnitAccel *unit,
                              SYMS_SymbolID id){
  // setup accel
  SYMS_MsfAccel *msf = dbg->msf;
  
  // read id
  SYMS_MsfRange range = syms_msf_range_from_sn(msf, unit->sn);
  SYMS_CvStub *stub = syms_cv_stub_from_unit_off(unit, SYMS_ID_u32_1(id));
  
  // type uid
  SYMS_UnitID type_uid = dbg->type_uid;
  
  // zero clear result
  SYMS_USID result = {0};
  
  // parse symbol
  if (stub != 0){
    SYMS_CvElement element = syms_cv_element(data, msf, range, stub->off, 1);
    switch (element.kind){
      default:break;
      
      case SYMS_CvSymKind_LDATA32:
      case SYMS_CvSymKind_GDATA32:
      {
        SYMS_CvData32 data32 = {0};
        syms_msf_read_struct_in_range(data, msf, element.range, 0, &data32);
        
        result.uid = type_uid;
        result.sid = SYMS_ID_u32_u32(SYMS_CvSymbolIDKind_Index, data32.itype);
      }break;
      
      case SYMS_CvSymKind_LOCAL:
      {
        SYMS_CvLocal loc = {0};
        syms_msf_read_struct_in_range(data, msf, element.range, 0, &loc);
        
        result.uid = type_uid;
        result.sid = SYMS_ID_u32_u32(SYMS_CvSymbolIDKind_Index, loc.itype);
      }break;
      
      case SYMS_CvSymKind_REGREL32:
      {
        SYMS_CvRegrel32 regrel32 = {0};
        syms_msf_read_struct_in_range(data, msf, element.range, 0, &regrel32);
        
        result.uid = type_uid;
        result.sid = SYMS_ID_u32_u32(SYMS_CvSymbolIDKind_Index, regrel32.itype);
      }break;
      
      case SYMS_CvSymKind_LTHREAD32:
      case SYMS_CvSymKind_GTHREAD32:
      {
        SYMS_CvThread32 thread32 = {0};
        syms_msf_read_struct_in_range(data, msf, element.range, 0, &thread32);
        
        result.uid = type_uid;
        result.sid = SYMS_ID_u32_u32(SYMS_CvSymbolIDKind_Index, thread32.itype);
      }break;
    }
  }
  
  return(result);
}


SYMS_API SYMS_U64
syms_pdb_sym_voff_from_var_sid(SYMS_String8 data, SYMS_PdbDbgAccel *dbg, SYMS_CvUnitAccel *unit,
                               SYMS_SymbolID sid){
  // setup accel
  SYMS_MsfAccel *msf = dbg->msf;
  
  // read id
  SYMS_MsfRange range = syms_msf_range_from_sn(msf, unit->sn);
  SYMS_CvStub *stub = syms_cv_stub_from_unit_off(unit, SYMS_ID_u32_1(sid));
  
  // zero clear result
  SYMS_U64 result = 0;
  
  // parse symbol
  if (stub != 0){
    SYMS_CvElement element = syms_cv_element(data, msf, range, stub->off, 1);
    switch (element.kind){
      default:break;
      
      case SYMS_CvSymKind_LDATA32:
      case SYMS_CvSymKind_GDATA32:
      {
        SYMS_CvData32 data32 = {0};
        syms_msf_read_struct_in_range(data, msf, element.range, 0, &data32);
        SYMS_U64 section_voff = syms_pdb_voff_from_section_n(dbg, data32.sec);
        result = section_voff + data32.sec_off;
      }break;
    }
  }
  
  return(result);
}

SYMS_API SYMS_RegSection
syms_pdb_reg_section_from_x86_reg(SYMS_CvReg cv_reg){
  SYMS_RegSection result = {0};
  if (cv_reg < SYMS_ARRAY_SIZE(syms_reg_slices_X86)){
    SYMS_RegSlice *slice = syms_reg_slices_X86 + cv_reg;
    SYMS_RegID reg_id = slice->reg_id;
    if (reg_id < SYMS_RegX86Code_COUNT){
      SYMS_RegSection *section = syms_reg_section_X86 + reg_id;
      result.off = section->off + slice->byte_off;
      result.size = slice->byte_size;
    }
  }
  return(result);
}

SYMS_API SYMS_RegSection
syms_pdb_reg_section_from_x64_reg(SYMS_CvReg cv_reg){
  SYMS_RegSection result = {0};
  if (cv_reg < SYMS_ARRAY_SIZE(syms_reg_slices_X64)){
    SYMS_RegSlice *slice = syms_reg_slices_X64 + cv_reg;
    SYMS_RegID reg_id = slice->reg_id;
    if (reg_id < SYMS_RegX64Code_COUNT){
      SYMS_RegSection *section = syms_reg_section_X64 + reg_id;
      result.off = section->off + slice->byte_off;
      result.size = slice->byte_size;
    }
  }
  return(result);
}

SYMS_API SYMS_RegSection
syms_pdb_reg_section_from_arch_reg(SYMS_Arch arch, SYMS_CvReg cv_reg){
  SYMS_RegSection result = {0};
  switch (arch){
    case SYMS_Arch_X86: result = syms_pdb_reg_section_from_x86_reg(cv_reg); break;
    case SYMS_Arch_X64: result = syms_pdb_reg_section_from_x64_reg(cv_reg); break;
  }
  return(result);
}

SYMS_API SYMS_RegSection
syms_pdb_reg_section_from_framepointer(SYMS_String8 data,  SYMS_PdbDbgAccel *dbg,
                                       SYMS_MsfRange range, SYMS_CvStub *framepointer_stub){
  //- get accelerator
  SYMS_MsfAccel *msf = dbg->msf;
  
  //- get LOCAL
  SYMS_CvStub *local_stub = framepointer_stub->parent;
  SYMS_CvElement local_element = {0};
  SYMS_B32 got_local = syms_false;
  if (local_stub != 0){
    local_element = syms_cv_element(data, msf, range, local_stub->off, 1);
    if (local_element.kind == SYMS_CvSymKind_LOCAL){
      got_local = syms_true;
    }
  }
  
  //- get *PROC32
  SYMS_CvElement root_element = {0};
  if (got_local){
    SYMS_CvStub *root_stub = local_stub->parent;
    for (;root_stub->parent != 0;){
      root_stub = root_stub->parent;
    }
    root_element = syms_cv_element(data, msf, range, root_stub->off, 1);
  }
  SYMS_B32 got_proc32 = (root_element.kind == SYMS_CvSymKind_LPROC32 ||
                         root_element.kind == SYMS_CvSymKind_GPROC32);
  
  //- get FRAMEPROC
  SYMS_B32 got_fp_flags = syms_false;
  SYMS_CvFrameprocFlags fp_flags = 0;
  if (got_proc32){
    SYMS_U64 fp_off = root_element.range.off + root_element.range.size;
    SYMS_CvElement fp_element = syms_cv_element(data, msf, range, fp_off, 1);
    if (fp_element.kind == SYMS_CvSymKind_FRAMEPROC){
      SYMS_U64 flags_off = SYMS_MEMBER_OFFSET(SYMS_CvFrameproc, flags);
      got_fp_flags = syms_msf_read_struct_in_range(data, msf, fp_element.range,
                                                   flags_off, &fp_flags);
    }
  }
  
  //- determine the register
  SYMS_RegSection result = {0};
  SYMS_Arch arch = syms_pdb_arch_from_dbg(dbg);
  if (got_fp_flags){
    // check if parameter
    SYMS_U64 flags_off = SYMS_MEMBER_OFFSET(SYMS_CvLocal, flags);
    SYMS_CvLocalFlags local_flags = 0;
    syms_msf_read_struct_in_range(data, msf, local_element.range, flags_off, &local_flags);
    SYMS_B32 is_parameter = (local_flags & SYMS_CvLocalFlag_PARAM);
    
    // get frame pointer code
    SYMS_CvEncodedFramePtrReg frame_ptr_reg = 0;
    if (is_parameter){
      frame_ptr_reg = SYMS_CvFrameprocFlags_Extract_ParamBasePointer(fp_flags);
    }
    else{
      frame_ptr_reg = SYMS_CvFrameprocFlags_Extract_LocalBasePointer(fp_flags);
    }
    
    // get register section
    switch (arch){
      case SYMS_Arch_X86:
      {
        switch (frame_ptr_reg){
          case SYMS_CvEncodedFramePtrReg_StackPtr:
          {
            // TODO(allen): support SYMS_CvAllReg_VFRAME
          }break;
          case SYMS_CvEncodedFramePtrReg_FramePtr:
          {
            result = syms_pdb_reg_section_from_x86_reg(SYMS_CvRegx86_EBP);
          }break;
          case SYMS_CvEncodedFramePtrReg_BasePtr:
          {
            result = syms_pdb_reg_section_from_x86_reg(SYMS_CvRegx86_EBX);
          }break;
        }
      }break;
      case SYMS_Arch_X64:
      {
        switch (frame_ptr_reg){
          case SYMS_CvEncodedFramePtrReg_StackPtr:
          {
            result = syms_pdb_reg_section_from_x64_reg(SYMS_CvRegx64_RSP);
          }break;
          case SYMS_CvEncodedFramePtrReg_FramePtr:
          {
            result = syms_pdb_reg_section_from_x64_reg(SYMS_CvRegx64_RBP);
          }break;
          case SYMS_CvEncodedFramePtrReg_BasePtr:
          {
            result = syms_pdb_reg_section_from_x64_reg(SYMS_CvRegx64_R13);
          }break;
        }
      }break;
    }
  }
  
  //- fallback
  else{
    switch (arch){
      case SYMS_Arch_X86:
      {
        result = syms_reg_section_X86[SYMS_RegX86Code_ebp];
      }break;
      case SYMS_Arch_X64:
      {
        result = syms_reg_section_X64[SYMS_RegX64Code_rbp];
      }break;
    }
  }
  
  return(result);
}

// main api

SYMS_API SYMS_USID
syms_pdb_type_from_var_id(SYMS_String8 data, SYMS_PdbDbgAccel *dbg, SYMS_CvUnitAccel *unit, SYMS_SymbolID id){
  SYMS_USID result = {0};
  if (!unit->leaf_set){
    result = syms_pdb_sym_type_from_var_id(data, dbg, unit, id);
  }
  return(result);
}

SYMS_API SYMS_U64
syms_pdb_voff_from_var_sid(SYMS_String8 data, SYMS_PdbDbgAccel *dbg, SYMS_CvUnitAccel *unit, SYMS_SymbolID sid){
  SYMS_U64 result = 0;
  if (!unit->leaf_set){
    result = syms_pdb_sym_voff_from_var_sid(data, dbg, unit, sid);
  }
  return(result);
}

SYMS_API SYMS_Location
syms_pdb_location_from_var_sid(SYMS_Arena *arena, SYMS_String8 data, SYMS_PdbDbgAccel *dbg,
                               SYMS_CvUnitAccel *unit, SYMS_SymbolID sid){
  SYMS_Location result = {0};
  if (!unit->leaf_set){
    // setup accel
    SYMS_MsfAccel *msf = dbg->msf;
    
    // read id
    SYMS_MsfRange range = syms_msf_range_from_sn(msf, unit->sn);
    SYMS_CvStub *stub = syms_cv_stub_from_unit_off(unit, SYMS_ID_u32_1(sid));
    
    // parse symbol
    if (stub != 0){
      SYMS_CvElement element = syms_cv_element(data, msf, range, stub->off, 1);
      
      switch (element.kind){
        case SYMS_CvSymKind_LDATA32:
        case SYMS_CvSymKind_GDATA32:
        {
          SYMS_CvData32 data32 = {0};
          syms_msf_read_struct_in_range(data, msf, element.range, 0, &data32);
          SYMS_U64 section_voff = syms_pdb_voff_from_section_n(dbg, data32.sec);
          SYMS_U32 voff = section_voff + data32.sec_off;
          
          //- build the location info for addr:`module base + off`
          SYMS_EvalOpList list = {0};
          syms_eval_op_push(arena, &list, SYMS_EvalOp_ModuleOff, syms_eval_op_params(voff));
          
          result.op_list = list;
          result.mode = SYMS_EvalMode_Address;
        }break;
        
        case SYMS_CvSymKind_REGREL32:
        {
          //- extract frameproc's frame size
          SYMS_U32 frame_size = 0;
          {
            // find containing procedure
            SYMS_CvStub *containing_proc = 0;
            for (SYMS_CvStub *node = stub->parent;
                 node != 0;
                 node = node->parent){
              SYMS_CvElement el = syms_cv_element(data, msf, range, node->off, 1);
              if (el.kind == SYMS_CvSymKind_LPROC32 ||
                  el.kind == SYMS_CvSymKind_GPROC32){
                containing_proc = node;
                break;
              }
            }
            
            // find frameproc symbol inside this procedure
            if (containing_proc != 0){
              for (SYMS_CvStub *node = containing_proc->first;
                   node != 0;
                   node = node->sibling_next){
                SYMS_CvElement el = syms_cv_element(data, msf, range, node->off, 1);
                if (el.kind == SYMS_CvSymKind_FRAMEPROC){
                  syms_msf_read_struct_in_range(data, msf, el.range, 0, &frame_size);
                  break;
                }
              }
            }
          }
          
          //- extract regrel's info
          SYMS_CvRegrel32 regrel32 = {0};
          syms_msf_read_struct_in_range(data, msf, element.range, 0, &regrel32);
          
          SYMS_CvReg cv_reg = regrel32.reg;
          SYMS_U32 off = regrel32.reg_off;
          SYMS_Arch arch = syms_pdb_arch_from_dbg(dbg);
          // TODO(allen): report unimplemented architecture, unimplemented register conversion
          SYMS_RegSection sec = syms_pdb_reg_section_from_arch_reg(arch, cv_reg);
          
          //- determine if this is a stack register relative
          SYMS_B32 stack_reg = ((arch == SYMS_Arch_X64 && cv_reg == SYMS_CvReg_X64_RSP) ||
                                (arch == SYMS_Arch_X86 && cv_reg == SYMS_CvReg_X86_ESP));
          
          //- determine if this is a parameter
          SYMS_B32 is_param = (stack_reg && off > frame_size);
          
          //- build the location info addr:`reg + off`
          SYMS_EvalOpList list = {0};
          syms_eval_op_push(arena, &list, SYMS_EvalOp_RegRead, syms_eval_op_params_2u16(sec.off, sec.size));
          if (off != 0){
            syms_eval_op_encode_u(arena, &list, off);
            syms_eval_op_push(arena, &list, SYMS_EvalOp_Add, syms_eval_op_params(SYMS_EvalTypeGroup_U));
          }
          
          result.op_list = list;
          result.mode = SYMS_EvalMode_Address;
          result.is_parameter = is_param;
        }break;
        
        case SYMS_CvSymKind_GTHREAD32:
        case SYMS_CvSymKind_LTHREAD32:
        {
          SYMS_CvThread32 thread32 = {0};
          syms_msf_read_struct_in_range(data, msf, element.range, 0, &thread32);
          
          //- build the location info for addr:`TLS base + off`
          SYMS_EvalOpList list = {0};
          if (thread32.tls_off <= 0xFFFF){
            syms_eval_op_push(arena, &list, SYMS_EvalOp_TLSOff, syms_eval_op_params(thread32.tls_off));
          }
          else{
            syms_eval_op_push(arena, &list, SYMS_EvalOp_TLSOff, syms_eval_op_params(0xFFFF));
            syms_eval_op_encode_u(arena, &list, (thread32.tls_off - 0xFFFF));
            syms_eval_op_push(arena, &list, SYMS_EvalOp_Add, syms_eval_op_params(SYMS_EvalTypeGroup_U));
          }
          
          result.op_list = list;
          result.mode = SYMS_EvalMode_Address;
        }break;
      }
    }
  }
  return(result);
}

SYMS_API SYMS_LocRangeArray
syms_pdb_location_ranges_from_var_sid(SYMS_Arena *arena, SYMS_String8 data, SYMS_PdbDbgAccel *dbg,
                                      SYMS_CvUnitAccel *unit, SYMS_SymbolID sid){
  SYMS_LocRangeArray result = {0};
  if (!unit->leaf_set){
    //- setup accel
    SYMS_MsfAccel *msf = dbg->msf;
    
    //- read id
    SYMS_MsfRange range = syms_msf_range_from_sn(msf, unit->sn);
    SYMS_CvStub *stub = syms_cv_stub_from_unit_off(unit, SYMS_ID_u32_1(sid));
    
    //- parse symbol
    if (stub != 0){
      SYMS_CvElement element = syms_cv_element(data, msf, range, stub->off, 1);
      if (element.kind == SYMS_CvSymKind_LOCAL){
        SYMS_ArenaTemp scratch = syms_get_scratch(&arena, 1);
        
        //- gather ranges
        SYMS_LocRangeList list = {0};
        for (SYMS_CvStub *child = stub->first;
             child != 0;
             child = child->sibling_next){
          SYMS_CvElement child_element = syms_cv_element(data, msf, range, child->off, 1);
          SYMS_LocID loc_id = SYMS_ID_u32_u32(SYMS_CvSymbolIDKind_Off, child->off);
          
          //- determine handling path
          SYMS_B32 do_standard_range = syms_true;
          SYMS_U64 range_off = 0;
          switch (child_element.kind){
            case SYMS_CvSymKind_DEFRANGE_2005:
            case SYMS_CvSymKind_DEFRANGE2_2005:
            {
              // TODO(allen): Investigate these cases further
              do_standard_range = syms_false;
            }break;
            
            case SYMS_CvSymKind_DEFRANGE:
            {
              range_off = SYMS_MEMBER_OFFSET(SYMS_CvDefrange, range);
            }break;
            
            case SYMS_CvSymKind_DEFRANGE_SUBFIELD:
            {
              range_off = SYMS_MEMBER_OFFSET(SYMS_CvDefrangeSubfield, range);
            }break;
            
            case SYMS_CvSymKind_DEFRANGE_REGISTER:
            {
              range_off = SYMS_MEMBER_OFFSET(SYMS_CvDefrangeRegister, range);
            }break;
            
            case SYMS_CvSymKind_DEFRANGE_FRAMEPOINTER_REL:
            {
              range_off = SYMS_MEMBER_OFFSET(SYMS_CvDefrangeFramepointerRel, range);
            }break;
            
            case SYMS_CvSymKind_DEFRANGE_SUBFIELD_REGISTER:
            {
              range_off = SYMS_MEMBER_OFFSET(SYMS_CvDefrangeSubfieldRegister, range);
            }break;
            
            case SYMS_CvSymKind_DEFRANGE_FRAMEPOINTER_REL_FULL_SCOPE:
            {
              do_standard_range = syms_false;
              
              //- emit a single range that covers all U64
              SYMS_LocRangeNode *node = syms_push_array(scratch.arena, SYMS_LocRangeNode, 1);
              SYMS_QueuePush(list.first, list.last, node);
              list.count += 1;
              node->loc_range.vrange.min = 0;
              node->loc_range.vrange.max = SYMS_U64_MAX;
              node->loc_range.loc_id = loc_id;
            }break;
            
            case SYMS_CvSymKind_DEFRANGE_REGISTER_REL:
            {
              range_off = SYMS_MEMBER_OFFSET(SYMS_CvDefrangeRegisterRel, range);
            }break;
          }
          
          //- apply range handling
          if (do_standard_range){
            SYMS_CvLvarAddrRange addr_range = {0};
            if (syms_msf_read_struct_in_range(data, msf, child_element.range, range_off, &addr_range)){
              // setup main range
              SYMS_U64 section_voff = syms_pdb_voff_from_section_n(dbg, addr_range.sec);
              SYMS_U64 vaddr_base = section_voff + addr_range.off;
              SYMS_U64 vaddr_last = vaddr_base + addr_range.len;
              
              // setup gaps
              SYMS_U64 gap_off = range_off + sizeof(addr_range);
              SYMS_U64 gap_count = (child_element.range.size - gap_off)/sizeof(SYMS_CvLvarAddrGap);
              SYMS_U64 gap_opl = gap_off + gap_count*sizeof(SYMS_CvLvarAddrGap);
              
              // emit loop
              SYMS_U64 vaddr_cursor = vaddr_base;
              for (SYMS_U64 gap_cursor = gap_off;
                   gap_cursor < gap_opl;
                   gap_cursor += sizeof(SYMS_CvLvarAddrGap)){
                SYMS_CvLvarAddrGap gap = {0};
                syms_msf_read_struct_in_range(data, msf, child_element.range, gap_off, &gap);
                
                SYMS_U64 gap_first = vaddr_base + gap.off;
                
                // emit range
                {
                  SYMS_LocRangeNode *node = syms_push_array(scratch.arena, SYMS_LocRangeNode, 1);
                  SYMS_QueuePush(list.first, list.last, node);
                  list.count += 1;
                  node->loc_range.vrange.min = vaddr_cursor;
                  node->loc_range.vrange.max = gap_first;
                  node->loc_range.loc_id = loc_id;
                }
                
                // advance vaddr cursor
                vaddr_cursor = gap_first + gap.len;
              }
              
              // emit range
              {
                SYMS_LocRangeNode *node = syms_push_array(scratch.arena, SYMS_LocRangeNode, 1);
                SYMS_QueuePush(list.first, list.last, node);
                list.count += 1;
                node->loc_range.vrange.min = vaddr_cursor;
                node->loc_range.vrange.max = vaddr_last;
                node->loc_range.loc_id = loc_id;
              }
            }
          }
        }
        
        //- flatten list
        SYMS_LocRange *loc_ranges = syms_push_array(arena, SYMS_LocRange, list.count);
        SYMS_LocRange *loc_range_ptr = loc_ranges;
        for (SYMS_LocRangeNode *node = list.first;
             node != 0;
             node = node->next, loc_range_ptr += 1){
          syms_memmove(loc_range_ptr, &node->loc_range, sizeof(node->loc_range));
        }
        
        //- fill result
        result.loc_ranges = loc_ranges;
        result.count = list.count;
        
        syms_release_scratch(scratch);
      }
    }
  }
  return(result);
}

SYMS_API SYMS_Location
syms_pdb_location_from_id(SYMS_Arena *arena, SYMS_String8 data, SYMS_PdbDbgAccel *dbg,
                          SYMS_CvUnitAccel *unit, SYMS_LocID loc_id){
  SYMS_Location result = {0};
  if (!unit->leaf_set){
    // setup accel
    SYMS_MsfAccel *msf = dbg->msf;
    
    // read id
    SYMS_MsfRange range = syms_msf_range_from_sn(msf, unit->sn);
    SYMS_CvStub *stub = syms_cv_stub_from_unit_off(unit, SYMS_ID_u32_1(loc_id));
    
    // parse symbol
    if (stub != 0){
      
      // determine if this location is part of a parameter variable
      SYMS_B32 is_param = syms_false;
      {
        SYMS_CvStub *parent = stub->parent;
        if (parent != 0){
          SYMS_CvElement el = syms_cv_element(data, msf, range, parent->off, 1);
          if (el.kind == SYMS_CvSymKind_LOCAL){
            SYMS_CvLocal slocal = {0};
            syms_msf_read_struct_in_range(data, msf, el.range, 0, &slocal);
            if (slocal.flags & SYMS_CvLocalFlag_PARAM){
              is_param = syms_true;
            }
          }
        }
      }
      result.is_parameter = is_param;
      
      // construct location from this symbol
      SYMS_CvElement element = syms_cv_element(data, msf, range, stub->off, 1);
      
      switch (element.kind){
        case SYMS_CvSymKind_DEFRANGE_2005:
        case SYMS_CvSymKind_DEFRANGE2_2005:
        {
          // TODO(allen): No known cases; Don't know how to interpret
        }break;
        
        case SYMS_CvSymKind_DEFRANGE:
        {
          // TODO(allen): No known cases; SYMS_CvDefrange 'program' - don't know how to interpret
        }break;
        
        case SYMS_CvSymKind_DEFRANGE_SUBFIELD:
        {
          // TODO(allen): No known cases; SYMS_CvDefrangeSubfield 'program' - don't know how to interpret
          // TODO(allen): SYMS_CvDefrangeSubfield 'off_in_parent' - not a supported concept
        }break;
        
        case SYMS_CvSymKind_DEFRANGE_REGISTER:
        {
          // extract info
          SYMS_CvDefrangeRegister defrange_register = {0};
          syms_msf_read_struct_in_range(data, msf, element.range, 0, &defrange_register);
          
          SYMS_CvReg cv_reg = defrange_register.reg;
          SYMS_Arch arch = syms_pdb_arch_from_dbg(dbg);
          SYMS_RegSection sec = syms_pdb_reg_section_from_arch_reg(arch, cv_reg);
          
          // location info ops
          SYMS_EvalOpList list = {0};
          syms_eval_op_encode_reg_section(arena, &list, sec);
          
          // fill result
          result.op_list = list;
          result.mode = SYMS_EvalMode_Register;
        }break;
        
        case SYMS_CvSymKind_DEFRANGE_FRAMEPOINTER_REL:
        case SYMS_CvSymKind_DEFRANGE_FRAMEPOINTER_REL_FULL_SCOPE:
        {
          // extract framepointer register
          SYMS_RegSection sec = syms_pdb_reg_section_from_framepointer(data, dbg, range, stub);
          
          // extract offset
          SYMS_U32 offset = 0;
          syms_msf_read_struct_in_range(data, msf, element.range, 0, &offset);
          
          // location info ops
          SYMS_EvalOpList list = {0};
          syms_eval_op_push(arena, &list, SYMS_EvalOp_RegRead, syms_eval_op_params_2u16(sec.off, sec.size));
          if (offset != 0){
            syms_eval_op_encode_u(arena, &list, offset);
            syms_eval_op_push(arena, &list, SYMS_EvalOp_Add, syms_eval_op_params(SYMS_EvalTypeGroup_U));
          }
          
          // fill result
          result.op_list = list;
          result.mode = SYMS_EvalMode_Address;
        }break;
        
        case SYMS_CvSymKind_DEFRANGE_SUBFIELD_REGISTER:
        {
          // TODO(allen): SYMS_CvDefrangeSubfieldRegister 'off_in_parent' - not a supported concept
        }break;
        
        case SYMS_CvSymKind_DEFRANGE_REGISTER_REL:
        {
          // extract info
          SYMS_CvDefrangeRegisterRel defrange_register_rel = {0};
          syms_msf_read_struct_in_range(data, msf, element.range, 0, &defrange_register_rel);
          
          SYMS_CvReg cv_reg = defrange_register_rel.reg;
          SYMS_U32 off = defrange_register_rel.reg_off;
          SYMS_Arch arch = syms_pdb_arch_from_dbg(dbg);
          SYMS_RegSection sec = syms_pdb_reg_section_from_arch_reg(arch, cv_reg);
          
          // location info ops
          SYMS_EvalOpList list = {0};
          syms_eval_op_push(arena, &list, SYMS_EvalOp_RegRead, syms_eval_op_params_2u16(sec.off, sec.size));
          if (off != 0){
            syms_eval_op_encode_u(arena, &list, off);
            syms_eval_op_push(arena, &list, SYMS_EvalOp_Add, syms_eval_op_params(SYMS_EvalTypeGroup_U));
          }
          
          // fill result
          result.op_list = list;
          result.mode = SYMS_EvalMode_Address;;
        }break;
      }
      
    }
  }
  return(result);
}


////////////////////////////////
//~ allen: PDB Member Info

// cv parse

SYMS_API void
syms_pdb__field_list_parse(SYMS_Arena *arena, SYMS_String8 data, SYMS_MsfAccel *msf,
                           SYMS_CvUnitAccel *unit, SYMS_U32 index, SYMS_PdbMemStubList *out){
  // get field list from ti
  SYMS_CvStub *list_stub = syms_cv_stub_from_unit_index(unit, index);
  
  // parse loop
  if (list_stub != 0){
    // leaf range
    SYMS_MsfRange range = syms_msf_range_from_sn(msf, unit->sn);
    
    // use the stubs from initial parse
    for (SYMS_CvStub *field_stub = list_stub->first;
         field_stub != 0;
         field_stub = field_stub->sibling_next){
      // read kind
      SYMS_U32 lf_off = field_stub->off;
      SYMS_CvLeaf lf_kind = 0;
      syms_msf_read_struct_in_range(data, msf, range, lf_off, &lf_kind);
      
      // recurse into indexes
      if (lf_kind == SYMS_CvLeaf_INDEX){
        // TODO(allen): infinite recursion breaker.
        SYMS_CvLeafIndex index = {0};
        if (syms_msf_read_struct_in_range(data, msf, range, lf_off, &index)){
          syms_pdb__field_list_parse(arena, data, msf, unit, index.itype, out);
        }
      }
      
      // gather members
      else{
        switch (lf_kind){
          case SYMS_CvLeaf_MEMBER:
          case SYMS_CvLeaf_STMEMBER:
          case SYMS_CvLeaf_ONEMETHOD:
          case SYMS_CvLeaf_NESTTYPE:
          case SYMS_CvLeaf_NESTTYPEEX:
          case SYMS_CvLeaf_BCLASS:
          case SYMS_CvLeaf_VBCLASS:
          case SYMS_CvLeaf_IVBCLASS:
          case SYMS_CvLeaf_VFUNCTAB:
          case SYMS_CvLeaf_VFUNCOFF:
          {
            SYMS_CvMemStubNode *node = syms_push_array(arena, SYMS_CvMemStubNode, 1);
            node->name = syms_push_string_copy(arena, field_stub->name);
            node->num = field_stub->num;
            node->off = field_stub->off;
            node->off2 = 0;
            SYMS_QueuePush(out->first, out->last, node);
            out->mem_count += 1;
          }break;
          
          case SYMS_CvLeaf_METHOD:
          {
            SYMS_U32 off = field_stub->off;
            // TODO(allen): do we get anything useful from this num?
            SYMS_U32 num = field_stub->num;
            SYMS_String8 name = syms_push_string_copy(arena, field_stub->name);
            
            SYMS_U32 method_off = lf_off + sizeof(lf_kind);
            SYMS_CvLeafMethod method = {0};
            syms_msf_read_struct_in_range(data, msf, range, method_off, &method);
            
            SYMS_CvStub *method_list_stub = syms_cv_stub_from_unit_index(unit, method.itype_list);
            
            if (method_list_stub != 0){
              SYMS_U32 method_list_off = method_list_stub->off;
              SYMS_CvStub *method_stub = method_list_stub->first;
              for (SYMS_U32 i = 0;
                   i < method.count && method_stub != 0;
                   i += 1, method_stub = method_stub->sibling_next){
                SYMS_CvMemStubNode *node = syms_push_array(arena, SYMS_CvMemStubNode, 1);
                node->name = name;
                node->num = num;
                node->off = off;
                node->off2 = method_stub->off - method_list_off;
                SYMS_QueuePush(out->first, out->last, node);
                out->mem_count += 1;
              }
            }
          }break;
        }
      }
    }
  }
}

// main api

SYMS_API SYMS_CvMemsAccel*
syms_pdb_mems_accel_from_sid(SYMS_Arena *arena, SYMS_String8 data, SYMS_PdbDbgAccel *dbg,
                             SYMS_CvUnitAccel *unit, SYMS_SymbolID id){
  // setup accels
  SYMS_MsfAccel *msf = dbg->msf;
  
  // dispatch to leaf
  SYMS_CvMemsAccel *result = (SYMS_CvMemsAccel*)&syms_format_nil;
  if (unit->leaf_set){
    
    // read id
    SYMS_CvStub *stub = 0;
    if (SYMS_ID_u32_0(id) == SYMS_CvSymbolIDKind_Index){
      stub = syms_cv_stub_from_unit_index(unit, SYMS_ID_u32_1(id));
    }
    
    // extract field list from stub
    SYMS_CvTypeIndex ti = 0;
    if (stub != 0){
      SYMS_MsfRange range = syms_msf_range_from_sn(msf, unit->sn);
      SYMS_CvElement element = syms_cv_element(data, msf, range, stub->off, 1);
      switch (element.kind){
        case SYMS_CvLeaf_CLASS:
        case SYMS_CvLeaf_STRUCTURE:
        case SYMS_CvLeaf_INTERFACE:
        {
          SYMS_CvLeafStruct struct_ = {0};
          if (syms_msf_read_struct_in_range(data, msf, element.range, 0, &struct_)){
            ti = struct_.field;
          }
        }break;
        
        case SYMS_CvLeaf_UNION:
        {
          SYMS_CvLeafUnion union_ = {0};
          if (syms_msf_read_struct_in_range(data, msf, element.range, 0, &union_)){
            ti = union_.field;
          }
        }break;
        
        case SYMS_CvLeaf_CLASSPTR:
        case SYMS_CvLeaf_CLASSPTR2:
        {
          SYMS_CvLeafClassPtr class_ptr = {0};
          if (syms_msf_read_struct_in_range(data, msf, element.range, 0, &class_ptr)){
            ti = class_ptr.field;
          }
        }break;
      }
    }
    
    if (ti != 0){
      // recursively gather members
      SYMS_PdbMemStubList list = {0};
      syms_pdb__field_list_parse(arena, data, msf, unit, ti, &list);
      
      // construct result
      SYMS_String8 type_name = syms_push_string_copy(arena, stub->name);
      SYMS_CvMemStubNode **members = syms_push_array(arena, SYMS_CvMemStubNode*, list.mem_count);
      {
        SYMS_U64 i = 0;
        for (SYMS_CvMemStubNode *node = list.first;
             node != 0;
             node = node->next, i += 1){
          members[i] = node;
        }
      }
      
      result = syms_push_array(arena, SYMS_CvMemsAccel, 1);
      syms_memzero_struct(result);
      result->format = SYMS_FileFormat_PDB;
      result->type_name = type_name;
      result->count = list.mem_count;
      result->members = members;
    }
  }
  
  return(result);
}

SYMS_API SYMS_U64
syms_pdb_mem_count_from_mems(SYMS_CvMemsAccel *mems){
  SYMS_U64 result = mems->count;
  return(result);
}

SYMS_API void
syms_pdb__fill_method_mem_info(SYMS_String8 data, SYMS_MsfAccel *msf, SYMS_MsfRange range, SYMS_U32 vbaseoff_off,
                               SYMS_CvFieldAttribs attribs, SYMS_String8 type_name, SYMS_String8 name, 
                               SYMS_MemInfo *out){
  SYMS_CvMemberAccess access = SYMS_CvFieldAttribs_Extract_ACCESS(attribs);
  SYMS_MemVisibility visibility = syms_mem_visibility_from_member_access(access);
  
  SYMS_U32 vbaseoff = 0;
  SYMS_B32 is_virtual = syms_false;
  SYMS_B32 is_static = syms_false;
  
  SYMS_CvMethodProp mprop = SYMS_CvFieldAttribs_Extract_MPROP(attribs);
  switch (mprop){
    case SYMS_CvMethodProp_INTRO:
    case SYMS_CvMethodProp_PUREINTRO:
    {
      syms_msf_read_struct_in_range(data, msf, range, vbaseoff_off, &vbaseoff);
    }//fallthrough;
    case SYMS_CvMethodProp_VIRTUAL:
    case SYMS_CvMethodProp_PUREVIRT:
    {
      is_virtual = syms_true;
    }break;
    case SYMS_CvMethodProp_STATIC:
    {
      is_static = syms_true;
    }break;
  }
  
  SYMS_MemKind kind = SYMS_MemKind_Method;
  if (is_static){
    kind = SYMS_MemKind_StaticMethod;
  }
  
  SYMS_MemFlags flags = 0;
  if (is_virtual){
    flags |= SYMS_MemFlag_Virtual;
  }
  
  SYMS_B32 destructor_flag = syms_false;
  SYMS_String8 name_destructor_skipped = name;
  if (name.size > 0 && name.str[0] == '~'){
    name_destructor_skipped.str += 1;
    name_destructor_skipped.size -= 1;
    destructor_flag = syms_true;
  }
  
  SYMS_String8 type_name_trunc = syms_string_trunc_symbol_heuristic(type_name);
  SYMS_String8 name_trunc = syms_string_trunc_symbol_heuristic(name_destructor_skipped);
  
  if (syms_string_match(type_name_trunc, name_trunc, 0)){
    if (destructor_flag){
      flags |= SYMS_MemFlag_Destructor;
    }
    else{
      flags |= SYMS_MemFlag_Constructor;
    }
  }
  
  out->kind = kind;
  out->visibility = visibility;
  out->flags = flags;
  out->virtual_off = vbaseoff;
}

SYMS_API SYMS_MemInfo
syms_pdb_mem_info_from_number(SYMS_Arena *arena, SYMS_String8 data, SYMS_PdbDbgAccel *dbg,
                              SYMS_CvUnitAccel *unit, SYMS_CvMemsAccel *mems, SYMS_U64 n){
  // setup accels
  SYMS_MsfAccel *msf = dbg->msf;
  
  SYMS_MemInfo result = {SYMS_MemKind_Null};
  if (1 <= n && n <= mems->count){
    SYMS_U64 index = n - 1;
    SYMS_CvMemStubNode *stub = mems->members[index];
    
    if (stub != 0){
      // read kind
      SYMS_U32 lf_off = stub->off;
      SYMS_MsfRange range = syms_msf_range_from_sn(msf, unit->sn);
      SYMS_CvLeaf lf_kind = 0;
      syms_msf_read_struct_in_range(data, msf, range, lf_off, &lf_kind);
      
      SYMS_U32 data_off = lf_off + sizeof(lf_kind);
      
      switch (lf_kind){
        default:break;
        
        case SYMS_CvLeaf_MEMBER:
        {
          SYMS_CvLeafMember member = {0};
          syms_msf_read_struct_in_range(data, msf, range, data_off, &member);
          
          SYMS_CvMemberAccess access = SYMS_CvFieldAttribs_Extract_ACCESS(member.attribs);
          SYMS_MemVisibility visibility = syms_mem_visibility_from_member_access(access);
          
          result.kind = SYMS_MemKind_DataField;
          result.name = syms_push_string_copy(arena, stub->name);
          result.visibility = visibility;
          result.off = stub->num;
        }break;
        
        case SYMS_CvLeaf_STMEMBER:
        {
          SYMS_CvLeafStMember stmember = {0};
          syms_msf_read_struct_in_range(data, msf, range, data_off, &stmember);
          
          SYMS_CvMemberAccess access = SYMS_CvFieldAttribs_Extract_ACCESS(stmember.attribs);
          SYMS_MemVisibility visibility = syms_mem_visibility_from_member_access(access);
          
          result.kind = SYMS_MemKind_StaticData;
          result.name = syms_push_string_copy(arena, stub->name);
          result.visibility = visibility;
        }break;
        
        case SYMS_CvLeaf_METHOD:
        {
          SYMS_CvLeafMethod method = {0};
          syms_msf_read_struct_in_range(data, msf, range, data_off, &method);
          
          SYMS_CvStub *method_list_stub = syms_cv_stub_from_unit_index(unit, method.itype_list);
          
          if (method_list_stub != 0){
            SYMS_U32 rec_off = method_list_stub->off + stub->off2;
            SYMS_CvMethod methodrec = {0};
            syms_msf_read_struct_in_range(data, msf, range, rec_off, &methodrec);
            syms_pdb__fill_method_mem_info(data, msf, range, rec_off + sizeof(methodrec), methodrec.attribs,
                                           mems->type_name, stub->name, &result);
            result.name = syms_push_string_copy(arena, stub->name);
          }
        }break;
        
        case SYMS_CvLeaf_ONEMETHOD:
        {
          SYMS_CvLeafOneMethod onemethod = {0};
          syms_msf_read_struct_in_range(data, msf, range, data_off, &onemethod);
          
          syms_pdb__fill_method_mem_info(data, msf, range, data_off + sizeof(onemethod), onemethod.attribs,
                                         mems->type_name, stub->name, &result);
          result.name = syms_push_string_copy(arena, stub->name);
        }break;
        
        case SYMS_CvLeaf_NESTTYPE:
        case SYMS_CvLeaf_NESTTYPEEX:
        {
          SYMS_MemVisibility visibility = SYMS_MemVisibility_Public;
          if (lf_kind == SYMS_CvLeaf_NESTTYPEEX){
            SYMS_CvLeafNestTypeEx nest_type_ex = {0};
            syms_msf_read_struct_in_range(data, msf, range, data_off, &nest_type_ex);
            SYMS_CvMemberAccess access = SYMS_CvFieldAttribs_Extract_ACCESS(nest_type_ex.attribs);
            visibility = syms_mem_visibility_from_member_access(access);
          }
          
          result.kind = SYMS_MemKind_NestedType;
          result.name = syms_push_string_copy(arena, stub->name);
          result.visibility = visibility;
        }break;
        
        case SYMS_CvLeaf_BCLASS:
        {
          SYMS_CvLeafBClass bclass = {0};
          syms_msf_read_struct_in_range(data, msf, range, data_off, &bclass);
          
          SYMS_CvMemberAccess access = SYMS_CvFieldAttribs_Extract_ACCESS(bclass.attribs);
          SYMS_MemVisibility visibility = syms_mem_visibility_from_member_access(access);
          
          result.kind = SYMS_MemKind_BaseClass;
          result.name = syms_push_string_copy(arena, stub->name);
          result.visibility = visibility;
          result.off = stub->num;
        }break;
        
        case SYMS_CvLeaf_VBCLASS:
        case SYMS_CvLeaf_IVBCLASS:
        {
          SYMS_CvLeafVBClass vbclass = {0};
          syms_msf_read_struct_in_range(data, msf, range, data_off, &vbclass);
          
          SYMS_CvMemberAccess access = SYMS_CvFieldAttribs_Extract_ACCESS(vbclass.attribs);
          SYMS_MemVisibility visibility = syms_mem_visibility_from_member_access(access);
          
          result.kind = SYMS_MemKind_VBaseClassPtr;
          result.name = syms_push_string_copy(arena, stub->name);
          result.visibility = visibility;
          result.off = stub->num;
        }break;
        
        case SYMS_CvLeaf_VFUNCTAB:
        {
          SYMS_CvLeafVFuncTab vfunctab = {0};
          syms_msf_read_struct_in_range(data, msf, range, data_off, &vfunctab);
          
          result.kind = SYMS_MemKind_VTablePtr;
          result.off = 0;
        }break;
        
        case SYMS_CvLeaf_VFUNCOFF:
        {
          SYMS_CvLeafVFuncOff vfuncoff = {0};
          syms_msf_read_struct_in_range(data, msf, range, data_off, &vfuncoff);
          
          result.kind = SYMS_MemKind_VTablePtr;
          result.off = vfuncoff.off;
        }break;
      }
    }
  }
  
  return(result);
}

SYMS_API SYMS_USID
syms_pdb_type_from_mem_number(SYMS_String8 data, SYMS_PdbDbgAccel *dbg, SYMS_CvUnitAccel *unit,
                              SYMS_CvMemsAccel *mems, SYMS_U64 n){
  // setup accels
  SYMS_MsfAccel *msf = dbg->msf;
  
  SYMS_USID result = {0};
  if (1 <= n && n <= mems->count){
    SYMS_U64 index = n - 1;
    SYMS_CvMemStubNode *stub = mems->members[index];
    
    if (stub != 0){
      SYMS_U32 lf_off = stub->off;
      SYMS_MsfRange range = syms_msf_range_from_sn(msf, unit->sn);
      SYMS_CvLeaf lf_kind = 0;
      syms_msf_read_struct_in_range(data, msf, range, lf_off, &lf_kind);
      
      SYMS_U32 data_off = lf_off + sizeof(lf_kind);
      
      result.uid = unit->uid;
      
      switch (lf_kind){
        default:
        {
          result.uid = 0;
        }break;
        
        case SYMS_CvLeaf_MEMBER:
        {
          SYMS_CvLeafMember member = {0};
          syms_msf_read_struct_in_range(data, msf, range, data_off, &member);
          
          result.sid = SYMS_ID_u32_u32(SYMS_CvSymbolIDKind_Index, member.itype);
        }break;
        
        case SYMS_CvLeaf_STMEMBER:
        {
          SYMS_CvLeafStMember stmember = {0};
          syms_msf_read_struct_in_range(data, msf, range, data_off, &stmember);
          result.sid = SYMS_ID_u32_u32(SYMS_CvSymbolIDKind_Index, stmember.itype);
        }break;
        
        case SYMS_CvLeaf_NESTTYPE:
        case SYMS_CvLeaf_NESTTYPEEX:
        {
          SYMS_CvLeafNestTypeEx nest_type_ex = {0};
          syms_msf_read_struct_in_range(data, msf, range, data_off, &nest_type_ex);
          
          result.sid = SYMS_ID_u32_u32(SYMS_CvSymbolIDKind_Index, nest_type_ex.itype);
        }break;
        
        case SYMS_CvLeaf_BCLASS:
        {
          SYMS_CvLeafBClass bclass = {0};
          syms_msf_read_struct_in_range(data, msf, range, data_off, &bclass);
          
          result.sid = SYMS_ID_u32_u32(SYMS_CvSymbolIDKind_Index, bclass.itype);
        }break;
        
        case SYMS_CvLeaf_VBCLASS:
        case SYMS_CvLeaf_IVBCLASS:
        {
          SYMS_CvLeafVBClass vbclass = {0};
          syms_msf_read_struct_in_range(data, msf, range, data_off, &vbclass);
          
          result.sid = SYMS_ID_u32_u32(SYMS_CvSymbolIDKind_Index, vbclass.itype);
        }break;
      }
    }
  }
  
  return(result);
}

SYMS_API SYMS_SigInfo
syms_pdb_sig_info_from_mem_number(SYMS_Arena *arena, SYMS_String8 data, SYMS_PdbDbgAccel *dbg,
                                  SYMS_CvUnitAccel *unit, SYMS_CvMemsAccel *mems, SYMS_U64 n){
  // setup accels
  SYMS_MsfAccel *msf = dbg->msf;
  
  SYMS_SigInfo result = {0};
  if (1 <= n && n <= mems->count){
    SYMS_U64 index = n - 1;
    SYMS_CvMemStubNode *stub = mems->members[index];
    
    if (stub != 0){
      // read kind
      SYMS_U32 lf_off = stub->off;
      SYMS_MsfRange range = syms_msf_range_from_sn(msf, unit->sn);
      SYMS_CvLeaf lf_kind = 0;
      syms_msf_read_struct_in_range(data, msf, range, lf_off, &lf_kind);
      
      SYMS_U32 data_off = lf_off + sizeof(lf_kind);
      
      switch (lf_kind){
        default:
        {}break;
        
        case SYMS_CvLeaf_METHOD:
        {
          SYMS_CvLeafMethod method = {0};
          syms_msf_read_struct_in_range(data, msf, range, data_off, &method);
          
          SYMS_CvStub *method_list_stub = syms_cv_stub_from_unit_index(unit, method.itype_list);
          if (method_list_stub != 0){
            SYMS_U32 rec_off = method_list_stub->off + stub->off2;
            SYMS_CvMethod methodrec = {0};
            syms_msf_read_struct_in_range(data, msf, range, rec_off, &methodrec);
            result = syms_pdb_sig_info_from_sig_index(arena, data, dbg, unit, methodrec.itype);
          }
        }break;
        
        case SYMS_CvLeaf_ONEMETHOD:
        {
          SYMS_CvLeafOneMethod onemethod = {0};
          syms_msf_read_struct_in_range(data, msf, range, data_off, &onemethod);
          result = syms_pdb_sig_info_from_sig_index(arena, data, dbg, unit, onemethod.itype);
        }break;
      }
    }
  }
  
  return(result);
}

SYMS_API SYMS_EnumMemberArray
syms_pdb_enum_member_array_from_sid(SYMS_Arena *arena, SYMS_String8 data, SYMS_PdbDbgAccel *dbg,
                                    SYMS_CvUnitAccel *unit, SYMS_SymbolID sid){
  // setup accels
  SYMS_MsfAccel *msf = dbg->msf;
  
  // dispatch to leaf
  SYMS_EnumMemberArray result = {0};
  if (unit->leaf_set){
    SYMS_MsfRange range = syms_msf_range_from_sn(msf, unit->sn);
    
    // read id
    SYMS_CvStub *stub = 0;
    if (SYMS_ID_u32_0(sid) == SYMS_CvSymbolIDKind_Index){
      stub = syms_cv_stub_from_unit_index(unit, SYMS_ID_u32_1(sid));
    }
    
    // extract field list from stub
    SYMS_CvTypeIndex ti = 0;
    if (stub != 0){
      SYMS_CvElement element = syms_cv_element(data, msf, range, stub->off, 1);
      if (element.kind == SYMS_CvLeaf_ENUM){
        SYMS_CvLeafEnum enum_ = {0};
        if (syms_msf_read_struct_in_range(data, msf, element.range, 0, &enum_)){
          ti = enum_.field;
        }
      }
    }
    
    // get field list from ti
    SYMS_CvStub *list_stub = syms_cv_stub_from_unit_index(unit, ti);
    
    // parse loop
    if (list_stub != 0){
      // count the enumerates
      SYMS_U64 count = 0;
      for (SYMS_CvStub *field_stub = list_stub->first;
           field_stub != 0;
           field_stub = field_stub->sibling_next){
        SYMS_U32 lf_off = field_stub->off;
        SYMS_CvLeaf lf_kind = 0;
        syms_msf_read_struct_in_range(data, msf, range, lf_off, &lf_kind);
        if (lf_kind == SYMS_CvLeaf_ENUMERATE){
          count += 1;
        }
      }
      
      // fill the enumerates array
      result.count = count;
      result.enum_members = syms_push_array(arena, SYMS_EnumMember, count);
      
      SYMS_EnumMember *enum_mem_ptr = result.enum_members;
      for (SYMS_CvStub *field_stub = list_stub->first;
           field_stub != 0;
           field_stub = field_stub->sibling_next){
        SYMS_U32 lf_off = field_stub->off;
        SYMS_CvLeaf lf_kind = 0;
        syms_msf_read_struct_in_range(data, msf, range, lf_off, &lf_kind);
        if (lf_kind == SYMS_CvLeaf_ENUMERATE){
          enum_mem_ptr->name = syms_push_string_copy(arena, field_stub->name);
          enum_mem_ptr->val = field_stub->num;
          enum_mem_ptr += 1;
        }
      }
    }
  }
  
  return(result);
}


////////////////////////////////
//~ allen: PDB Procedure Info

// main api

SYMS_API SYMS_UnitIDAndSig
syms_pdb_proc_sig_handle_from_id(SYMS_String8 data, SYMS_PdbDbgAccel *dbg, SYMS_CvUnitAccel *unit,
                                 SYMS_SymbolID id){
  SYMS_UnitIDAndSig result = {0};
  
  if (!unit->leaf_set){
    // setup accels
    SYMS_MsfAccel *msf = dbg->msf;
    
    // read id
    SYMS_MsfRange range = syms_msf_range_from_sn(msf, unit->sn);
    SYMS_CvStub *stub = syms_cv_stub_from_unit_off(unit, SYMS_ID_u32_1(id));
    
    // read proc
    if (stub != 0){
      SYMS_CvElement element = syms_cv_element(data, msf, range, stub->off, 1);
      switch (element.kind){
        default:break;
        
        case SYMS_CvSymKind_LPROC32_ID:
        case SYMS_CvSymKind_GPROC32_ID:
        case SYMS_CvSymKind_LPROC32:
        case SYMS_CvSymKind_GPROC32:
        {
          SYMS_CvProc32 proc32 = {0};
          if (syms_msf_read_struct_in_range(data, msf, element.range, 0, &proc32)){
            result.uid = dbg->type_uid;
            result.sig.v = proc32.itype;
          }
        }break;
      }
    }
  }
  
  return(result);
}

SYMS_API SYMS_SigInfo
syms_pdb_sig_info_from_handle(SYMS_Arena *arena, SYMS_String8 data, SYMS_PdbDbgAccel *dbg,
                              SYMS_CvUnitAccel *unit, SYMS_SigHandle handle){
  SYMS_SigInfo result = {0};
  if (unit->leaf_set){
    result = syms_pdb_sig_info_from_sig_index(arena, data, dbg, unit, handle.v);
  }
  return(result);
}

SYMS_API SYMS_U64RangeArray
syms_pdb_scope_vranges_from_sid(SYMS_Arena *arena, SYMS_String8 data, SYMS_PdbDbgAccel *dbg,
                                SYMS_CvUnitAccel *unit, SYMS_SymbolID sid){
  SYMS_U64RangeArray result = {0};
  if (!unit->leaf_set){
    // setup accels
    SYMS_MsfAccel *msf = dbg->msf;
    
    // read id
    SYMS_MsfRange range = syms_msf_range_from_sn(msf, unit->sn);
    SYMS_CvStub *stub = syms_cv_stub_from_unit_off(unit, SYMS_ID_u32_1(sid));
    
    // read proc
    if (stub != 0){
      SYMS_CvElement element = syms_cv_element(data, msf, range, stub->off, 1);
      switch (element.kind){
        default:break;
        
        case SYMS_CvSymKind_LPROC32_ID:
        case SYMS_CvSymKind_GPROC32_ID:
        case SYMS_CvSymKind_LPROC32:
        case SYMS_CvSymKind_GPROC32:
        {
          SYMS_CvProc32 proc32 = {0};
          if (syms_msf_read_struct_in_range(data, msf, element.range, 0, &proc32)){
            SYMS_U64 section_voff = syms_pdb_voff_from_section_n(dbg, proc32.sec);
            
            SYMS_U64Range range = {0};
            range.min = section_voff + proc32.off;
            range.max = range.min + proc32.len;
            
            result.count = 1;
            result.ranges = syms_push_array(arena, SYMS_U64Range, 1);
            result.ranges[0] = range;
          }
        }break;
        
        case SYMS_CvSymKind_BLOCK32:
        {
          SYMS_CvBlock32 block32 = {0};
          if (syms_msf_read_struct_in_range(data, msf, element.range, 0, &block32)){
            SYMS_U64 section_voff = syms_pdb_voff_from_section_n(dbg, block32.sec);
            
            SYMS_U64Range range = {0};
            range.min = section_voff + block32.off;
            range.max = range.min + block32.len;
            
            result.count = 1;
            result.ranges = syms_push_array(arena, SYMS_U64Range, 1);
            result.ranges[0] = range;
          }
        }break;
      }
    }
  }
  
  return(result);
}

SYMS_API SYMS_SymbolIDArray
syms_pdb_scope_children_from_sid(SYMS_Arena *arena, SYMS_String8 data, SYMS_PdbDbgAccel *dbg,
                                 SYMS_CvUnitAccel *unit, SYMS_SymbolID id){
  // dispatch to sym or leaf
  SYMS_ArenaTemp scratch = syms_get_scratch(&arena, 1);
  SYMS_SymbolIDList list = {0};
  if (!unit->leaf_set){
    // setup accels
    SYMS_MsfAccel *msf = dbg->msf;
    
    // read id
    SYMS_MsfRange range = syms_msf_range_from_sn(msf, unit->sn);
    SYMS_CvStub *stub = syms_cv_stub_from_unit_off(unit, SYMS_ID_u32_1(id));
    
    // build list
    if (stub != 0){
      SYMS_CvElement element = syms_cv_element(data, msf, range, stub->off, 1);
      switch (element.kind){
        default:break;
        
        case SYMS_CvSymKind_LPROC32_ID:
        case SYMS_CvSymKind_GPROC32_ID:
        case SYMS_CvSymKind_LPROC32:
        case SYMS_CvSymKind_GPROC32:
        case SYMS_CvSymKind_BLOCK32:
        {
          for (SYMS_CvStub *child = stub->first;
               child != 0;
               child = child->sibling_next){
            SYMS_SymbolIDNode *node = syms_push_array(arena, SYMS_SymbolIDNode, 1);
            SYMS_QueuePush(list.first, list.last, node);
            node->id = SYMS_ID_u32_u32(SYMS_CvSymbolIDKind_Off, child->off);
            list.count += 1;
          }
        }break;
      }
    }
  }
  
  // convert list to array
  SYMS_SymbolIDArray result = syms_sid_array_from_list(arena, &list);
  
  syms_release_scratch(scratch);
  
  return(result);
}

////////////////////////////////
//~ allen: PDB Signature Info

// pdb specific helper
SYMS_API SYMS_SigInfo
syms_pdb_sig_info_from_sig_index(SYMS_Arena *arena, SYMS_String8 data, SYMS_PdbDbgAccel *dbg,
                                 SYMS_CvUnitAccel *unit, SYMS_CvTypeIndex index){
  SYMS_SigInfo result = {0};
  
  if (unit->leaf_set){
    // setup accels
    SYMS_MsfAccel *msf = dbg->msf;
    
    // get unit range
    SYMS_MsfRange range = syms_msf_range_from_sn(msf, unit->sn);
    
    // get member function data
    SYMS_CvTypeIndex arg_itype = 0;
    SYMS_CvTypeIndex ret_itype = 0;
    SYMS_CvTypeIndex this_itype = 0;
    
    {
      SYMS_CvStub *sig_stub = syms_cv_stub_from_unit_index(unit, index);
      if (sig_stub != 0){
        SYMS_CvElement sig_element = syms_cv_element(data, msf, range, sig_stub->off, 1);
        switch (sig_element.kind){
          case SYMS_CvLeaf_PROCEDURE:
          {
            SYMS_CvLeafProcedure proc = {0};
            syms_msf_read_struct_in_range(data, msf, sig_element.range, 0, &proc);
            arg_itype = proc.arg_itype;
            ret_itype = proc.ret_itype;
          }break;
          
          case SYMS_CvLeaf_MFUNCTION:
          {
            SYMS_CvLeafMFunction mfunc = {0};
            syms_msf_read_struct_in_range(data, msf, sig_element.range, 0, &mfunc);
            arg_itype = mfunc.arg_itype;
            ret_itype = mfunc.ret_itype;
            this_itype = mfunc.this_itype;
          }break;
        }
      }
    }
    
    // get args list
    SYMS_CvLeafArgList args_list = {0};
    SYMS_MsfRange args_list_range = {0};
    if (arg_itype != 0){
      SYMS_CvStub *args_stub = syms_cv_stub_from_unit_index(unit, arg_itype);
      if (args_stub != 0){
        SYMS_CvElement args_element = syms_cv_element(data, msf, range, args_stub->off, 1);
        if (args_element.kind == SYMS_CvLeaf_ARGLIST){
          syms_msf_read_struct_in_range(data, msf, args_element.range, 0, &args_list);
          args_list_range = syms_msf_sub_range(args_element.range, sizeof(args_list),
                                               args_element.range.size - sizeof(args_list));
        }
      }
    }
    
    // build params list
    
    SYMS_SymbolIDArray param_array = {0};
    if (args_list.count > 0){
      SYMS_ArenaTemp scratch = syms_get_scratch(&arena, 1);
      SYMS_String8 args_list_data = syms_msf_read_whole_range(scratch.arena, data, msf, args_list_range);
      
      SYMS_CvTypeIndex *ti_array = (SYMS_CvTypeIndex*)args_list_data.str;
      SYMS_U32 data_max_count = args_list_data.size/sizeof(SYMS_CvTypeIndex);
      SYMS_U32 count = SYMS_ClampTop(args_list.count, data_max_count);
      
      SYMS_SymbolID *ids = syms_push_array(arena, SYMS_SymbolID, count);
      SYMS_SymbolID *id_ptr = ids;
      SYMS_CvTypeIndex *ti = ti_array;
      for (SYMS_U32 i = 0; i < count; i += 1, ti += 1, id_ptr += 1){
        *id_ptr = SYMS_ID_u32_u32(SYMS_CvSymbolIDKind_Index, *ti);
      }
      
      param_array.ids = ids;
      param_array.count = count;
      syms_release_scratch(scratch);
    }
    
    result.uid = unit->uid;
    result.param_type_ids = param_array;
    result.return_type_id = SYMS_ID_u32_u32(SYMS_CvSymbolIDKind_Index, ret_itype);
    result.this_type_id = SYMS_ID_u32_u32(SYMS_CvSymbolIDKind_Index, this_itype);
    
  }
  
  return(result);
}

// main api
SYMS_API SYMS_SigInfo
syms_pdb_sig_info_from_id(SYMS_Arena *arena, SYMS_String8 data, SYMS_PdbDbgAccel *dbg,
                          SYMS_CvUnitAccel *unit, SYMS_SymbolID id){
  SYMS_SigInfo result = {0};
  
  if (unit->leaf_set){
    // setup accels
    SYMS_MsfAccel *msf = dbg->msf;
    SYMS_MsfRange range = syms_msf_range_from_sn(msf, unit->sn);
    
    // method stub symbol -> sig info
    if (SYMS_ID_u32_0(id) == SYMS_CvSymbolIDKind_Off){
      SYMS_CvStub *stub = syms_cv_stub_from_unit_off(unit, SYMS_ID_u32_1(id));
      if (stub != 0){
        SYMS_U32 lf_off = stub->off;
        SYMS_CvLeaf lf_kind = 0;
        syms_msf_read_struct_in_range(data, msf, range, lf_off, &lf_kind);
        
        SYMS_U32 data_off = lf_off + sizeof(lf_kind);
        
        switch (lf_kind){
          case SYMS_CvLeaf_METHOD:
          {
            SYMS_CvLeafMethod method = {0};
            syms_msf_read_struct_in_range(data, msf, range, data_off, &method);
            
            SYMS_CvStub *method_list_stub = syms_cv_stub_from_unit_index(unit, method.itype_list);
            
            if (method_list_stub != 0){
              SYMS_U32 rec_off = method_list_stub->off + SYMS_ID_u16_1(id);
              SYMS_CvMethod methodrec = {0};
              syms_msf_read_struct_in_range(data, msf, range, rec_off, &methodrec);
              result = syms_pdb_sig_info_from_sig_index(arena, data, dbg, unit, methodrec.itype);
            }
          }break;
          
          case SYMS_CvLeaf_ONEMETHOD:
          {
            SYMS_CvLeafOneMethod onemethod = {0};
            syms_msf_read_struct_in_range(data, msf, range, data_off, &onemethod);
            result = syms_pdb_sig_info_from_sig_index(arena, data, dbg, unit, onemethod.itype);
          }break;
        }
      }
    }
    
    // type symbol -> sig info
    else{
      result = syms_pdb_sig_info_from_sig_index(arena, data, dbg, unit, SYMS_ID_u32_1(id));
    }
  }
  
  return(result);
}

////////////////////////////////
//~ allen: PDB Line Info

// main api
SYMS_API SYMS_String8
syms_pdb_file_name_from_id(SYMS_Arena *arena, SYMS_String8 data, SYMS_PdbDbgAccel *dbg,
                           SYMS_PdbUnitSetAccel *unit_set, SYMS_UnitID uid, SYMS_FileID id){
  // setup accel
  SYMS_MsfAccel *msf = dbg->msf;
  
  // read string
  SYMS_String8 result = {0};
  switch (SYMS_ID_u32_0(id)){
    case SYMS_CvFileIDKind_IPIOff:
    {
      SYMS_PdbTpiAccel *ipi = &dbg->ipi;
      SYMS_MsfRange ipi_range = syms_msf_range_from_sn(msf, ipi->type_sn);
      result = syms_msf_read_zstring_in_range(arena, data, msf, ipi_range, SYMS_ID_u32_1(id));
    }break;
    
    case SYMS_CvFileIDKind_IPIStringID:
    {
      SYMS_PdbTpiAccel *ipi = &dbg->ipi;
      SYMS_MsfRange ipi_range = syms_msf_range_from_sn(msf, ipi->type_sn);
      SYMS_U32 src_item_off = syms_pdb_tpi_off_from_ti(data, msf, ipi, SYMS_ID_u32_1(id));
      SYMS_CvElement src_element = syms_cv_element(data, msf, ipi_range, src_item_off, 1);
      if (src_element.kind == SYMS_CvLeaf_STRING_ID){
        SYMS_CvLeafStringId string_id = {0};
        if (syms_msf_read_struct_in_range(data, msf, src_element.range, 0, &string_id)){
          result = syms_msf_read_zstring_in_range(arena, data, msf, src_element.range, sizeof(string_id));
        }
      }
    }break;
    
    case SYMS_CvFileIDKind_StrTblOff:
    {
      result = syms_pdb_strtbl_string_from_off(arena, data, dbg, SYMS_ID_u32_1(id));
    }break;
    
    case SYMS_CvFileIDKind_C11Off:
    {
      SYMS_PdbCompUnit *comp_unit = syms_pdb_comp_unit_from_id(unit_set, uid);
      SYMS_MsfRange c11_range = syms_pdb_msf_range_from_comp_unit(comp_unit, SYMS_PdbCompUnitRange_C11);
      SYMS_U32 off = SYMS_ID_u32_1(id);
      syms_msf_read_in_range(data, msf, c11_range, off, 1, &result.size);
      result.str = syms_push_array(arena, SYMS_U8, result.size);
      syms_msf_read_in_range(data, msf, c11_range, off + 1, result.size, result.str);
    }break;
  }
  return(result);
}

SYMS_API SYMS_LineParseOut
syms_pdb_line_parse_from_uid(SYMS_Arena *arena, SYMS_String8 data, SYMS_PdbDbgAccel *dbg,
                             SYMS_PdbUnitSetAccel *unit_set, SYMS_UnitID uid){
  SYMS_LineParseOut result = {0};
  
  //- get comp unit
  SYMS_PdbCompUnit *comp_unit = syms_pdb_comp_unit_from_id(unit_set, uid);
  if (comp_unit != 0){
    SYMS_ArenaTemp scratch = syms_get_scratch(&arena, 1);
    
    // build table
    SYMS_CvLineTableLoose loose = {0};
    
    SYMS_MsfAccel *msf = dbg->msf;
    
    // try C13
    SYMS_MsfRange c13_range = syms_pdb_msf_range_from_comp_unit(comp_unit, SYMS_PdbCompUnitRange_C13);
    if (c13_range.size > 0){
      SYMS_CvC13SubSectionList list = {0};
      syms_cv_c13_sub_sections_from_range(scratch.arena, data, msf, c13_range, &list);
      syms_cv_loose_lines_from_c13(scratch.arena, data, msf, c13_range, list.first,
                                   dbg->section_voffs, dbg->section_count, &loose);
    }
    
    // try C11
    SYMS_MsfRange c11_range = syms_pdb_msf_range_from_comp_unit(comp_unit, SYMS_PdbCompUnitRange_C11);
    if (c11_range.size > 0){
      syms_cv_loose_lines_from_c11(scratch.arena, data, msf, c11_range,
                                   dbg->section_voffs, dbg->section_count, &loose);
    }
    
    // bake line data
    result = syms_cv_line_parse_from_loose(arena, &loose);
    
    syms_release_scratch(scratch);
  }
  
  return(result);
}


////////////////////////////////
//~ allen: PDB Name Maps

SYMS_API SYMS_PdbMapAccel*
syms_pdb_type_map_from_dbg(SYMS_Arena *arena, SYMS_String8 data, SYMS_PdbDbgAccel *dbg){
  SYMS_PdbMapAccel *result = syms_push_array(arena, SYMS_PdbMapAccel, 1);
  result->format = SYMS_FileFormat_PDB;
  result->uid = SYMS_PdbPseudoUnit_TPI;
  return(result);
}

SYMS_API SYMS_PdbMapAccel*
syms_pdb_unmangled_symbol_map_from_dbg(SYMS_Arena *arena, SYMS_String8 data, SYMS_PdbDbgAccel *dbg){
  SYMS_PdbMapAccel *result = syms_push_array(arena, SYMS_PdbMapAccel, 1);
  result->format = SYMS_FileFormat_PDB;
  result->uid = SYMS_PdbPseudoUnit_SYM;
  return(result);
}

SYMS_API SYMS_UnitID
syms_pdb_partner_uid_from_map(SYMS_PdbMapAccel *map){
  return(map->uid);
}

SYMS_API SYMS_USIDList
syms_pdb_usid_list_from_string(SYMS_Arena *arena, SYMS_String8 data, SYMS_PdbDbgAccel *dbg,
                               SYMS_CvUnitAccel *unit, SYMS_PdbMapAccel *map, SYMS_String8 string){
  SYMS_USIDList result = {0};
  if (unit->format == SYMS_FileFormat_PDB){
    switch (map->uid){
      case SYMS_PdbPseudoUnit_SYM:
      {
        result = syms_pdb_symbols_from_name(arena, data, dbg->msf, &dbg->gsi, unit, string);
      }break;
      
      case SYMS_PdbPseudoUnit_TPI:
      {
        result = syms_pdb_types_from_name(arena, data, dbg, unit, string);
      }break;
    }
  }
  return(result);
}


////////////////////////////////
//~ allen: PDB Mangled Names

SYMS_API SYMS_UnitID
syms_pdb_link_names_uid(void){
  return(SYMS_PdbPseudoUnit_SYM);
}

SYMS_API SYMS_PdbLinkMapAccel*
syms_pdb_link_map_from_dbg(SYMS_Arena *arena, SYMS_String8 data, SYMS_PdbDbgAccel *dbg){
  SYMS_PdbLinkMapAccel *result = syms_push_array(arena, SYMS_PdbLinkMapAccel, 1);
  result->format = SYMS_FileFormat_PDB;
  return(result);
}

SYMS_API SYMS_U64
syms_pdb_voff_from_link_name(SYMS_String8 data, SYMS_PdbDbgAccel *dbg, SYMS_PdbLinkMapAccel *map,
                             SYMS_CvUnitAccel *link_unit, SYMS_String8 name){
  SYMS_U64 result = 0;
  
  // get accelerators
  SYMS_MsfAccel *msf = dbg->msf;
  SYMS_PdbGsiAccel *psi = &dbg->psi;
  
  if (psi->bucket_count > 0){
    // get unit range
    SYMS_MsfRange range = syms_msf_range_from_sn(msf, link_unit->sn);
    
    // grab scratch
    SYMS_ArenaTemp scratch = syms_get_scratch(0, 0);
    
    // get bucket
    SYMS_U32 name_hash = syms_pdb_hashV1(name);
    SYMS_U32 bucket_index = name_hash%psi->bucket_count;
    
    // iterate bucket
    for (SYMS_PdbChain *bucket = psi->buckets[bucket_index];
         bucket != 0;
         bucket = bucket->next){
      SYMS_U32 off = bucket->v;
      SYMS_CvStub *stub = syms_cv_stub_from_unit_off(link_unit, off);
      if (stub != 0 && syms_string_match(stub->name, name, 0)){
        SYMS_CvElement element = syms_cv_element(data, msf, range, stub->off, 1);
        if (element.kind == SYMS_CvSymKind_PUB32){
          SYMS_CvPubsym32 pub = {0};
          syms_msf_read_struct_in_range(data, msf, element.range, 0, &pub);
          SYMS_U64 section_voff = syms_pdb_voff_from_section_n(dbg, pub.sec);
          result = section_voff + pub.off;
          break;
        }
      }
    }
    
    // release scratch
    syms_release_scratch(scratch);
  }
  
  return(result);
}

SYMS_API SYMS_LinkNameRecArray
syms_pdb_link_name_array_from_unit(SYMS_Arena *arena, SYMS_String8 data,
                                   SYMS_PdbDbgAccel *dbg, SYMS_CvUnitAccel *unit){
  SYMS_LinkNameRecArray result = {0};
  
  if (unit->pub_count != 0){
    // setup accels
    SYMS_MsfAccel *msf = dbg->msf;
    
    // allocate output memory
    SYMS_U64 count = unit->pub_count;
    SYMS_LinkNameRec *rec = syms_push_array(arena, SYMS_LinkNameRec, count);
    
    // fill output array
    SYMS_LinkNameRec *rec_ptr = rec;
    SYMS_CvStub **stub_opl = unit->pub_stubs + count;
    for (SYMS_CvStub **stub_ptr = unit->pub_stubs; stub_ptr < stub_opl; stub_ptr += 1, rec_ptr += 1){
      SYMS_CvStub *stub = *stub_ptr;
      
      // extract rec from stub
      SYMS_String8 name = stub->name;
      SYMS_U64 voff = 0;
      {
        SYMS_MsfRange range = syms_msf_range_from_sn(msf, unit->sn);
        SYMS_CvElement sig_element = syms_cv_element(data, msf, range, stub->off, 1);
        SYMS_CvPubsym32 pubsym32 = {0};
        if (syms_msf_read_struct_in_range(data, msf, sig_element.range, 0, &pubsym32)){
          SYMS_U64 section_voff = syms_pdb_voff_from_section_n(dbg, pubsym32.sec);
          voff = section_voff + pubsym32.off;
        }
      }
      
      // fill stripped rec
      rec_ptr->name = syms_push_string_copy(arena, name);
      rec_ptr->voff = voff;
    }
    
    // fill result struct
    result.recs = rec;
    result.count = count;
  }
  
  return(result);
}

////////////////////////////////
//~ allen: PDB CV -> Syms Enums and Flags

SYMS_API SYMS_TypeKind
syms_pdb_type_kind_from_cv_pointer_mode(SYMS_CvPointerMode mode){
  SYMS_TypeKind result = SYMS_TypeKind_Null;
  switch (mode){
    default:break;
    case SYMS_CvPointerMode_PTR:    result = SYMS_TypeKind_Ptr;             break;
    case SYMS_CvPointerMode_LVREF:  result = SYMS_TypeKind_LValueReference; break;
    case SYMS_CvPointerMode_RVREF:  result = SYMS_TypeKind_RValueReference; break;
    case SYMS_CvPointerMode_PMEM:
    case SYMS_CvPointerMode_PMFUNC: result = SYMS_TypeKind_MemberPtr;       break;
  }
  return(result);
}

SYMS_API SYMS_TypeModifiers
syms_pdb_modifier_from_cv_pointer_attribs(SYMS_CvPointerAttribs attribs){
  SYMS_TypeModifiers result = 0;
  if (attribs & SYMS_CvPointerAttrib_IS_VOLATILE){
    result |= SYMS_TypeModifier_Volatile;
  }
  if (attribs & SYMS_CvPointerAttrib_IS_CONST){
    result |= SYMS_TypeModifier_Const;
  }
  if (attribs & SYMS_CvPointerAttrib_IS_UNALIGNED){
    result |= SYMS_TypeModifier_Packed;
  }
  if (attribs & SYMS_CvPointerAttrib_IS_RESTRICTED){
    result |= SYMS_TypeModifier_Restrict;
  }
  if (attribs & SYMS_CvPointerAttrib_IS_LREF){
    result |= SYMS_TypeModifier_Reference;
  }
  if (attribs & SYMS_CvPointerAttrib_IS_RREF){
    result |= SYMS_TypeModifier_RValueReference;
  }
  return(result);
}

SYMS_API SYMS_TypeModifiers
syms_pdb_modifier_from_cv_modifier_flags(SYMS_CvModifierFlags flags){
  SYMS_TypeModifiers result = 0;
  if (flags & SYMS_CvModifierFlag_VOLATILE){
    result |= SYMS_TypeModifier_Volatile;
  }
  if (flags & SYMS_CvModifierFlag_CONST){
    result |= SYMS_TypeModifier_Const;
  }
  if (flags & SYMS_CvModifierFlag_UNALIGNED){
    result |= SYMS_TypeModifier_Packed;
  }
  return(result);
}

SYMS_API SYMS_CallConvention
syms_pdb_call_convention_from_cv_call_kind(SYMS_CvCallKind kind){
  SYMS_CallConvention result = SYMS_CallConvention_NULL;
  switch (kind){
    case SYMS_CvCallKind_NEAR_C:      result = SYMS_CallConvention_NEAR_C;      break;
    case SYMS_CvCallKind_FAR_C:       result = SYMS_CallConvention_FAR_C;       break;
    case SYMS_CvCallKind_NEAR_PASCAL: result = SYMS_CallConvention_NEAR_PASCAL; break;
    case SYMS_CvCallKind_FAR_PASCAL:  result = SYMS_CallConvention_FAR_PASCAL;  break;
    case SYMS_CvCallKind_NEAR_FAST:   result = SYMS_CallConvention_NEAR_FAST;   break;
    case SYMS_CvCallKind_FAR_FAST:    result = SYMS_CallConvention_FAR_FAST;    break;
    case SYMS_CvCallKind_NEAR_STD:    result = SYMS_CallConvention_NEAR_STD;    break;
    case SYMS_CvCallKind_FAR_STD:     result = SYMS_CallConvention_FAR_STD;     break;
    case SYMS_CvCallKind_NEAR_SYS:    result = SYMS_CallConvention_NEAR_SYS;    break;
    case SYMS_CvCallKind_FAR_SYS:     result = SYMS_CallConvention_FAR_SYS;     break;
    case SYMS_CvCallKind_THISCALL:    result = SYMS_CallConvention_THISCALL;    break;
    case SYMS_CvCallKind_MIPSCALL:    result = SYMS_CallConvention_MIPSCALL;    break;
    case SYMS_CvCallKind_GENERIC:     result = SYMS_CallConvention_GENERIC;     break;
    case SYMS_CvCallKind_ALPHACALL:   result = SYMS_CallConvention_ALPHACALL;   break;
    case SYMS_CvCallKind_PPCCALL:     result = SYMS_CallConvention_PPCCALL;     break;
    case SYMS_CvCallKind_SHCALL:      result = SYMS_CallConvention_SHCALL;      break;
    case SYMS_CvCallKind_ARMCALL:     result = SYMS_CallConvention_ARMCALL;     break;
    case SYMS_CvCallKind_AM33CALL:    result = SYMS_CallConvention_AM33CALL;    break;
    case SYMS_CvCallKind_TRICALL:     result = SYMS_CallConvention_TRICALL;     break;
    case SYMS_CvCallKind_SH5CALL:     result = SYMS_CallConvention_SH5CALL;     break;
    case SYMS_CvCallKind_M32RCALL:    result = SYMS_CallConvention_M32RCALL;    break;
    case SYMS_CvCallKind_CLRCALL:     result = SYMS_CallConvention_CLRCALL;     break;
    case SYMS_CvCallKind_INLINE:      result = SYMS_CallConvention_INLINE;      break;
    case SYMS_CvCallKind_NEAR_VECTOR: result = SYMS_CallConvention_NEAR_VECTOR; break;
  }
  return(result);
}

#if 0
SYMS_API SYMS_ProcedureFlags
syms_pdb_procedure_flags_from_cv_procedure_flags(SYMS_CvProcFlags pdb_flags){
  SYMS_ProcedureFlags result = 0;
  if (pdb_flags){
    
  }
  return(result);
}

SYMS_API SYMS_VarFlags
syms_var_flags_from_cv_local_flags(SYMS_CvLocalFlags pdb_flags){
  SYMS_VarFlags result = 0;
  // TODO(allen): canonical flag conversion
  if (pdb_flags & SYMS_CvLocalFlag_PARAM){
    result |= SYMS_VarFlag_Parameter;
  }
  if (pdb_flags & SYMS_CvLocalFlag_COMPGEN){
    result |= SYMS_VarFlag_CompilerGen;
  }
  if (pdb_flags & SYMS_CvLocalFlag_ALIASED){
    result |= SYMS_VarFlag_Aliased;
  }
  if (pdb_flags & SYMS_CvLocalFlag_RETVAL){
    result |= SYMS_VarFlag_ReturnValue;
  }
  if (pdb_flags & SYMS_CvLocalFlag_OPTOUT){
    result |= SYMS_VarFlag_OptOut;
  }
  if (pdb_flags & SYMS_CvLocalFlag_STATIC){
    result |= SYMS_VarFlag_Static;
  }
  if (pdb_flags & SYMS_CvLocalFlag_GLOBAL){
    result |= SYMS_VarFlag_Global;
  }
  return(result);
}
#endif

#endif //SYMS_PDB_PARSER_C
