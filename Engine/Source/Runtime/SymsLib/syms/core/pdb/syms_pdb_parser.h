// Copyright Epic Games, Inc. All Rights Reserved.
/* date = May 11th 2021 4:59 pm */

#ifndef SYMS_PDB_PARSER_H
#define SYMS_PDB_PARSER_H

////////////////////////////////
//~ allen: PDB Fundamental

// TODO(allen): version of this with optimized memory locality could be pretty useful.
typedef struct SYMS_PdbChain{
  struct SYMS_PdbChain *next;
  SYMS_U32 v;
} SYMS_PdbChain;

typedef enum SYMS_PdbPseudoUnit{
  SYMS_PdbPseudoUnit_Null,
  SYMS_PdbPseudoUnit_SYM,
  SYMS_PdbPseudoUnit_TPI,
  SYMS_PdbPseudoUnit_COUNT = SYMS_PdbPseudoUnit_TPI,
  SYMS_PdbPseudoUnit_FIRST_COMP_UNIT,
} SYMS_PdbPseudoUnit;

////////////////////////////////
//~ allen: PDB DBI Information

typedef struct SYMS_PdbDbiAccel{
  SYMS_B16 valid;
  SYMS_CoffMachineType machine_type;
  SYMS_MsfStreamNumber gsi_sn;
  SYMS_MsfStreamNumber psi_sn;
  SYMS_MsfStreamNumber sym_sn;
  
  SYMS_U64 range_off[(SYMS_U64)(SYMS_PdbDbiRange_COUNT) + 1];
  SYMS_MsfStreamNumber dbg_sn[SYMS_PdbDbiStream_COUNT];
} SYMS_PdbDbiAccel;

typedef struct SYMS_PdbInfoSlot{
  struct SYMS_PdbInfoSlot *next;
  SYMS_String8 string;
  SYMS_MsfStreamNumber sn;
} SYMS_PdbInfoSlot;

typedef struct SYMS_PdbInfoTable{
  SYMS_PdbInfoSlot *first;
  SYMS_PdbInfoSlot *last;
  SYMS_PeGuid auth_guid;
} SYMS_PdbInfoTable;

typedef struct SYMS_PdbNamedStreamArray{
  SYMS_MsfStreamNumber sn[SYMS_PdbNamedStream_COUNT];
} SYMS_PdbNamedStreamArray;

typedef struct SYMS_PdbStrtblAccel{
  SYMS_U32 bucket_count;
  SYMS_MsfStreamNumber sn;
  SYMS_U32Range strblock;
  SYMS_U32Range buckets;
} SYMS_PdbStrtblAccel;

////////////////////////////////
//~ allen: PDB Units

typedef struct SYMS_PdbCompUnit{
  SYMS_MsfStreamNumber sn;
  SYMS_U64 range_off[(SYMS_U64)(SYMS_PdbCompUnitRange_COUNT) + 1];
  
  SYMS_PdbStringIndex src_file;
  SYMS_PdbStringIndex pdb_file;
  
  SYMS_String8 obj_name;
  SYMS_String8 group_name;
} SYMS_PdbCompUnit;

typedef struct SYMS_PdbCompUnitNode{
  struct SYMS_PdbCompUnitNode *next;
  SYMS_PdbCompUnit comp_unit;
} SYMS_PdbCompUnitNode;

typedef struct SYMS_PdbUnitSetAccel{
  SYMS_FileFormat format;
  SYMS_PdbCompUnit **comp_units;
  SYMS_U64 comp_count;
} SYMS_PdbUnitSetAccel;

////////////////////////////////
//~ allen: PDB TPI

typedef struct SYMS_PdbTpiOffRange{
  SYMS_U32 first_off;
  SYMS_CvTypeIndex first_ti;
  SYMS_CvTypeIndex opl_ti;
} SYMS_PdbTpiOffRange;

typedef struct SYMS_PdbTpiAccel{
  SYMS_MsfStreamNumber type_sn;
  // TODO(allen): Stop playing this TPI hints game.
  SYMS_PdbTpiOffHint *hints;
  SYMS_PdbChain **buckets;
  SYMS_U32 bucket_count;
  SYMS_U64 count;
  SYMS_U32 base_off;
  SYMS_CvTypeIndex first_ti;
  SYMS_CvTypeIndex opl_ti;
  SYMS_U32 *off;
} SYMS_PdbTpiAccel;

////////////////////////////////
//~ allen: PDB GSI

typedef struct SYMS_PdbGsiAccel{
  SYMS_PdbChain **buckets;
  SYMS_U32 bucket_count;
} SYMS_PdbGsiAccel;

////////////////////////////////
//~ allen: PDB Accel Types

typedef struct SYMS_PdbFileAccel{
  SYMS_FileFormat format;
  SYMS_MsfAccel *msf;
} SYMS_PdbFileAccel;

typedef struct SYMS_PdbDbgAccel{
  SYMS_FileFormat format;
  
  // MSF view into the data
  SYMS_MsfAccel *msf;
  
  // specific to true PDB file
  SYMS_PdbDbiAccel dbi;
  SYMS_PdbNamedStreamArray named;
  SYMS_PdbStrtblAccel strtbl;
  SYMS_PdbTpiAccel tpi;
  SYMS_PdbTpiAccel ipi;
  SYMS_PdbGsiAccel gsi;
  SYMS_PdbGsiAccel psi;
  SYMS_PeGuid auth_guid;
  
  // TODO(allen): simplification pass
  
  // arch
  SYMS_Arch arch;
  
  // type uid
  SYMS_UnitID type_uid;
  
  // section data
  SYMS_MsfRange section_data_range;
  SYMS_U64 *section_voffs;
  SYMS_U64 section_count;
} SYMS_PdbDbgAccel;

////////////////////////////////
//~ allen: PDB Map Accel Type

typedef struct SYMS_PdbMapAccel{
  SYMS_FileFormat format;
  SYMS_UnitID uid;
} SYMS_PdbMapAccel;

typedef struct SYMS_PdbLinkMapAccel{
  SYMS_FileFormat format;
} SYMS_PdbLinkMapAccel;

////////////////////////////////
//~ allen: PDB Constants

#define SYMS_CV_SYMBOL_ALIGN_IN_PDB  4

////////////////////////////////
//~ allen: PDB TPI Functions

//- tpi accel helpers
SYMS_API SYMS_PdbTpiOffRange syms_pdb_tpi__hint_from_index(SYMS_PdbTpiAccel *tpi, SYMS_CvTypeIndex ti);
SYMS_API void                syms_pdb_tpi__fill_off_range(SYMS_String8 data, SYMS_MsfAccel *msf,
                                                          SYMS_PdbTpiAccel *tpi, SYMS_PdbTpiOffRange *fill);

//- tpi accel
SYMS_API SYMS_PdbTpiAccel syms_pdb_tpi_accel_from_sn(SYMS_Arena *arena, SYMS_String8 data, SYMS_MsfAccel *msf,
                                                     SYMS_MsfStreamNumber sn);
SYMS_API SYMS_U32         syms_pdb_tpi_off_from_ti(SYMS_String8 data, SYMS_MsfAccel *msf, SYMS_PdbTpiAccel *tpi,
                                                   SYMS_CvTypeIndex ti);
SYMS_API SYMS_U32         syms_pdb_tpi_base_off(SYMS_PdbTpiAccel *tpi);
SYMS_API SYMS_MsfRange    syms_pdb_tpi_range(SYMS_MsfAccel *msf, SYMS_PdbTpiAccel *tpi);

//- tpi name lookups
SYMS_API SYMS_USIDList syms_pdb_types_from_name(SYMS_Arena *arena, SYMS_String8 data,
                                                SYMS_PdbDbgAccel *dbg, SYMS_CvUnitAccel *unit,
                                                SYMS_String8 name);


////////////////////////////////
//~ allen: PDB GSI Functions

SYMS_API SYMS_PdbGsiAccel  syms_pdb_gsi_accel_from_range(SYMS_Arena *arena, SYMS_String8 data, SYMS_MsfAccel *msf,
                                                         SYMS_MsfRange range);
SYMS_API SYMS_MsfRange     syms_pdb_gsi_part_from_psi_range(SYMS_MsfRange psi_range);

SYMS_API SYMS_USIDList     syms_pdb_symbols_from_name(SYMS_Arena *arena, SYMS_String8 data, SYMS_MsfAccel *msf,
                                                      SYMS_PdbGsiAccel *gsi, SYMS_CvUnitAccel *unit,
                                                      SYMS_String8 name);


////////////////////////////////
//~ allen: PDB Accelerator Setup

// pdb specific parsing
SYMS_API SYMS_PdbDbiAccel syms_pdb_dbi_from_msf(SYMS_String8 data, SYMS_MsfAccel *msf,
                                                SYMS_MsfStreamNumber sn);
SYMS_API SYMS_MsfRange    syms_pdb_dbi_sub_range(SYMS_PdbDbiAccel *dbi, SYMS_MsfStreamNumber sn,
                                                 SYMS_PdbDbiRange n);
SYMS_API SYMS_MsfRange    syms_pdb_dbi_stream(SYMS_MsfAccel *msf, SYMS_PdbDbiAccel *dbi, SYMS_PdbDbiStream n);

SYMS_API SYMS_PdbInfoTable        syms_pdb_parse_info(SYMS_Arena *arena, SYMS_String8 data, SYMS_MsfAccel *msf);
SYMS_API SYMS_PdbNamedStreamArray syms_pdb_named_stream_array(SYMS_PdbInfoTable *table);
SYMS_API SYMS_PdbStrtblAccel      syms_pdb_dbi_parse_strtbl(SYMS_Arena *arena, SYMS_String8 data,
                                                            SYMS_MsfAccel *msf, SYMS_MsfStreamNumber sn);

// pdb specific api
SYMS_API SYMS_String8 syms_pdb_strtbl_string_from_off(SYMS_Arena *arena, SYMS_String8 data,
                                                      SYMS_PdbDbgAccel *dbg, SYMS_U32 off);
SYMS_API SYMS_String8 syms_pdb_strtbl_string_from_index(SYMS_Arena *arena, SYMS_String8 data,
                                                        SYMS_PdbDbgAccel *dbg, SYMS_PdbStringIndex n);

// main api
SYMS_API SYMS_PdbFileAccel* syms_pdb_file_accel_from_data(SYMS_Arena *arena, SYMS_String8 data);
SYMS_API SYMS_PdbDbgAccel*  syms_pdb_dbg_accel_from_file(SYMS_Arena *arena, SYMS_String8 data,
                                                         SYMS_PdbFileAccel *file);


////////////////////////////////
//~ allen: PDB Architecture

// main api
SYMS_API SYMS_Arch syms_pdb_arch_from_dbg(SYMS_PdbDbgAccel *dbg);


////////////////////////////////
//~ allen: PDB Match Keys

SYMS_API SYMS_ExtMatchKey syms_pdb_ext_match_key_from_dbg(SYMS_String8 data, SYMS_PdbDbgAccel *dbg);


////////////////////////////////
//~ allen: PDB Sections

// pdb specific
SYMS_API SYMS_CoffSectionHeader syms_pdb_coff_section_header(SYMS_String8 data, SYMS_PdbDbgAccel *dbg,
                                                             SYMS_U64 n);
SYMS_API SYMS_U64 syms_pdb_voff_from_section_n(SYMS_PdbDbgAccel *dbg, SYMS_U64 n);

// main api
SYMS_API SYMS_SecInfoArray syms_pdb_sec_info_array_from_dbg(SYMS_Arena *arena, SYMS_String8 data,
                                                            SYMS_PdbDbgAccel *dbg);

////////////////////////////////
//~ allen: PDB Compilation Units

// pdb specific
SYMS_API SYMS_PdbCompUnit* syms_pdb_comp_unit_from_id(SYMS_PdbUnitSetAccel *unit_set, SYMS_UnitID id);
SYMS_API SYMS_MsfRange     syms_pdb_msf_range_from_comp_unit(SYMS_PdbCompUnit *unit,
                                                             SYMS_PdbCompUnitRange n);

// main api
SYMS_API SYMS_PdbUnitSetAccel*syms_pdb_unit_set_accel_from_dbg(SYMS_Arena *arena, SYMS_String8 data,
                                                               SYMS_PdbDbgAccel *dbg);
SYMS_API SYMS_U64             syms_pdb_unit_count_from_set(SYMS_PdbUnitSetAccel *unit_set);
SYMS_API SYMS_UnitInfo        syms_pdb_unit_info_from_uid(SYMS_PdbUnitSetAccel *unit_set,
                                                          SYMS_UnitID uid);
SYMS_API SYMS_UnitNames       syms_pdb_unit_names_from_uid(SYMS_Arena *arena,
                                                           SYMS_PdbUnitSetAccel *unit_set,
                                                           SYMS_UnitID uid);

SYMS_API SYMS_UnitRangeArray  syms_pdb_unit_ranges_from_set(SYMS_Arena *arena,
                                                            SYMS_String8 data, SYMS_PdbDbgAccel *dbg,
                                                            SYMS_PdbUnitSetAccel *unit_set);

SYMS_API SYMS_UnitFeatures    syms_pdb_unit_features_from_number(SYMS_PdbUnitSetAccel *unit_set,
                                                                 SYMS_U64 n);

////////////////////////////////
//~ allen: PDB Symbol Parsing

// cv parse
SYMS_API SYMS_CvUnitAccel*  syms_pdb_pub_sym_accel_from_dbg(SYMS_Arena *arena, SYMS_String8 data,
                                                            SYMS_PdbDbgAccel *dbg);

SYMS_API SYMS_CvUnitAccel*  syms_pdb_leaf_accel_from_dbg(SYMS_Arena *arena, SYMS_String8 data,
                                                         SYMS_PdbDbgAccel *dbg, SYMS_UnitID uid);

SYMS_API SYMS_TypeInfo      syms_pdb_leaf_type_info_from_id(SYMS_String8 data, SYMS_PdbDbgAccel *dbg,
                                                            SYMS_CvUnitAccel *unit, SYMS_SymbolID id);
SYMS_API SYMS_ConstInfo     syms_pdb_leaf_const_info_from_id(SYMS_String8 data, SYMS_PdbDbgAccel *dbg,
                                                             SYMS_CvUnitAccel *unit, SYMS_SymbolID id);

// main api
SYMS_API SYMS_CvUnitAccel*syms_pdb_unit_accel_from_uid(SYMS_Arena *arena, SYMS_String8 data,
                                                       SYMS_PdbDbgAccel *dbg, SYMS_PdbUnitSetAccel *set,
                                                       SYMS_UnitID uid);

SYMS_API SYMS_UnitID        syms_pdb_tls_var_uid_from_dbg(SYMS_PdbDbgAccel *dbg);
SYMS_API SYMS_SymbolIDArray syms_pdb_tls_var_sid_array_from_unit(SYMS_Arena *arena,
                                                                 SYMS_CvUnitAccel *unit);
SYMS_API SYMS_U64          syms_pdb_symbol_count_from_unit(SYMS_CvUnitAccel *unit);

SYMS_API SYMS_SymbolKind   syms_pdb_symbol_kind_from_sid(SYMS_String8 data, SYMS_PdbDbgAccel *dbg,
                                                         SYMS_CvUnitAccel *unit, SYMS_SymbolID id);

SYMS_API SYMS_TypeInfo     syms_pdb_type_info_from_sid(SYMS_String8 data, SYMS_PdbDbgAccel *dbg,
                                                       SYMS_CvUnitAccel *unit, SYMS_SymbolID sid);
SYMS_API SYMS_ConstInfo    syms_pdb_const_info_from_id(SYMS_String8 data, SYMS_PdbDbgAccel *dbg,
                                                       SYMS_CvUnitAccel *unit, SYMS_SymbolID id);

////////////////////////////////
//~ allen: PDB Variable Info

// cv parse
SYMS_API SYMS_USID   syms_pdb_sym_type_from_var_id(SYMS_String8 data, SYMS_PdbDbgAccel *dbg,
                                                   SYMS_CvUnitAccel *unit, SYMS_SymbolID id);

SYMS_API SYMS_U64    syms_pdb_sym_voff_from_var_sid(SYMS_String8 data, SYMS_PdbDbgAccel *dbg,
                                                    SYMS_CvUnitAccel *unit, SYMS_SymbolID sid);

SYMS_API SYMS_RegSection syms_pdb_reg_section_from_x86_reg(SYMS_CvReg cv_reg);
SYMS_API SYMS_RegSection syms_pdb_reg_section_from_x64_reg(SYMS_CvReg cv_reg);
SYMS_API SYMS_RegSection syms_pdb_reg_section_from_arch_reg(SYMS_Arch arch, SYMS_CvReg cv_reg);

SYMS_API SYMS_RegSection syms_pdb_reg_section_from_framepointer(SYMS_String8 data,  SYMS_PdbDbgAccel *dbg,
                                                                SYMS_MsfRange range,
                                                                SYMS_CvStub *framepointer_stub);

// main api
SYMS_API SYMS_USID   syms_pdb_type_from_var_id(SYMS_String8 data, SYMS_PdbDbgAccel *dbg,
                                               SYMS_CvUnitAccel *unit, SYMS_SymbolID id);

SYMS_API SYMS_U64    syms_pdb_voff_from_var_sid(SYMS_String8 data, SYMS_PdbDbgAccel *dbg,
                                                SYMS_CvUnitAccel *unit, SYMS_SymbolID sid);

SYMS_API SYMS_Location syms_pdb_location_from_var_sid(SYMS_Arena *arena, SYMS_String8 data,
                                                      SYMS_PdbDbgAccel *dbg, SYMS_CvUnitAccel *unit,
                                                      SYMS_SymbolID sid);
SYMS_API SYMS_LocRangeArray syms_pdb_location_ranges_from_var_sid(SYMS_Arena *arena, SYMS_String8 data,
                                                                  SYMS_PdbDbgAccel *dbg,
                                                                  SYMS_CvUnitAccel *unit,
                                                                  SYMS_SymbolID sid);
SYMS_API SYMS_Location syms_pdb_location_from_id(SYMS_Arena *arena, SYMS_String8 data,
                                                 SYMS_PdbDbgAccel *dbg, SYMS_CvUnitAccel *unit,
                                                 SYMS_LocID loc_id);


////////////////////////////////
//~ allen: PDB Member Info

// cv parse
SYMS_API void syms_pdb__field_list_parse(SYMS_Arena *arena, SYMS_String8 data, SYMS_MsfAccel *msf,
                                         SYMS_CvUnitAccel *unit, SYMS_U32 index,
                                         SYMS_PdbMemStubList *out);

// main api
SYMS_API SYMS_CvMemsAccel*syms_pdb_mems_accel_from_sid(SYMS_Arena *arena, SYMS_String8 data,
                                                       SYMS_PdbDbgAccel *dbg, SYMS_CvUnitAccel *unit,
                                                       SYMS_SymbolID id);

SYMS_API SYMS_U64      syms_pdb_mem_count_from_mems(SYMS_CvMemsAccel *mems);
SYMS_API SYMS_MemInfo  syms_pdb_mem_info_from_number(SYMS_Arena *arena, SYMS_String8 data,
                                                     SYMS_PdbDbgAccel *dbg, SYMS_CvUnitAccel *unit,
                                                     SYMS_CvMemsAccel *mems, SYMS_U64 n);

SYMS_API SYMS_USID     syms_pdb_type_from_mem_number(SYMS_String8 data, SYMS_PdbDbgAccel *dbg,
                                                     SYMS_CvUnitAccel *unit, SYMS_CvMemsAccel *mems,
                                                     SYMS_U64 n);

SYMS_API SYMS_SigInfo  syms_pdb_sig_info_from_mem_number(SYMS_Arena *arena, SYMS_String8 data,
                                                         SYMS_PdbDbgAccel *dbg, SYMS_CvUnitAccel *unit,
                                                         SYMS_CvMemsAccel*mems, SYMS_U64 n);

SYMS_API SYMS_EnumMemberArray syms_pdb_enum_member_array_from_sid(SYMS_Arena *arena, SYMS_String8 data,
                                                                  SYMS_PdbDbgAccel *dbg,
                                                                  SYMS_CvUnitAccel *unit, SYMS_SymbolID sid);

////////////////////////////////
//~ allen: PDB Procedure Info

// main api

SYMS_API SYMS_UnitIDAndSig syms_pdb_proc_sig_handle_from_id(SYMS_String8 data, SYMS_PdbDbgAccel *dbg,
                                                            SYMS_CvUnitAccel *unit, SYMS_SymbolID id);
SYMS_API SYMS_SigInfo      syms_pdb_sig_info_from_handle(SYMS_Arena *arena, SYMS_String8 data,
                                                         SYMS_PdbDbgAccel *dbg, SYMS_CvUnitAccel *unit,
                                                         SYMS_SigHandle handle);

SYMS_API SYMS_U64RangeArray syms_pdb_scope_vranges_from_sid(SYMS_Arena *arena, SYMS_String8 data,
                                                            SYMS_PdbDbgAccel *dbg, SYMS_CvUnitAccel *unit,
                                                            SYMS_SymbolID sid);
SYMS_API SYMS_SymbolIDArray syms_pdb_scope_children_from_sid(SYMS_Arena *arena, SYMS_String8 data,
                                                             SYMS_PdbDbgAccel *dbg, SYMS_CvUnitAccel *unit,
                                                             SYMS_SymbolID id);

////////////////////////////////
//~ allen: PDB Signature Info

// pdb specific helper
SYMS_API SYMS_SigInfo syms_pdb_sig_info_from_sig_index(SYMS_Arena *arena, SYMS_String8 data, SYMS_PdbDbgAccel *dbg,
                                                       SYMS_CvUnitAccel *unit, SYMS_CvTypeIndex index);

// main api
SYMS_API SYMS_SigInfo syms_pdb_sig_info_from_id(SYMS_Arena *arena, SYMS_String8 data,
                                                SYMS_PdbDbgAccel *dbg, SYMS_CvUnitAccel *unit,
                                                SYMS_SymbolID id);

////////////////////////////////
//~ allen: PDB Line Info

// main api
SYMS_API SYMS_String8 syms_pdb_file_name_from_id(SYMS_Arena *arena, SYMS_String8 data, SYMS_PdbDbgAccel *dbg,
                                                 SYMS_PdbUnitSetAccel *unit_set, SYMS_UnitID uid,
                                                 SYMS_FileID id);

SYMS_API SYMS_LineParseOut syms_pdb_line_parse_from_uid(SYMS_Arena *arena, SYMS_String8 data,
                                                        SYMS_PdbDbgAccel *dbg, SYMS_PdbUnitSetAccel *set,
                                                        SYMS_UnitID uid);

////////////////////////////////
//~ allen: PDB Name Maps

// main api
SYMS_API SYMS_PdbMapAccel* syms_pdb_type_map_from_dbg(SYMS_Arena *arena, SYMS_String8 data, SYMS_PdbDbgAccel *dbg);
SYMS_API SYMS_PdbMapAccel* syms_pdb_unmangled_symbol_map_from_dbg(SYMS_Arena *arena, SYMS_String8 data,
                                                                  SYMS_PdbDbgAccel *dbg);
SYMS_API SYMS_UnitID       syms_pdb_partner_uid_from_map(SYMS_PdbMapAccel *map);
SYMS_API SYMS_USIDList     syms_pdb_usid_list_from_string(SYMS_Arena *arena, SYMS_String8 data,
                                                          SYMS_PdbDbgAccel *dbg, SYMS_CvUnitAccel *unit,
                                                          SYMS_PdbMapAccel *map, SYMS_String8 string);

////////////////////////////////
//~ allen: PDB Mangled Names

// main api
SYMS_API SYMS_UnitID syms_pdb_link_names_uid(void);

SYMS_API SYMS_PdbLinkMapAccel* syms_pdb_link_map_from_dbg(SYMS_Arena *arena, SYMS_String8 data,
                                                          SYMS_PdbDbgAccel *dbg);
SYMS_API SYMS_U64              syms_pdb_voff_from_link_name(SYMS_String8 data, SYMS_PdbDbgAccel *dbg,
                                                            SYMS_PdbLinkMapAccel *map,
                                                            SYMS_CvUnitAccel *link_unit, SYMS_String8 name);
SYMS_API SYMS_LinkNameRecArray syms_pdb_link_name_array_from_unit(SYMS_Arena *arena, SYMS_String8 data,
                                                                  SYMS_PdbDbgAccel *dbg, SYMS_CvUnitAccel *unit);

////////////////////////////////
//~ allen: PDB CV -> Syms Enums and Flags

SYMS_API SYMS_TypeKind       syms_pdb_type_kind_from_cv_pointer_mode(SYMS_CvPointerMode mode);
SYMS_API SYMS_TypeModifiers  syms_pdb_modifier_from_cv_pointer_attribs(SYMS_CvPointerAttribs attribs);
SYMS_API SYMS_TypeModifiers  syms_pdb_modifier_from_cv_modifier_flags(SYMS_CvModifierFlags flags);
SYMS_API SYMS_CallConvention syms_pdb_call_convention_from_cv_call_kind(SYMS_CvCallKind kind);
//SYMS_API SYMS_ProcedureFlags syms_pdb_procedure_flags_from_cv_procedure_flags(SYMS_CvProcFlags pdb_flags);
//SYMS_API SYMS_VarFlags       syms_var_flags_from_cv_local_flags(SYMS_CvLocalFlags pdb_flags);

#endif //SYMS_PDB_PARSER_H
