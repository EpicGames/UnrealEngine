// Copyright Epic Games, Inc. All Rights Reserved.

#ifndef SYMS_COFF_C
#define SYMS_COFF_C

////////////////////////////////
//~ nick: Generated code

#include "syms/core/generated/syms_meta_coff.c"

////////////////////////////////
//~ nick: COFF Format Functions

SYMS_API void
syms_coff_symbol32_from_coff_symbol16(SYMS_CoffSymbol32 *sym32, SYMS_CoffSymbol16 *sym16){
  sym32->name             = sym16->name;
  sym32->value            = sym16->value;
  if (sym16->section_number == SYMS_COFF_SYMBOL_DEBUG_SECTION_16) {
    sym32->section_number = SYMS_COFF_SYMBOL_DEBUG_SECTION;
  } else if (sym16->section_number == SYMS_COFF_SYMBOL_ABS_SECTION_16) {
    sym32->section_number = SYMS_COFF_SYMBOL_ABS_SECTION;
  } else {
    sym32->section_number = (SYMS_U32)sym16->section_number;
  }
  sym32->type.v           = sym16->type.v;
  sym32->storage_class    = sym16->storage_class;
  sym32->aux_symbol_count = sym16->aux_symbol_count;
}

#endif // SYMS_COFF_C
