// Copyright Epic Games, Inc. All Rights Reserved.

#ifndef SYMS_COFF_H
#define SYMS_COFF_H

////////////////////////////////
//~ nick: Generated Code

#include "syms/core/generated/syms_meta_coff.h"

////////////////////////////////
//~ nick: COFF Format Types

#define SYMS_COFF_MIN_BIG_OBJ_VERSION 2

SYMS_GLOBAL SYMS_U8 syms_coff_big_obj_magic[] = {
  0xC7,0xA1,0xBA,0xD1,0xEE,0xBA,0xA9,0x4B,
  0xAF,0x20,0xFA,0xF6,0x6A,0xA4,0xDC,0xB8,
};

typedef struct SYMS_CoffHeaderBigObj{
  SYMS_U16 sig1; // SYMS_CoffMachineType_UNKNOWN
  SYMS_U16 sig2; // SYMS_U16_MAX
  SYMS_U16 version;
  SYMS_U16 machine;
  SYMS_U32 time_stamp;
  SYMS_U8 magic[16];
  SYMS_U32 unused[4];
  SYMS_U32 section_count;
  SYMS_U32 pointer_to_symbol_table;
  SYMS_U32 number_of_symbols;
} SYMS_CoffHeaderBigObj;

#pragma pack(push, 1)

// Special values for section number field in coff symbol
#define SYMS_COFF_SYMBOL_UNDEFINED_SECTION 0
#define SYMS_COFF_SYMBOL_ABS_SECTION      ((SYMS_U32)-1)
#define SYMS_COFF_SYMBOL_DEBUG_SECTION    ((SYMS_U32)-2)
#define SYMS_COFF_SYMBOL_ABS_SECTION_16   ((SYMS_U16)-1)
#define SYMS_COFF_SYMBOL_DEBUG_SECTION_16 ((SYMS_U16)-2)

typedef union SYMS_CoffSymbolName{
  SYMS_U8 short_name[8];
  struct{
    // if this field is filled with zeroes we have a long name,
    // which means name is stored in the string table 
    // and we need to use the offset to look it up...
    SYMS_U32 zeroes;
    SYMS_U32 string_table_offset;
  } long_name;
} SYMS_CoffSymbolName;

typedef struct SYMS_CoffSymbol16{
  SYMS_CoffSymbolName name;
  SYMS_U32 value;
  SYMS_U16 section_number;
  union{
    struct{
      SYMS_CoffSymDType msb;
      SYMS_CoffSymType  lsb;
    } u;
    SYMS_U16 v;
  } type;
  SYMS_CoffSymStorageClass storage_class;
  SYMS_U8 aux_symbol_count;
} SYMS_CoffSymbol16;

typedef struct SYMS_CoffSymbol32{
  SYMS_CoffSymbolName name;
  SYMS_U32 value;
  SYMS_U32 section_number;
  union{
    struct{
      SYMS_CoffSymDType msb;
      SYMS_CoffSymType  lsb;
    } u;
    SYMS_U16 v;
  } type;
  SYMS_CoffSymStorageClass storage_class;
  SYMS_U8 aux_symbol_count;
} SYMS_CoffSymbol32;

// specifies how section data should be modified when placed in the image file.
typedef struct SYMS_CoffReloc{
  SYMS_U32 apply_off; // section relative offset where relocation is placed
  SYMS_U32 isymbol;   // zero based index into coff symbol table
  SYMS_U16 type;      // relocation type that depends on the arch
} SYMS_CoffReloc;

#pragma pack(pop)

////////////////////////////////
//~ nick: COFF Format Functions

SYMS_API void syms_coff_symbol32_from_coff_symbol16(SYMS_CoffSymbol32 *sym32, SYMS_CoffSymbol16 *sym16);

#endif // SYMS_COFF_H
