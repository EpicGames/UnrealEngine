// Copyright Epic Games, Inc. All Rights Reserved.
/* date = January 13th 2021 4:55 pm */

#ifndef SYMS_PE_H
#define SYMS_PE_H

/* 
** NOTE(allen): Overview of PE format
** 
** At the offset 0: DOS Magic Number, DOS Header
**  DOS Header contains a "pointer" to the COFF Header.
**
** Starting at the COFF Header the following are packed in order:
** 1. PE Magic Number
** 2. COFF Header (fixed size)
** 3. COFF Optional Header (size determined by 1)
** 4. Section Table (size determined by 1)
**
*/

////////////////////////////////
//~ allen: Generated

#include "syms/core/generated/syms_meta_pe.h"

////////////////////////////////
//~ allen: DOS Header

// this is the "MZ" as a 16-bit short
#define SYMS_DOS_MAGIC 0x5a4d

////////////////////////////////
//~ allen: PE Magic Numbers

#define SYMS_PE_MAGIC   0x00004550u
#define SYMS_PE32_MAGIC     0x010bu
#define SYMS_PE32PLUS_MAGIC 0x020bu

////////////////////////////////
//~ allen: .pdata

// TODO(allen): renaming pass here.
typedef struct SYMS_PeMipsPdata{
  SYMS_U32 voff_first;
  SYMS_U32 voff_one_past_last;
  SYMS_U32 voff_exception_handler;
  SYMS_U32 voff_exception_handler_data;
  SYMS_U32 voff_one_past_prolog;
} SYMS_PeMipsPdata;

typedef struct SYMS_PeArmPdata{
  SYMS_U32 voff_first;
  // NOTE(allen):
  // bits    | meaning
  // [0:7]   | prolog length
  // [8:29]  | function length
  // [30:30] | instructions_are_32bits (otherwise they are 16 bits)
  // [31:31] | has_exception_handler
  SYMS_U32 combined;
} SYMS_PeArmPdata;

typedef struct SYMS_PeIntelPdata{
  SYMS_U32 voff_first;
  SYMS_U32 voff_one_past_last;
  SYMS_U32 voff_unwind_info;
} SYMS_PeIntelPdata;

////////////////////////////////
//~ allen: Codeview Info

typedef struct SYMS_PeGuid{
  SYMS_U32 data1;
  SYMS_U16 data2;
  SYMS_U16 data3;
  SYMS_U8  data4[8];
} SYMS_PeGuid;
SYMS_STATIC_ASSERT(sizeof(SYMS_PeGuid) == 16);

#define SYMS_CODEVIEW_PDB20_MAGIC 0x3031424e
#define SYMS_CODEVIEW_PDB70_MAGIC 0x53445352

typedef struct SYMS_PeCvHeaderPDB20{
  SYMS_U32 magic;
  SYMS_U32 offset;
  SYMS_U32 time;
  SYMS_U32 age;
  // file name packed after struct
} SYMS_PeCvHeaderPDB20;

#pragma pack(push)
typedef struct SYMS_PeCvHeaderPDB70{
  SYMS_U32 magic;
  SYMS_PeGuid guid;
  SYMS_U32 age;
  // file name packed after struct
} SYMS_PeCvHeaderPDB70;
#pragma pack(pop)

////////////////////////////////
//~ nick: Imports & Exports

typedef struct SYMS_PeImportEntry{
  SYMS_U32 lookup_table_voff;
  SYMS_U32 timestamp;
  SYMS_U32 forwarder_chain;
  SYMS_U32 name_voff;
  SYMS_U32 import_addr_table_voff;
} SYMS_PeImportEntry;

typedef struct SYMS_PeDelayedImportEntry{
  // According to PE/COFF spec this field is unused and should be set zero,
  // but when I compile mule with MSVC 2019 this is set to 1.
  SYMS_U32 attributes;
  SYMS_U32 name_voff;                       // Name of the DLL
  SYMS_U32 module_handle_voff;              // Place where module handle from LoadLibrary is stored
  SYMS_U32 iat_voff;
  SYMS_U32 name_table_voff;                 // Array of hint/name or oridnals
  SYMS_U32 bound_table_voff;                // (Optional) Points to an array of SYMS_PeThunkData
  SYMS_U32 unload_table_voff;               // (Optional) Copy of iat_voff
  //  0 not bound
  // -1 if bound and real timestamp located in bounded import directory
  // Otherwise time when dll was bound
  SYMS_U32 timestamp; 
} SYMS_PeDelayedImportEntry;

typedef struct SYMS_PeExportTable{
  SYMS_U32 flags;                       // must be zero
  SYMS_U32 timestamp;                   // time and date when export table was created
  SYMS_U16 major_ver;                   // table version, user can change major and minor version
  SYMS_U16 minor_ver; 
  SYMS_U32 name_voff;                   // ASCII name of the dll
  SYMS_U32 ordinal_base;                // Starting oridnal number
  SYMS_U32 export_address_table_count;
  SYMS_U32 name_pointer_table_count;
  SYMS_U32 export_address_table_voff;
  SYMS_U32 name_pointer_table_voff;
  SYMS_U32 ordinal_table_voff;
} SYMS_PeExportTable;

#endif //SYMS_PE_H
