// Copyright Epic Games, Inc. All Rights Reserved.

#ifndef SYMS_PE_PARSER_C
#define SYMS_PE_PARSER_C

////////////////////////////////
//~ allen: Functions

SYMS_API SYMS_PeFileAccel*
syms_pe_file_accel_from_data(SYMS_Arena *arena, SYMS_String8 data){
  // based range setup
  void *base = data.str;
  SYMS_U64Range range = {0, data.size};
  
  // read dos header
  SYMS_DosHeader dos_header = {0};
  syms_based_range_read_struct(base, range, 0, &dos_header);
  
  // fill result
  SYMS_PeFileAccel *result = (SYMS_PeFileAccel*)&syms_format_nil;
  if (dos_header.magic == SYMS_DOS_MAGIC){
    result = syms_push_array(arena, SYMS_PeFileAccel, 1);
    result->format = SYMS_FileFormat_PE;
    result->coff_off = dos_header.coff_file_offset;
  }
  
  return(result);
}

SYMS_API SYMS_PeBinAccel*
syms_pe_bin_accel_from_file(SYMS_Arena *arena, SYMS_String8 data, SYMS_PeFileAccel *accel){
  // based range setup
  void *base = data.str;
  SYMS_U64Range range = {0, data.size};
  
  // read pe magic
  SYMS_U32 coff_off = accel->coff_off;
  SYMS_U32 pe_magic = 0;
  syms_based_range_read_struct(base, range, coff_off, &pe_magic);
  
  // check pe magic
  SYMS_PeBinAccel *result = (SYMS_PeBinAccel*)&syms_format_nil;
  if (pe_magic == SYMS_PE_MAGIC){
    
    // read coff header
    SYMS_U32 coff_header_off = coff_off + sizeof(pe_magic);
    SYMS_CoffHeader coff_header = {0};
    syms_based_range_read_struct(base, range, coff_header_off, &coff_header);
    
    // range of optional extension header ("optional" for short)
    SYMS_U32 optional_size = coff_header.size_of_optional_header;
    SYMS_U64 after_coff_header_off = coff_header_off + sizeof(coff_header);
    SYMS_U64 after_optional_header_off = after_coff_header_off + optional_size;
    SYMS_U64Range optional_range = {0};
    optional_range.min = SYMS_ClampTop(after_coff_header_off, data.size);
    optional_range.max = SYMS_ClampTop(after_optional_header_off, data.size);
    
    // get sections
    SYMS_U64 sec_array_off = optional_range.max;
    SYMS_U64 sec_array_raw_opl = sec_array_off + coff_header.section_count*sizeof(SYMS_CoffSectionHeader);
    SYMS_U64 sec_array_opl = SYMS_ClampTop(sec_array_raw_opl, data.size);
    SYMS_U64 clamped_sec_count = (sec_array_opl - sec_array_off)/sizeof(SYMS_CoffSectionHeader);
    SYMS_CoffSectionHeader *sections = (SYMS_CoffSectionHeader*)((SYMS_U8*)base + sec_array_off);
    
    // get symbols
    SYMS_U64 symbol_array_off = coff_header.pointer_to_symbol_table;
    SYMS_U64 symbol_count = coff_header.number_of_symbols;
    
    // get string table
    SYMS_U64 string_table_off = symbol_array_off + sizeof(SYMS_CoffSymbol16) * symbol_count;
    
    // read ptional header
    SYMS_U16 optional_magic = 0;
    SYMS_U64 image_base = 0;
    SYMS_U64 entry_point = 0;
    SYMS_U32 data_dir_count = 0;
    SYMS_U64 virt_section_align = 0;
    SYMS_U64 file_section_align = 0;
    SYMS_U64Range *data_dir_franges = 0;
    if (optional_size > 0){
      // read magic number
      syms_based_range_read_struct(base, optional_range, 0, &optional_magic);
      
      // read optional
      SYMS_U32 reported_data_dir_offset = 0;
      SYMS_U32 reported_data_dir_count = 0;
      switch (optional_magic){
        case SYMS_PE32_MAGIC:
        {
          SYMS_PeOptionalHeader32 pe_optional = {0};
          syms_based_range_read_struct(base, optional_range, 0, &pe_optional);
          
          image_base = pe_optional.image_base;
          entry_point = pe_optional.entry_point_va;
          virt_section_align = pe_optional.section_alignment;
          file_section_align = pe_optional.file_alignment;
          
          reported_data_dir_offset = sizeof(pe_optional);
          reported_data_dir_count = pe_optional.data_dir_count;
        }break;
        case SYMS_PE32PLUS_MAGIC:
        {
          SYMS_PeOptionalHeader32Plus pe_optional = {0};
          syms_based_range_read_struct(base, optional_range, 0, &pe_optional);
          
          image_base = pe_optional.image_base;
          entry_point = pe_optional.entry_point_va;
          virt_section_align = pe_optional.section_alignment;
          file_section_align = pe_optional.file_alignment;
          
          reported_data_dir_offset = sizeof(pe_optional);
          reported_data_dir_count = pe_optional.data_dir_count;
        }break;
      }
      
      SYMS_U32 data_dir_max = (optional_size - reported_data_dir_offset)/sizeof(SYMS_PeDataDirectory);
      data_dir_count = SYMS_ClampTop(reported_data_dir_count, data_dir_max);
      // convert PE directories to ranges
      data_dir_franges = syms_push_array(arena, SYMS_U64Range, data_dir_count);
      for (SYMS_U32 dir_idx = 0; dir_idx < data_dir_count; dir_idx += 1){
        SYMS_U64 dir_offset = optional_range.min + reported_data_dir_offset + sizeof(SYMS_PeDataDirectory)*dir_idx;
        SYMS_PeDataDirectory dir; syms_memzero_struct(&dir);
        syms_based_range_read_struct(base, range, dir_offset, &dir);
        SYMS_U64 file_off = syms_pe_foff_from_voff(sections, clamped_sec_count, dir.virt_off);
        data_dir_franges[dir_idx] = syms_make_u64_inrange(range, file_off, dir.virt_size);
      }
    }
    
    // read info about debug file
    SYMS_U32 dbg_time = 0;
    SYMS_U32 dbg_age = 0;
    SYMS_PeGuid dbg_guid = {0};
    SYMS_U64 dbg_path_off = 0;
    SYMS_U64 dbg_path_size = 0;
    
    if (SYMS_PeDataDirectoryIndex_DEBUG < data_dir_count){
      // read debug directory
      SYMS_PeDebugDirectory dbg_data = {0};
      syms_based_range_read_struct(base, data_dir_franges[SYMS_PeDataDirectoryIndex_DEBUG], 0, &dbg_data);
      // extract external file info from codeview header
      if (dbg_data.type == SYMS_PeDebugDirectoryType_CODEVIEW){
        SYMS_U64 cv_offset = dbg_data.file_offset;
        SYMS_U32 cv_magic = 0;
        syms_based_range_read_struct(base, range, cv_offset, &cv_magic);
        switch (cv_magic){
          default:break;
          case SYMS_CODEVIEW_PDB20_MAGIC:
          {
            SYMS_PeCvHeaderPDB20 cv = {0};
            syms_based_range_read_struct(base, range, cv_offset, &cv);
            // TODO(allen): can we extend the ext match key system to use this in some way?
            dbg_time = cv.time;
            dbg_age = cv.age;
            dbg_path_off = cv_offset + sizeof(cv);
          }break;
          case SYMS_CODEVIEW_PDB70_MAGIC:
          {
            SYMS_PeCvHeaderPDB70 cv = {0};
            syms_based_range_read_struct(base, range, cv_offset, &cv);
            dbg_guid = cv.guid;
            dbg_age = cv.age;
            dbg_path_off = cv_offset + sizeof(cv);
          }break;
        }
        if (dbg_path_off > 0){
          SYMS_String8 dbg_path = syms_based_range_read_string(base, range, dbg_path_off);
          dbg_path_size = dbg_path.size;
        }
      }
    }
    
    // fill result
    result = syms_push_array(arena, SYMS_PeBinAccel, 1);
    result->format = SYMS_FileFormat_PE;
    result->image_base = image_base;
    result->entry_point = entry_point;
    result->is_pe32 = (optional_magic == SYMS_PE32_MAGIC);
    result->virt_section_align = virt_section_align;
    result->file_section_align = file_section_align;
    result->section_array_off = sec_array_off;
    result->section_count = clamped_sec_count;
    result->symbol_array_off = symbol_array_off;
    result->symbol_count = symbol_count;
    result->string_table_off = string_table_off;
    result->dbg_path_off = dbg_path_off;
    result->dbg_path_size = dbg_path_size;
    result->dbg_guid = dbg_guid;
    result->dbg_age = dbg_age;
    result->dbg_time = dbg_time;
    result->arch = syms_arch_from_coff_machine_type(coff_header.machine);
    // store file ranges for data directories
    result->data_dir_franges = data_dir_franges;
    result->data_dir_count = data_dir_count;
  }
  
  return(result);
}

SYMS_API SYMS_Arch
syms_pe_arch_from_bin(SYMS_PeBinAccel *bin){
  SYMS_Arch result = bin->arch;
  return(result);
}

SYMS_API SYMS_ExtFileList
syms_pe_ext_file_list_from_bin(SYMS_Arena *arena, SYMS_String8 data, SYMS_PeBinAccel *bin){
  SYMS_ExtFileList result = {0};
  if (bin->dbg_path_size > 0 && bin->dbg_path_off + bin->dbg_path_size <= data.size){
    SYMS_String8 name = {0};
    name.str = data.str + bin->dbg_path_off;
    name.size = bin->dbg_path_size;
    SYMS_ExtFileNode *node = syms_push_array_zero(arena, SYMS_ExtFileNode, 1);
    SYMS_QueuePush(result.first, result.last, node);
    result.node_count += 1;
    node->ext_file.file_name = name;
    syms_memmove(&node->ext_file.match_key, &bin->dbg_guid, sizeof(bin->dbg_guid));
  }
  return(result);
}

SYMS_API SYMS_SecInfoArray
syms_pe_sec_info_array_from_bin(SYMS_Arena *arena, SYMS_String8 data, SYMS_PeBinAccel *bin){
  SYMS_SecInfoArray result = syms_pecoff_sec_info_from_coff_sec(arena, data, bin->section_array_off, bin->section_count);
  return(result);
}

SYMS_API SYMS_U64
syms_pe_entry_point_voff_from_bin(SYMS_PeBinAccel *bin){
  SYMS_U64 result = bin->entry_point;
  return(result);
}

SYMS_API SYMS_ImportArray
syms_pe_imports_from_bin(SYMS_Arena *arena, SYMS_String8 data, SYMS_PeBinAccel *bin){
  SYMS_ArenaTemp scratch = syms_get_scratch(&arena, 1);
  
  //- allen: get import sources
  SYMS_PeImportDllArray imp_sources[2] = {0};
  imp_sources[0] = syms_pe_regular_imports_from_bin(scratch.arena, data, bin);
  imp_sources[1] = syms_pe_delayed_imports_from_bin(scratch.arena, data, bin);
  
  //- allen: compute total # of imports
  SYMS_U64 total_import_count = 0;
  for (SYMS_U64 i = 0; i < SYMS_ARRAY_SIZE(imp_sources); i += 1){
    SYMS_PeImportDllArray *imports = &imp_sources[i];
    SYMS_PeImportDll *opl = imports->import_dlls + imports->count;
    for (SYMS_PeImportDll *n = imports->import_dlls; n < opl; n += 1){
      total_import_count += n->name_table.count;
    }
  }
  
  //- allen: allocate output array
  SYMS_ImportArray result = {0};
  result.imports = syms_push_array_zero(arena, SYMS_Import, total_import_count);
  result.count = total_import_count;
  
  //- allen: fill output array
  SYMS_Import *imp = result.imports;
  
  // for each source...
  for (SYMS_U64 i = 0; i < SYMS_ARRAY_SIZE(imp_sources); i += 1){
    
    // for each dll...
    SYMS_PeImportDllArray *imports = &imp_sources[i];
    SYMS_PeImportDll *opl = imports->import_dlls + imports->count;
    for (SYMS_PeImportDll *n = imports->import_dlls; n < opl; n += 1){
      
      // for each import name...
      SYMS_U64 import_count = n->name_table.count;
      SYMS_PeImportName *name_ptr = n->name_table.names;
      SYMS_String8 library_name = syms_push_string_copy(arena, n->name);
      for (SYMS_U64 j = 0; j < import_count; j += 1, name_ptr += 1){
        
        // fill imp & advance
        imp->library_name = library_name;
        if (name_ptr->lookup == SYMS_PeImportLookup_NameHint){
          imp->name = name_ptr->name;
        }
        else{
          imp->ordinal = name_ptr->ordinal;
        }
        imp += 1;
      }
    }
  }
  
  //- allen: finish
  syms_release_scratch(scratch);
  return(result);
}

SYMS_API SYMS_ExportArray
syms_pe_exports_from_bin(SYMS_Arena *arena, SYMS_String8 data, SYMS_PeBinAccel *bin){
  // TODO(allen): accelerate voff -> foff
  
  SYMS_ArenaTemp scratch = syms_get_scratch(&arena, 1);
  
  SYMS_U64Range dir_range  = bin->data_dir_franges[SYMS_PeDataDirectoryIndex_EXPORT];
  SYMS_U64Range data_range = syms_make_u64_range(0, data.size);
  void         *base       = (void*)data.str;
  
  SYMS_PeExportTable *header = (SYMS_PeExportTable*)(data.str + dir_range.min);
  
  // export table
  SYMS_U32Array export_table = {0};
  {
    SYMS_U64 foff = syms_pe_bin_foff_from_voff(data, bin, header->export_address_table_voff);
    export_table.u32 = (SYMS_U32*)(data.str + foff);
    export_table.count = header->export_address_table_count;
  }
  
  // name pointer table
  SYMS_U32Array name_table = {0};
  {
    SYMS_U64 foff = syms_pe_bin_foff_from_voff(data, bin, header->name_pointer_table_voff);
    name_table.u32 = (SYMS_U32*)(data.str + foff);
    name_table.count = header->name_pointer_table_count;
  }
  
  // ordinal table
  SYMS_U16Array ordinal_table = {0};
  {
    SYMS_U64 foff = syms_pe_bin_foff_from_voff(data, bin, header->ordinal_table_voff);
    ordinal_table.u16 = (SYMS_U16*)(data.str + foff);
    ordinal_table.count = header->name_pointer_table_count;
  }
  
  // Scan export address table to get accruate count of ordinals. 
  // We can't rely on "name_pointer_table_count" becuase it is possible
  // to define an export without a name through NONAME attribute in DEF file
  SYMS_U64 ordinal_count = 0;
  for (SYMS_U64 voff_idx = 0; voff_idx < export_table.count; voff_idx += 1){
    if (export_table.u32[voff_idx] != 0){
      ordinal_count += 1;
    }
  }
  
  // ordinal base
  SYMS_U32 ordinal_base = header->ordinal_base;
  
  //
  SYMS_U64 max_ordinal = header->export_address_table_count;
  SYMS_B8 *is_ordinal_used = syms_push_array_zero(scratch.arena, SYMS_B8, max_ordinal);
  
  // allocate output array
  SYMS_ExportArray result = {0};
  result.exports = syms_push_array_zero(arena, SYMS_Export, ordinal_count);
  result.count   = ordinal_count;
  
  // fill output array with named exports
  SYMS_Export *exp = result.exports;
  SYMS_Export *exp_opl = result.exports + result.count;
  SYMS_U32 *name_ptr = name_table.u32;
  SYMS_U32 *name_opl = name_table.u32 + name_table.count;
  SYMS_U16 *ordinal_ptr = ordinal_table.u16;
  for (; name_ptr < name_opl;
       name_ptr += 1, ordinal_ptr += 1, exp += 1){
    // get name
    SYMS_String8 name = {0};
    {
      SYMS_U32 foff = syms_pe_bin_foff_from_voff(data, bin, *name_ptr);
      SYMS_String8 raw_name = syms_based_range_read_string(base, data_range, foff);
      name = syms_push_string_copy(arena, raw_name);
    }

    // get ordinal
    SYMS_U16 ordinal_nobase = *ordinal_ptr;

    // mark ordinal
    is_ordinal_used[ordinal_nobase] = syms_true;
    
    // get voff
    SYMS_U32 export_voff = 0;
    if (ordinal_nobase < export_table.count){
      export_voff = export_table.u32[ordinal_nobase];
    }
    
    // make ordinal
    SYMS_U16 ordinal = ordinal_nobase + ordinal_base;

    // fill out export
    exp->name = name;
    exp->address = export_voff;
    exp->ordinal = ordinal;
  }

  // fill output array with noname exports
  for (SYMS_U64 ordinal_nobase = 0; ordinal_nobase < export_table.count; ordinal_nobase += 1){
    SYMS_U32 voff = export_table.u32[ordinal_nobase];
    SYMS_B32 is_voff_taken = (voff != 0);
    SYMS_B32 is_ordinal_free = !is_ordinal_used[ordinal_nobase];
    if (is_voff_taken && is_ordinal_free){
      if (exp < exp_opl){
        exp->name = syms_str8(0,0);
        exp->address = voff;
        exp->ordinal = ordinal_nobase + ordinal_base;
        exp += 1;
      }
    }
  }

  // resolve forward ferences 
  for (exp = result.exports; exp < exp_opl; exp += 1){
    // determine if this is a forwarder
    SYMS_U32 export_foff = syms_pe_bin_foff_from_voff(data, bin, exp->address);
    SYMS_B32 is_forwarder = (dir_range.min <= export_foff && exp->address < dir_range.max);
    
    if (is_forwarder){
      // parse forwarder command string that is formatted like "<DLL>.<IMPORT>" (e.g. NTDLL.RtlZeroMemory)
      SYMS_String8 library_name = syms_str8_lit("<error-dll>");
      SYMS_String8 import_name = syms_str8_lit("<error-import>");
      {
        SYMS_String8 forwarder = syms_based_range_read_string(base, data_range, export_foff);
        SYMS_ArenaTemp temp = syms_arena_temp_begin(arena);
        SYMS_String8List list = syms_string_split(temp.arena, forwarder, '.');
        if (list.node_count == 2){
          library_name = list.first->string;
          import_name  = list.last->string;
        }
        syms_arena_temp_end(temp);
      }
      
      // fill forwarder
      exp->address = 0;
      exp->forwarder_library_name = syms_push_string_copy(arena, library_name);
      exp->forwarder_import_name = syms_push_string_copy(arena, import_name);
    }
  }
  
  syms_release_scratch(scratch);
  return(result);
}

////////////////////////////////
//~ allen: PE Specific Helpers

// pdata

SYMS_API SYMS_U64
syms_pe_binary_search_intel_pdata(SYMS_String8 data, SYMS_PeBinAccel *bin, SYMS_U64 voff){
  // TODO(allen): cleanup pass.
  
  SYMS_ASSERT(bin->arch == SYMS_Arch_X86 || bin->arch == SYMS_Arch_X64);
  SYMS_U64Range range = bin->data_dir_franges[SYMS_PeDataDirectoryIndex_EXCEPTIONS];
  SYMS_U64 pdata_off = range.min;
  SYMS_U64 pdata_count = (range.max - range.min)/sizeof(SYMS_PeIntelPdata);
  
  SYMS_U64 result = 0;
  // check if this bin includes a pdata array
  if (pdata_count > 0){
    SYMS_PeIntelPdata *pdata_array = (SYMS_PeIntelPdata*)(data.str + pdata_off);
    if (voff >= pdata_array[0].voff_first){
      
      // binary search:
      //  find max index s.t. pdata_array[index].voff_first <= voff
      //  we assume (i < j) -> (pdata_array[i].voff_first < pdata_array[j].voff_first)
      SYMS_U64 index = pdata_count;
      SYMS_U64 min = 0;
      SYMS_U64 opl = pdata_count;
      for (;;){
        SYMS_U64 mid = (min + opl)/2;
        SYMS_PeIntelPdata *pdata = pdata_array + mid;
        if (voff < pdata->voff_first){
          opl = mid;
        }
        else if (pdata->voff_first < voff){
          min = mid;
        }
        else{
          index = mid;
          break;
        }
        if (min + 1 >= opl){
          index = min;
          break;
        }
      }
      
      // if we are in range fill result
      {
        SYMS_PeIntelPdata *pdata = pdata_array + index;
        if (pdata->voff_first <= voff && voff < pdata->voff_one_past_last){
          result = pdata_off + index*sizeof(SYMS_PeIntelPdata);
        }
      }
    }
  }
  
  return(result);
}

// sections

SYMS_API SYMS_U64
syms_pe_sec_number_from_voff(SYMS_String8 data, SYMS_PeBinAccel *bin, SYMS_U64 voff){
  SYMS_U64 sec_count = bin->section_count;
  SYMS_CoffSectionHeader *sec_array = (SYMS_CoffSectionHeader*)((SYMS_U8*)data.str + bin->section_array_off);
  SYMS_CoffSectionHeader *sec_ptr = sec_array;
  SYMS_U64 result = 0;
  for (SYMS_U64 i = 1; i <= sec_count; i += 1, sec_ptr += 1){
    if (sec_ptr->virt_off <= voff && voff < sec_ptr->virt_off + sec_ptr->virt_size){
      result = i;
      break;
    }
  }
  return(result);
}

SYMS_API void*
syms_pe_ptr_from_sec_number(SYMS_String8 data, SYMS_PeBinAccel *bin, SYMS_U64 n){
  void *result = 0;
  SYMS_U64 sec_count = bin->section_count;
  if (1 <= n && n <= sec_count){
    SYMS_CoffSectionHeader *sec_array = (SYMS_CoffSectionHeader*)((SYMS_U8*)data.str + bin->section_array_off);
    SYMS_CoffSectionHeader *sec = sec_array + n - 1;
    if (sec->file_size > 0){
      result = data.str + sec->file_off;
    }
  }
  return(result);
}

SYMS_API void*
syms_pe_ptr_from_foff(SYMS_String8 data, SYMS_PeBinAccel *bin, SYMS_U64 foff){
  void *result = 0;
  if (foff < data.size){
    result = data.str + foff;
  }
  return(result);
}

SYMS_API void*
syms_pe_ptr_from_voff(SYMS_String8 data, SYMS_PeBinAccel *bin, SYMS_U64 voff){
  //- get the section for this voff
  SYMS_U64 sec_count = bin->section_count;
  SYMS_CoffSectionHeader *sec_array = (SYMS_CoffSectionHeader*)((SYMS_U8*)data.str + bin->section_array_off);
  SYMS_CoffSectionHeader *sec_ptr = sec_array;
  SYMS_CoffSectionHeader *sec = 0;
  for (SYMS_U64 i = 1; i <= sec_count; i += 1, sec_ptr += 1){
    if (sec_ptr->virt_off <= voff && voff < sec_ptr->virt_off + sec_ptr->virt_size){
      sec = sec_ptr;
      break;
    }
  }
  
  //- adjust to file pointer
  void *result = 0;
  if (sec != 0 & sec_ptr->file_size > 0){
    result = data.str + voff - sec->virt_off + sec->file_off;
  }
  
  return(result);
}

SYMS_API SYMS_U64
syms_pe_foff_from_voff(SYMS_CoffSectionHeader *sections, SYMS_U32 section_count, SYMS_U64 virt_off){
  SYMS_U64 file_off = 0;
  for (SYMS_U32 sect_idx = 0; sect_idx < section_count; sect_idx += 1){
    SYMS_CoffSectionHeader *sect = &sections[sect_idx];
    if (sect->virt_off <= virt_off && virt_off < sect->virt_off + sect->virt_size){
      if (!(sect->flags & SYMS_CoffSectionFlag_CNT_UNINITIALIZED_DATA)){
        file_off = sect->file_off + (virt_off - sect->virt_off);
      }
      break;
    }
  }
  return file_off;
}

SYMS_API SYMS_U64
syms_pe_bin_foff_from_voff(SYMS_String8 data, SYMS_PeBinAccel *bin, SYMS_U64 virt_off){
  return syms_pe_foff_from_voff((SYMS_CoffSectionHeader*)((SYMS_U8*)data.str + bin->section_array_off), bin->section_count, virt_off);
}

// imports & exports

SYMS_API SYMS_U64Array
syms_u64_array_from_null_term_u64_string(SYMS_Arena *arena, SYMS_U64 *src, SYMS_U64 *opl){
  // scan for terminator
  SYMS_U64 *ptr = src;
  for (; *ptr != 0 && ptr < opl; ptr += 1);
  
  // make copy of the array
  SYMS_U64Array result = {0};
  result.count = (SYMS_U64)(ptr - src);
  result.u64 = syms_push_array(arena, SYMS_U64, result.count);
  syms_memmove(result.u64, src, result.count*sizeof(SYMS_U64));
  return(result);
}

SYMS_API SYMS_U64Array
syms_u64_array_from_null_term_u32_string(SYMS_Arena *arena, SYMS_U32 *src, SYMS_U32 *opl){
  // scan for terminator
  SYMS_U32 *ptr = src;
  for (; *ptr != 0 && ptr < opl; ptr += 1);
  
  // make copy of the array (and transform U32 -> U64)
  SYMS_U64Array result = {0};
  result.count = (SYMS_U64)(ptr - src);
  result.u64 = syms_push_array(arena, SYMS_U64, result.count);
  SYMS_U64 *o = result.u64;
  SYMS_U32 *i = src;
  for (; i < ptr; o += 1, i += 1){
    *o = *i;
  }
  return(result);
}

SYMS_API SYMS_U64Array
syms_pe_u64_array_from_null_term_addr_string(SYMS_Arena *arena, SYMS_String8 data,
                                             SYMS_PeBinAccel *bin, SYMS_U64 foff){
  SYMS_U64Array result = {0};
  if (bin->is_pe32){
    SYMS_U32* src = (SYMS_U32*)(data.str + foff);
    SYMS_U32* opl = (SYMS_U32*)(data.str + SYMS_AlignDownPow2(data.size, 4));
    result = syms_u64_array_from_null_term_u32_string(arena, src, opl);
  }
  else {
    SYMS_U64* src = (SYMS_U64*)(data.str + foff);
    SYMS_U64* opl = (SYMS_U64*)(data.str + SYMS_AlignDownPow2(data.size, 8));
    result = syms_u64_array_from_null_term_u64_string(arena, src, opl);
  }
  return(result);
}

SYMS_API SYMS_PeImportName
syms_pe_import_name_from_name_entry(SYMS_Arena *arena, SYMS_String8 data, SYMS_PeBinAccel *bin,
                                    SYMS_U64 name_entry){
  SYMS_PeImportName result = {0};
  
  //- determine case
  SYMS_B32 is_ordinal = syms_false;
  if (bin->is_pe32){
    is_ordinal = !!(name_entry & (1 << 31));
  }
  else{
    is_ordinal = !!(name_entry & ((SYMS_U64)1 << 63));
  }
  
  //- ordinal case
  if (is_ordinal){
    result.lookup = SYMS_PeImportLookup_Ordinal;
    result.ordinal = (name_entry & SYMS_U16_MAX);
  }
  
  //- voff case
  else{
    // TODO(allen): accelerate voff -> foff
    SYMS_U64 foff = syms_pe_bin_foff_from_voff(data, bin, name_entry);
    
    // setup read helper
    void *base = (void*)data.str;
    SYMS_U64Range range = syms_make_u64_range(0, data.size);
    
    // read hint & string
    SYMS_U16 hint = 0;
    syms_based_range_read_struct(base, range, foff, &hint);
    SYMS_String8 name = syms_based_range_read_string(base, range, foff + sizeof(SYMS_U16));
    
    // fill result
    result.lookup = SYMS_PeImportLookup_NameHint;
    result.name = syms_push_string_copy(arena, name);
    result.ordinal = hint;
  }
  
  return(result);
}

SYMS_API SYMS_PeImportNameArray
syms_pe_import_name_array_from_name_entries(SYMS_Arena *arena, SYMS_String8 data, SYMS_PeBinAccel *bin,
                                            SYMS_U64Array name_entries){
  //- allocate output array
  SYMS_PeImportNameArray result = {0};
  result.names = syms_push_array_zero(arena, SYMS_PeImportName, name_entries.count);
  result.count = name_entries.count;
  
  //- setup read helper
  void *base = (void*)data.str;
  SYMS_U64Range range = syms_make_u64_range(0, data.size);
  
  //- determine ordinal mask
  SYMS_U64 ordinal_mask = ((SYMS_U64)1 << 63);
  if (bin->is_pe32){
    ordinal_mask = (1 << 31);
  }
  
  //- fill output
  SYMS_PeImportName *name_ptr = result.names;
  SYMS_U64 *name_entry_ptr = name_entries.u64;
  for (SYMS_U64 i = 0;
       i < name_entries.count;
       i += 1, name_ptr += 1, name_entry_ptr += 1){
    SYMS_U64 name_entry = *name_entry_ptr;
    SYMS_B32 is_ordinal = !!(name_entry & ordinal_mask);
    
    if (is_ordinal){
      name_ptr->lookup = SYMS_PeImportLookup_Ordinal;
      name_ptr->ordinal = (name_entry & SYMS_U16_MAX);
    }
    else{
      // TODO(allen): accelerate voff -> foff
      SYMS_U64 foff = syms_pe_bin_foff_from_voff(data, bin, name_entry);
      
      // read hint & string
      SYMS_U16 hint = 0;
      syms_based_range_read_struct(base, range, foff, &hint);
      SYMS_String8 name = syms_based_range_read_string(base, range, foff + sizeof(SYMS_U16));
      
      // fill result
      name_ptr->lookup = SYMS_PeImportLookup_NameHint;
      name_ptr->name = syms_push_string_copy(arena, name);
      name_ptr->ordinal = hint;
    }
  }
  
  return(result);
}

SYMS_API SYMS_PeImportDllArray
syms_pe_regular_imports_from_bin(SYMS_Arena *arena, SYMS_String8 data, SYMS_PeBinAccel *bin){
  //- grab ranges
  void *base = (void *)data.str;
  SYMS_U64Range dir_range = bin->data_dir_franges[SYMS_PeDataDirectoryIndex_IMPORT];
  SYMS_U64Range data_range = syms_make_u64_range(0, data.size);
  
  //- get import entries array
  SYMS_PeImportEntry *import_entries = (SYMS_PeImportEntry*)(data.str + dir_range.min);
  
  //- count imports
  SYMS_U64 max_import_count = (dir_range.max - dir_range.min)/sizeof(SYMS_PeImportEntry);
  SYMS_U64 import_count = max_import_count;
  {
    SYMS_PeImportEntry *ptr = import_entries;
    SYMS_PeImportEntry *opl = import_entries + max_import_count;
    for (; ptr < opl; ptr += 1){
      if (syms_memisnull_struct(ptr)){
        break;
      }
    }
    import_count = (SYMS_U64)(ptr - import_entries);
  }
  
  //- allocate output array
  SYMS_PeImportDllArray result = {0};
  result.import_dlls = syms_push_array_zero(arena, SYMS_PeImportDll, import_count);
  result.count = import_count;
  
  //- fill output
  SYMS_PeImportDll *dll = result.import_dlls;
  SYMS_PeImportEntry *import_entry = import_entries;
  SYMS_PeImportEntry *opl = import_entries + import_count;
  for (; import_entry < opl; import_entry += 1, dll += 1){
    // TODO(allen): accelerate voff -> foff
    
    // get name
    SYMS_String8 name = {0};
    {
      SYMS_U64 foff = syms_pe_bin_foff_from_voff(data, bin, import_entry->name_voff);
      SYMS_String8 name_raw = syms_based_range_read_string(base, data_range, foff);
      name = syms_push_string_copy(arena, name_raw);
    }
    
    // get name table
    SYMS_PeImportNameArray name_table = {0};
    {
      SYMS_U64 foff = syms_pe_bin_foff_from_voff(data, bin, import_entry->lookup_table_voff);
      SYMS_U64Array entries = syms_pe_u64_array_from_null_term_addr_string(arena, data, bin, foff);
      name_table = syms_pe_import_name_array_from_name_entries(arena, data, bin, entries);
    }
    
    // fill dll slot
    dll->name = name;
    dll->name_table = name_table;
  }
  
  return(result);
}

SYMS_API SYMS_PeImportDllArray
syms_pe_delayed_imports_from_bin(SYMS_Arena *arena, SYMS_String8 data, SYMS_PeBinAccel *bin){
  SYMS_ArenaTemp scratch = syms_get_scratch(&arena, 1);
  
  //- grab prerequisites
  void         *base       = (void *)data.str;
  SYMS_U64Range data_range = syms_make_u64_range(0, data.size);
  SYMS_U64Range dir_range  = bin->data_dir_franges[SYMS_PeDataDirectoryIndex_DELAY_IMPORT];
  
  //- get import entries array
  SYMS_PeDelayedImportEntry *import_entries = (SYMS_PeDelayedImportEntry*)(data.str + dir_range.min);
  
  //- count imports
  SYMS_U64 max_import_count = (dir_range.max - dir_range.min)/sizeof(SYMS_PeDelayedImportEntry);
  SYMS_U64 import_count = max_import_count;
  {
    SYMS_PeDelayedImportEntry *ptr = import_entries;
    SYMS_PeDelayedImportEntry *opl = import_entries + max_import_count;
    for (; ptr < opl; ptr += 1){
      if (syms_memisnull_struct(ptr)){
        break;
      }
    }
    import_count = (SYMS_U64)(ptr - import_entries);
  }
  
  //- allocate output array
  SYMS_PeImportDllArray result = {0};
  result.import_dlls = syms_push_array_zero(arena, SYMS_PeImportDll, import_count);
  result.count = import_count;
  
  //- parse directory
  SYMS_PeImportDll *dll = result.import_dlls;
  SYMS_PeDelayedImportEntry *import_ptr = import_entries;
  SYMS_PeDelayedImportEntry *opl = import_entries + import_count;
  for (SYMS_U64 i = 0; i < import_count; i += 1, import_ptr += 1, dll += 1){
    // TODO(allen): accelerate voff -> foff
    
    //- get name
    SYMS_String8 name = {0};
    {
      SYMS_U64 foff = syms_pe_bin_foff_from_voff(data, bin, import_ptr->name_voff);
      SYMS_String8 raw_name = syms_based_range_read_string(base, data_range, foff);
      name = syms_push_string_copy(arena, raw_name);
    }
    
    //- get name table
    SYMS_PeImportNameArray name_table = {0};
    {
      SYMS_U64 foff = syms_pe_bin_foff_from_voff(data, bin, import_ptr->name_table_voff);
      SYMS_U64Array entries = syms_pe_u64_array_from_null_term_addr_string(scratch.arena, data, bin, foff);
      name_table = syms_pe_import_name_array_from_name_entries(arena, data, bin, entries);
    }
    
    //- bound imports
    SYMS_U64Array bound_table = {0};
    {
      SYMS_U64 foff = syms_pe_bin_foff_from_voff(data, bin, import_ptr->bound_table_voff);
      bound_table = syms_pe_u64_array_from_null_term_addr_string(arena, data, bin, foff);
    }
    
    //- build node
    dll->name         = name;
    dll->name_table   = name_table;
    dll->bound_table  = bound_table;
  }
  
  syms_release_scratch(scratch);
  
  return(result);
}

SYMS_API SYMS_String8
syms_pe_binary_name_from_bin(SYMS_Arena *arena, SYMS_String8 data, SYMS_PeBinAccel *bin){
  // TODO(allen): accelerate voff -> foff
  SYMS_U64Range dir_range  = bin->data_dir_franges[SYMS_PeDataDirectoryIndex_EXPORT];
  SYMS_U64Range data_range = syms_make_u64_range(0, data.size);
  void         *base       = (void*)data.str;
  SYMS_PeExportTable *header = (SYMS_PeExportTable*)(data.str + dir_range.min);
  SYMS_U32 binary_name_foff = syms_pe_bin_foff_from_voff(data, bin, header->name_voff);
  SYMS_String8 binary_name = syms_based_range_read_string(base, data_range, binary_name_foff);
  SYMS_String8 result = syms_push_string_copy(arena, binary_name);
  return(result);
}

#endif //SYMS_PE_PARSER_C

