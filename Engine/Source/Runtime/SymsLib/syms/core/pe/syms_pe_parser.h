// Copyright Epic Games, Inc. All Rights Reserved.
/* date = June 7th 2021 1:06 pm */

#ifndef SYMS_PE_PARSER_H
#define SYMS_PE_PARSER_H

////////////////////////////////
//~ allen: PE Parser Types

typedef struct SYMS_PeFileAccel{
  SYMS_FileFormat format;
  SYMS_U32 coff_off;
} SYMS_PeFileAccel;

typedef struct SYMS_PeBinAccel{
  SYMS_FileFormat format;
  SYMS_U64 image_base;
  SYMS_U64 entry_point;
  SYMS_B32 is_pe32;
  SYMS_U64 virt_section_align;
  SYMS_U64 file_section_align;
  SYMS_U64 section_array_off;
  SYMS_U64 section_count;
  SYMS_U64 symbol_array_off;
  SYMS_U64 symbol_count;
  SYMS_U64 string_table_off;
  SYMS_U64 dbg_path_off;
  SYMS_U64 dbg_path_size;
  SYMS_PeGuid dbg_guid;
  SYMS_U32 dbg_age;
  SYMS_U32 dbg_time;
  SYMS_Arch arch;
  SYMS_U64Range *data_dir_franges;
  SYMS_U32 data_dir_count;
} SYMS_PeBinAccel;

////////////////////////////////
//~ allen: Imports

typedef enum SYMS_PeImportLookup{
  SYMS_PeImportLookup_NULL,
  SYMS_PeImportLookup_Ordinal,
  SYMS_PeImportLookup_NameHint,
  SYMS_PeImportLookup_COUNT,
} SYMS_PeImportLookup;

typedef struct SYMS_PeImportName{
  SYMS_String8 name;
  SYMS_PeImportLookup lookup;
  SYMS_U16 ordinal;
  SYMS_U16 _padding_;
} SYMS_PeImportName;

typedef struct SYMS_PeImportNameArray{
  SYMS_PeImportName *names;
  SYMS_U64 count;
} SYMS_PeImportNameArray;

typedef struct SYMS_PeImportDll{
  SYMS_String8 name;
  SYMS_PeImportNameArray name_table;
  SYMS_U64Array bound_table;
} SYMS_PeImportDll;

typedef struct SYMS_PeImportDllArray{
  SYMS_PeImportDll *import_dlls;
  SYMS_U64 count;
} SYMS_PeImportDllArray;


////////////////////////////////
//~ allen: PE Parser Functions

// accelerators
SYMS_API SYMS_PeFileAccel* syms_pe_file_accel_from_data(SYMS_Arena *arena, SYMS_String8 data);
SYMS_API SYMS_PeBinAccel*  syms_pe_bin_accel_from_file(SYMS_Arena *arena, SYMS_String8 data,
                                                       SYMS_PeFileAccel *file);

SYMS_API SYMS_Arch         syms_pe_arch_from_bin(SYMS_PeBinAccel *bin);
SYMS_API SYMS_ExtFileList  syms_pe_ext_file_list_from_bin(SYMS_Arena *arena, SYMS_String8 data,
                                                          SYMS_PeBinAccel *bin);
SYMS_API SYMS_SecInfoArray syms_pe_sec_info_array_from_bin(SYMS_Arena *arena, SYMS_String8 data,
                                                           SYMS_PeBinAccel *bin);
SYMS_API SYMS_U64          syms_pe_entry_point_voff_from_bin(SYMS_PeBinAccel *bin);
SYMS_API SYMS_ImportArray  syms_pe_imports_from_bin(SYMS_Arena *arena, SYMS_String8 data, SYMS_PeBinAccel *bin);
SYMS_API SYMS_ExportArray  syms_pe_exports_from_bin(SYMS_Arena *arena, SYMS_String8 data, SYMS_PeBinAccel *bin);

////////////////////////////////
//~ allen: PE Specific Helpers

// pdata
SYMS_API SYMS_U64 syms_pe_binary_search_intel_pdata(SYMS_String8 data, SYMS_PeBinAccel *bin, SYMS_U64 voff);
// sections
SYMS_API SYMS_U64 syms_pe_sec_number_from_voff(SYMS_String8 data, SYMS_PeBinAccel *bin, SYMS_U64 voff);

SYMS_API void*    syms_pe_ptr_from_sec_number(SYMS_String8 data, SYMS_PeBinAccel *bin, SYMS_U64 n);
SYMS_API void*    syms_pe_ptr_from_foff(SYMS_String8 data, SYMS_PeBinAccel *bin, SYMS_U64 foff);
SYMS_API void*    syms_pe_ptr_from_voff(SYMS_String8 data, SYMS_PeBinAccel *bin, SYMS_U64 voff);

SYMS_API SYMS_U64 syms_pe_foff_from_voff(SYMS_CoffSectionHeader *sections, SYMS_U32 section_count, SYMS_U64 voff);
SYMS_API SYMS_U64 syms_pe_bin_foff_from_voff(SYMS_String8 data, SYMS_PeBinAccel *bin, SYMS_U64 voff);

// imports & exports
SYMS_API SYMS_U64Array syms_u64_array_from_null_term_u64_string(SYMS_Arena *arena, SYMS_U64 *src, SYMS_U64 *opl);
SYMS_API SYMS_U64Array syms_u64_array_from_null_term_u32_string(SYMS_Arena *arena, SYMS_U32 *src, SYMS_U32 *opl);
SYMS_API SYMS_U64Array syms_pe_u64_array_from_null_term_addr_string(SYMS_Arena *arena, SYMS_String8 data,
                                                                    SYMS_PeBinAccel *bin, SYMS_U64 foff);

SYMS_API SYMS_PeImportName syms_pe_import_name_from_name_entry(SYMS_Arena *arena, SYMS_String8 data,
                                                               SYMS_PeBinAccel *bin, SYMS_U64 name_entry);
SYMS_API SYMS_PeImportNameArray syms_pe_import_name_array_from_name_entries(SYMS_Arena *arena,
                                                                            SYMS_String8 data,
                                                                            SYMS_PeBinAccel *bin,
                                                                            SYMS_U64Array name_entries);

SYMS_API SYMS_PeImportDllArray syms_pe_regular_imports_from_bin(SYMS_Arena *arena, SYMS_String8 data,
                                                                SYMS_PeBinAccel *bin);
SYMS_API SYMS_PeImportDllArray syms_pe_delayed_imports_from_bin(SYMS_Arena *arena, SYMS_String8 data,
                                                                SYMS_PeBinAccel *bin);

SYMS_API SYMS_String8     syms_pe_binary_name_from_bin(SYMS_Arena *arena, SYMS_String8 data,
                                                       SYMS_PeBinAccel *bin);

#endif //SYMS_PE_PARSER_H
