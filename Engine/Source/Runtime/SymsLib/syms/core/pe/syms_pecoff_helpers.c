// Copyright Epic Games, Inc. All Rights Reserved.

////////////////////////////////
//~ allen): PE/COFF Sections

SYMS_API SYMS_CoffSectionHeader*
syms_pecoff_sec_hdr_from_n(SYMS_String8 data, SYMS_U64 sec_hrds_off, SYMS_U64 n){
  SYMS_CoffSectionHeader *result = &syms_pecoff_sec_hdr_nil;
  if (1 <= n && sec_hrds_off + n*sizeof(SYMS_CoffSectionHeader) <= data.size){
    result = (SYMS_CoffSectionHeader*)(data.str + sec_hrds_off + (n - 1)*sizeof(SYMS_CoffSectionHeader));
  }
  return(result);
}

SYMS_API SYMS_U64Range
syms_pecoff_name_range_from_hdr_off(SYMS_String8 data, SYMS_U64 sec_hdr_off){
  SYMS_U64Range result = {0};
  
  // check offset
  if (sec_hdr_off + 8 <= data.size){
    SYMS_U8 *name_data = (data.str + sec_hdr_off);
    
    // scan to end (name + 8 or null terminator)
    SYMS_U8 *name_ptr = name_data;
    SYMS_U8 *name_opl = name_data + 8;
    for (;name_ptr < name_opl && *name_ptr != 0; name_ptr += 1);
    
    // resolve name
    SYMS_U8 *name_min = name_data;
    SYMS_U8 *name_max = name_ptr;
    if (name_data[0] == '/'){
      // ascii -> integer
      SYMS_String8 ascii_off = syms_str8_range(name_data + 1, name_ptr);
      SYMS_U64 off = syms_u64_from_string(ascii_off, 10);
      
      // scan string
      SYMS_U8 *data_opl = data.str + data.size;
      if (off < data.size){
        name_min = data.str + off;
        name_max = name_min;
        for (;name_max < data_opl && *name_max != 0; name_max += 1);
      }
    }
    
    // convert ptr range to result
    result.min = (SYMS_U64)(name_min - data.str);
    result.max = (SYMS_U64)(name_max - data.str);
  }
  
  return(result);
}

SYMS_API SYMS_String8
syms_pecoff_name_from_hdr_off(SYMS_Arena *arena, SYMS_String8 data, SYMS_U64 sec_hdr_off){
  // locate name
  SYMS_U64Range range = syms_pecoff_name_range_from_hdr_off(data, sec_hdr_off);
  // copy range onto arena
  SYMS_U64 size = range.max - range.min;
  SYMS_String8 result = {0};
  result.str = syms_push_array(arena, SYMS_U8, size);
  result.size = size;
  syms_memmove(result.str, data.str + range.min, size);
  return(result);
}

SYMS_API SYMS_U64Array
syms_pecoff_voff_array_from_coff_hdr_array(SYMS_Arena *arena, SYMS_CoffSectionHeader *sec_hdrs,
                                           SYMS_U64 sec_count){
  // allocate output
  SYMS_U64Array result = {0};
  result.u64 = syms_push_array(arena, SYMS_U64, sec_count);
  result.count = sec_count;
  
  // fill output
  SYMS_CoffSectionHeader *src = sec_hdrs;
  SYMS_U64 *dst = result.u64;
  for (SYMS_U64 i = 0; i < sec_count; i += 1, src += 1, dst += 1){
    *dst = src->virt_off;
  }
  
  return(result);
}


SYMS_API SYMS_SecInfoArray
syms_pecoff_sec_info_from_coff_sec(SYMS_Arena *arena, SYMS_String8 data,
                                   SYMS_U64 sec_hdrs_off, SYMS_U64 sec_count){
  SYMS_SecInfoArray result = {0};
  
  // check offset
  if (sec_hdrs_off + sec_count*sizeof(SYMS_CoffSectionHeader) <= data.size){
    
    // allocate output array
    result.sec_info = syms_push_array(arena, SYMS_SecInfo, sec_count);
    result.count = sec_count;
    
    // fill output array
    SYMS_CoffSectionHeader *hdr = (SYMS_CoffSectionHeader*)(data.str + sec_hdrs_off);
    SYMS_CoffSectionHeader *hdr_opl = hdr + sec_count;
    SYMS_SecInfo *ptr = result.sec_info;
    SYMS_U64 off = sec_hdrs_off;
    for (; hdr < hdr_opl;
         hdr += 1, ptr += 1, off += sizeof(SYMS_CoffSectionHeader)){
      ptr->name = syms_pecoff_name_from_hdr_off(arena, data, off);
      ptr->vrange = syms_make_u64_range(hdr->virt_off, hdr->virt_off + hdr->virt_size);
      ptr->frange = syms_make_u64_range(hdr->file_off, hdr->file_off + hdr->file_size);
    }
  }
  
  return(result);
}
