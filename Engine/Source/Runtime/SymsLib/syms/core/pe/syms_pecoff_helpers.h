// Copyright Epic Games, Inc. All Rights Reserved.
/* date = May 19th 2022 11:39 am */

#ifndef SYMS_PECOFF_HELPERS_H
#define SYMS_PECOFF_HELPERS_H

////////////////////////////////
//~ allen): PE/COFF Sections

SYMS_READ_ONLY SYMS_GLOBAL SYMS_CoffSectionHeader syms_pecoff_sec_hdr_nil = {0};

SYMS_API SYMS_CoffSectionHeader* syms_pecoff_sec_hdr_from_n(SYMS_String8 data, SYMS_U64 sec_hrds_off,
                                                            SYMS_U64 n);

SYMS_API SYMS_U64Range syms_pecoff_name_range_from_hdr_off(SYMS_String8 data, SYMS_U64 sec_hdr_off);
SYMS_API SYMS_String8  syms_pecoff_name_from_hdr_off(SYMS_Arena *arena, SYMS_String8 data,
                                                     SYMS_U64 sec_hdr_off);

SYMS_API SYMS_U64Array syms_pecoff_voff_accelerator_from_coff_hdr_array(SYMS_Arena *arena,
                                                                        SYMS_CoffSectionHeader *sec_hdrs,
                                                                        SYMS_U64 sec_count);

SYMS_API SYMS_SecInfoArray syms_pecoff_sec_info_from_coff_sec(SYMS_Arena *arena, SYMS_String8 data,
                                                              SYMS_U64 sec_hdrs_off,
                                                              SYMS_U64 sec_count);


#endif //SYMS_PECOFF_HELPERS_H
