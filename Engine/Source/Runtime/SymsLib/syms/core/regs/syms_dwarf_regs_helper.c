// Copyright Epic Games, Inc. All Rights Reserved.

#ifndef SYMS_DWARF_REGS_CELPER_C
#define SYMS_DWARF_REGS_CELPER_C

////////////////////////////////
//~ NOTE(allen): Register Conversion

SYMS_API void
syms_dw_regs__set_dw_regs_from_full_regs(SYMS_DwRegsX64 *dst, SYMS_RegX64 *src){
  dst->rax = src->rax.u64;
  dst->rdx = src->rdx.u64;
  dst->rcx = src->rcx.u64;
  dst->rbx = src->rbx.u64;
  dst->rsi = src->rsi.u64;
  dst->rdi = src->rdi.u64;
  dst->rsp = src->rsp.u64;
  dst->rbp = src->rbp.u64;
  dst->r8  = src->r8.u64;
  dst->r9  = src->r9.u64;
  dst->r10 = src->r10.u64;
  dst->r11 = src->r11.u64;
  dst->r12 = src->r12.u64;
  dst->r13 = src->r13.u64;
  dst->r14 = src->r14.u64;
  dst->r15 = src->r15.u64;
  dst->rip = src->rip.u64;
}

SYMS_API void
syms_dw_regs__set_full_regs_from_dw_regs(SYMS_RegX64 *dst, SYMS_DwRegsX64 *src){
  dst->rax.u64 = src->rax;
  dst->rdx.u64 = src->rdx;
  dst->rcx.u64 = src->rcx;
  dst->rbx.u64 = src->rbx;
  dst->rsi.u64 = src->rsi;
  dst->rdi.u64 = src->rdi;
  dst->rsp.u64 = src->rsp;
  dst->rbp.u64 = src->rbp;
  dst->r8.u64  = src->r8;
  dst->r9.u64  = src->r9;
  dst->r10.u64 = src->r10;
  dst->r11.u64 = src->r11;
  dst->r12.u64 = src->r12;
  dst->r13.u64 = src->r13;
  dst->r14.u64 = src->r14;
  dst->r15.u64 = src->r15;
  dst->rip.u64 = src->rip;
}

#endif //SYMS_DWARF_REGS_CELPER_C
