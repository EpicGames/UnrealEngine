// Copyright Epic Games, Inc. All Rights Reserved.
/* date = October 4th 2021 6:03 pm */

#ifndef SYMS_DWARF_REGS_HELPER_H
#define SYMS_DWARF_REGS_HELPER_H

////////////////////////////////
//~ NOTE(allen): Register Conversion

SYMS_API void syms_dw_regs__set_unwind_regs_from_full_regs(SYMS_DwRegsX64 *dst, SYMS_RegX64 *src);
SYMS_API void syms_dw_regs__set_full_regs_from_unwind_regs(SYMS_RegX64 *dst, SYMS_DwRegsX64 *src);

#endif //SYMS_DWARF_REGS_HELPER_H
