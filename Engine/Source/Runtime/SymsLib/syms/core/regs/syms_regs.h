// Copyright Epic Games, Inc. All Rights Reserved.

#ifndef SYMS_REGS_H
#define SYMS_REGS_H

////////////////////////////////
// NOTE(allen): Basic Register Types

typedef union SYMS_Reg16{
  SYMS_U8 v[2];
  SYMS_U16 u16;
} SYMS_Reg16;

typedef union SYMS_Reg32{
  SYMS_U8 v[4];
  SYMS_U32 u32;
  SYMS_F32 f32;
} SYMS_Reg32;

typedef union SYMS_Reg64{
  SYMS_U8 v[8];
  SYMS_U64 u64;
  SYMS_F64 f64;
} SYMS_Reg64;

#pragma pack(push, 1)
typedef struct SYMS_Reg80{
  SYMS_U64 int1_frac63;
  SYMS_U16 sign1_exp15;
} SYMS_Reg80;
#pragma pack(pop)

typedef union SYMS_Reg128{
  SYMS_U8 v[16];
  SYMS_U32 u32[4];
  SYMS_F32 f32[4];
  SYMS_U64 u64[2];
  SYMS_F64 f64[2];
} SYMS_Reg128;

typedef union SYMS_Reg256{
  SYMS_U8 v[32];
  SYMS_U32 u32[8];
  SYMS_F32 f32[8];
  SYMS_U64 u64[4];
  SYMS_F64 f64[4];
} SYMS_Reg256;

typedef struct SYMS_RegSection{
  // TODO(allen): naming pass byte_*; double check these are encoded as byte sizes
  SYMS_U16 off;
  SYMS_U16 size;
} SYMS_RegSection;

typedef struct SYMS_RegSlice{
  SYMS_U16 reg_id;
  SYMS_U8 byte_off;
  SYMS_U8 byte_size;
} SYMS_RegSlice;

#endif // SYMS_REGS_H
