// Copyright Epic Games, Inc. All Rights Reserved.

#ifndef SYMS_REGS_HELPERS_C
#define SYMS_REGS_HELPERS_C

SYMS_API SYMS_U64
syms_reg_count_from_arch(SYMS_Arch arch)
{
  SYMS_U64 result = 0;
  switch(arch)
  {
    default:break;
    case SYMS_Arch_X64:   {result = SYMS_RegX64Code_COUNT;}break;
    case SYMS_Arch_X86:   {result = SYMS_RegX86Code_COUNT;}break;
    case SYMS_Arch_ARM:   {result = 0;}break; // NOTE(rjf): Currently unsupported.
    case SYMS_Arch_ARM32: {result = 0;}break; // NOTE(rjf): Currently unsupported.
    case SYMS_Arch_PPC64: {result = 0;}break; // NOTE(rjf): Currently unsupported.
    case SYMS_Arch_PPC:   {result = 0;}break; // NOTE(rjf): Currently unsupported.
    case SYMS_Arch_IA64:  {result = 0;}break; // NOTE(rjf): Currently unsupported.
  }
  return result;
}

SYMS_API SYMS_String8 *
syms_reg_name_table_from_arch(SYMS_Arch arch)
{
  SYMS_String8 *result = 0;
  switch(arch)
  {
    default:break;
    case SYMS_Arch_X64:   {result = syms_reg_names_X64;}break;
    case SYMS_Arch_X86:   {result = syms_reg_names_X86;}break;
    case SYMS_Arch_ARM:   {result = 0;}break; // NOTE(rjf): Currently unsupported.
    case SYMS_Arch_ARM32: {result = 0;}break; // NOTE(rjf): Currently unsupported.
    case SYMS_Arch_PPC64: {result = 0;}break; // NOTE(rjf): Currently unsupported.
    case SYMS_Arch_PPC:   {result = 0;}break; // NOTE(rjf): Currently unsupported.
    case SYMS_Arch_IA64:  {result = 0;}break; // NOTE(rjf): Currently unsupported.
  }
  return result;
}

SYMS_API SYMS_RegSection *
syms_reg_section_table_from_arch(SYMS_Arch arch)
{
  SYMS_RegSection *result = 0;
  switch(arch)
  {
    default:break;
    case SYMS_Arch_X64:   {result = syms_reg_section_X64;}break;
    case SYMS_Arch_X86:   {result = syms_reg_section_X86;}break;
    case SYMS_Arch_ARM:   {result = 0;}break; // NOTE(rjf): Currently unsupported.
    case SYMS_Arch_ARM32: {result = 0;}break; // NOTE(rjf): Currently unsupported.
    case SYMS_Arch_PPC64: {result = 0;}break; // NOTE(rjf): Currently unsupported.
    case SYMS_Arch_PPC:   {result = 0;}break; // NOTE(rjf): Currently unsupported.
    case SYMS_Arch_IA64:  {result = 0;}break; // NOTE(rjf): Currently unsupported.
  }
  return result;
}

SYMS_API SYMS_RegSection
syms_reg_section_from_reg_id(SYMS_Arch arch, SYMS_RegID reg_id){
  SYMS_RegSection result = {0};
  SYMS_U64 count = syms_reg_count_from_arch(arch);
  SYMS_RegSection *table = syms_reg_section_table_from_arch(arch);
  if (reg_id < count){
    result = table[reg_id];
  }
  return(result);
}

#endif // SYMS_REGS_HELPERS_C
