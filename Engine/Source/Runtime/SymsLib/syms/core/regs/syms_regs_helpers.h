// Copyright Epic Games, Inc. All Rights Reserved.
/* date = February 15th 2022 11:27 am */

#ifndef SYMS_REGS_HELPERS_H
#define SYMS_REGS_HELPERS_H

SYMS_API SYMS_U64 syms_reg_count_from_arch(SYMS_Arch arch);
SYMS_API SYMS_String8 *syms_reg_name_table_from_arch(SYMS_Arch arch);
SYMS_API SYMS_RegSection *syms_reg_section_table_from_arch(SYMS_Arch arch);

SYMS_API SYMS_RegSection syms_reg_section_from_reg_id(SYMS_Arch arch, SYMS_RegID reg_id);

#endif // SYMS_REGS_HELPERS_H
