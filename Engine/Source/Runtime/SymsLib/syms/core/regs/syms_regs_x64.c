// Copyright Epic Games, Inc. All Rights Reserved.

#ifndef SYMS_REGS_X64_C
#define SYMS_REGS_X64_C

////////////////////////////////
// NOTE(allen): x64 Register Layout Transformers

SYMS_API void
syms_x64_regs__set_full_regs_from_xsave_legacy(SYMS_RegX64 *dst, SYMS_XSaveLegacy *src){
  dst->fcw.u16 = src->fcw;
  dst->fsw.u16 = src->fsw;
  // TODO(allen): fix tag word
  dst->ftw.u16 = src->ftw;
  dst->fop.u16 = src->fop;
  // TODO(allen): these 32-bit things should be 64-bit in x64 
  dst->fip.u32 = src->b64.fip;
  // TODO(allen): these 16-bit things don't belong in x64
  dst->fcs.u16 = 0;
  dst->fdp.u32 = src->b64.fdp;
  dst->fds.u16 = 0;
  dst->mxcsr.u32 = src->mxcsr;
  dst->mxcsr_mask.u32 = src->mxcsr_mask;
  
  {
    SYMS_U8 *float_s = src->st_space;
    SYMS_Reg80 *float_d = &dst->st0;
    for (SYMS_U32 n = 0; n < 8; n += 1, float_s += 16, float_d += 1){
      syms_memmove(float_d, float_s, sizeof(*float_d));
    }
  }
  
  {
    SYMS_U8 *xmm_s = src->xmm_space;
    SYMS_Reg256 *xmm_d = &dst->ymm0;
    for (SYMS_U32 n = 0; n < 16; n += 1, xmm_s += 16, xmm_d += 1){
      syms_memmove(xmm_d, xmm_s, 16);
    }
  }
}

SYMS_API void
syms_x64_regs__set_xsave_legacy_from_full_regs(SYMS_XSaveLegacy *dst, SYMS_RegX64 *src){
  dst->fcw = src->fcw.u16;
  dst->fsw = src->fsw.u16;
  dst->ftw = src->ftw.u16;
  dst->fop = src->fop.u16;
  // TODO(allen): fip/fdp should be u64 in x64
  dst->b64.fip = src->fip.u32;
  dst->b64.fdp = src->fdp.u32;
  dst->mxcsr = src->mxcsr.u32;
  dst->mxcsr_mask = src->mxcsr_mask.u32;
  
  {
    SYMS_U8 *float_d = dst->st_space;
    SYMS_Reg80 *float_s = &src->st0;
    for (SYMS_U32 n = 0; n < 8; n += 1, float_s += 1, float_d += 16){
      syms_memmove(float_d, float_s, sizeof(*float_s));
    }
  }
  
  {
    SYMS_U8 *xmm_d = dst->xmm_space;
    SYMS_Reg256 *xmm_s = &src->ymm0;
    for (SYMS_U32 n = 0; n < 16; n += 1, xmm_s += 1, xmm_d += 16){
      syms_memmove(xmm_d, xmm_s, 16);
    }
  }
}

SYMS_API void
syms_x64_regs__set_full_regs_from_xsave_avx_extension(SYMS_RegX64 *dst, void *src){
  SYMS_U8 *ymm_s = (SYMS_U8*)src;
  SYMS_Reg256 *ymm_d = &dst->ymm0;
  for (U32 n = 0; n < 16; n += 1, ymm_s += 16, ymm_d += 1){
    MemoryCopy(((SYMS_U8*)ymm_d) + 16, ymm_s, 16);
  }
}

SYMS_API void
syms_x64_regs__set_xsave_avx_extension_from_full_regs(void *dst, SYMS_RegX64 *src){
  SYMS_U8 *ymm_d = (SYMS_U8*)dst;
  SYMS_Reg256 *ymm_s = &src->ymm0;
  for (SYMS_U32 n = 0; n < 16; n += 1, ymm_s += 1, ymm_d += 16){
    syms_memmove(ymm_d, ((SYMS_U8*)ymm_s) + 16, 16);
  }
}

#endif // SYMS_REGS_X64_C

