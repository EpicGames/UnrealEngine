// Copyright Epic Games, Inc. All Rights Reserved.
/* date = October 15th 2021 11:09 am */

#ifndef SYMS_REGS_X64_H
#define SYMS_REGS_X64_H

////////////////////////////////
// NOTE(allen): x64 Specific Register Layouts (from the Intel Manual)

// (32-Bit Protected Mode Format)
struct SYMS_FSave{
  // control registers
  SYMS_U16 fcw;
  SYMS_U16 _pad0;
  SYMS_U16 fsw;
  SYMS_U16 _pad1;
  SYMS_U16 ftw;
  SYMS_U16 _pad2;
  SYMS_U32 fip;
  SYMS_U16 fips;
  SYMS_U16 fop;
  SYMS_U32 fdp;
  SYMS_U16 fds;
  SYMS_U16 _pad3;
  
  // data registers
  SYMS_U8 st[80];
};

typedef struct SYMS_XSaveLegacy{
  SYMS_U16 fcw;
  SYMS_U16 fsw;
  SYMS_U16 ftw;
  SYMS_U16 fop;
  union{
    struct {
      SYMS_U64 fip;
      SYMS_U64 fdp;
    } b64;
    struct {
      SYMS_U32 fip;
      SYMS_U16 fcs, _pad0;
      SYMS_U32 fdp;
      SYMS_U16 fds, _pad1;
    } b32;
  };
  SYMS_U32 mxcsr;
  SYMS_U32 mxcsr_mask;
  SYMS_U8 st_space[128];
  SYMS_U8 xmm_space[256];
  SYMS_U8 padding[96];
} SYMS_XSaveLegacy;

typedef struct SYMS_XSaveHeader{
  SYMS_U64 xstate_bv;
  SYMS_U64 xcomp_bv;
  SYMS_U8 reserved[48];
} SYMS_XSaveHeader;

// TODO(allen): this one is hacked; ymmh is not gauranteed to be at a fixed location
// and there can be more after that. Requires CPUID to be totally compliant to the standard.
// See intel's manual on the xsave format for more info.
struct SYMS_XSave{
  SYMS_XSaveLegacy legacy;
  SYMS_XSaveHeader header;
  SYMS_U8 ymmh[256];
};

////////////////////////////////
// NOTE(allen): x64 Register Layout Transformers

SYMS_API void syms_x64_regs__set_full_regs_from_xsave_legacy(SYMS_RegX64 *dst, SYMS_XSaveLegacy *src);
SYMS_API void syms_x64_regs__set_xsave_legacy_from_full_regs(SYMS_XSaveLegacy *dst, SYMS_RegX64 *src);
SYMS_API void syms_x64_regs__set_full_regs_from_xsave_avx_extension(SYMS_RegX64 *dst, void *src);
SYMS_API void syms_x64_regs__set_xsave_avx_extension_from_full_regs(void *dst, SYMS_RegX64 *src);

#endif //SYMS_REGS_X64_H
