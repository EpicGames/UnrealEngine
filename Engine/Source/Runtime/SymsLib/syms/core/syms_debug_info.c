// Copyright Epic Games, Inc. All Rights Reserved.

#ifndef SYMS_DEBUG_INFO_C
#define SYMS_DEBUG_INFO_C

////////////////////////////////
//~ NOTE(allen): Generated Code

#include "syms/core/generated/syms_meta_debug_info.c"


////////////////////////////////
//~ NOTE(allen): Symbol Helper Functions

SYMS_API SYMS_B32
syms_ext_match_key_match(SYMS_ExtMatchKey *a, SYMS_ExtMatchKey *b){
  SYMS_B32 result = (syms_memcmp(a, b, sizeof(*a)) == 0);
  return(result);
}

SYMS_API SYMS_USID
syms_make_usid(SYMS_UnitID uid, SYMS_SymbolID sid){
  SYMS_USID result = {uid, sid};
  return(result);
}

SYMS_API void
syms_push_sid_to_list(SYMS_Arena *arena, SYMS_SymbolIDList *list, SYMS_SymbolID id){
  SYMS_SymbolIDNode *node = syms_push_array(arena, SYMS_SymbolIDNode, 1);
  node->id = id;
  SYMS_QueuePush(list->first, list->last, node);
  list->count += 1;
}

SYMS_API SYMS_TypeKind
syms_type_kind_fwd_from_main(SYMS_TypeKind kind){
  SYMS_TypeKind result = SYMS_TypeKind_Null;
  switch (kind){
    case SYMS_TypeKind_Struct: result = SYMS_TypeKind_ForwardStruct; break;
    case SYMS_TypeKind_Class:  result = SYMS_TypeKind_ForwardClass;  break;
    case SYMS_TypeKind_Union:  result = SYMS_TypeKind_ForwardUnion;  break;
    case SYMS_TypeKind_Enum:   result = SYMS_TypeKind_ForwardEnum;   break;
  }
  return(result);
}

SYMS_API SYMS_TypeKind
syms_type_kind_main_from_fwd(SYMS_TypeKind kind){
  SYMS_TypeKind result = SYMS_TypeKind_Null;
  switch (kind){
    case SYMS_TypeKind_ForwardStruct: result = SYMS_TypeKind_Struct; break;
    case SYMS_TypeKind_ForwardClass:  result = SYMS_TypeKind_Class;  break;
    case SYMS_TypeKind_ForwardUnion:  result = SYMS_TypeKind_Union;  break;
    case SYMS_TypeKind_ForwardEnum:   result = SYMS_TypeKind_Enum;   break;
  }
  return(result);
}

SYMS_API SYMS_B32
syms_type_kind_is_basic(SYMS_TypeKind kind){
  SYMS_B32 result = (SYMS_TypeKind_Int8 <= kind && kind <= SYMS_TypeKind_Void);
  return(result);
}

SYMS_API SYMS_B32
syms_type_kind_is_basic_or_enum(SYMS_TypeKind kind){
  SYMS_B32 result = ((SYMS_TypeKind_Int8 <= kind && kind <= SYMS_TypeKind_Void) ||
                     kind == SYMS_TypeKind_Enum || kind == SYMS_TypeKind_ForwardEnum);
  return(result);
}

SYMS_API SYMS_B32
syms_type_kind_is_integer(SYMS_TypeKind kind){
  SYMS_B32 result = (SYMS_TypeKind_Int8 <= kind && kind <= SYMS_TypeKind_Bool);
  return(result);
}

SYMS_API SYMS_B32
syms_type_kind_is_signed(SYMS_TypeKind kind){
  SYMS_B32 result = (SYMS_TypeKind_Int8 <= kind && kind <= SYMS_TypeKind_Int128);
  return(result);
}

SYMS_API SYMS_B32
syms_type_kind_is_complex(SYMS_TypeKind kind){
  SYMS_B32 result = (SYMS_TypeKind_Complex32 <= kind && kind <= SYMS_TypeKind_Complex128);
  return(result);
}

SYMS_API SYMS_B32
syms_type_kind_is_user_defined(SYMS_TypeKind kind){
  SYMS_B32 result = (SYMS_TypeKind_Struct <= kind && kind <= SYMS_TypeKind_ForwardEnum);
  return(result);
}

SYMS_API SYMS_B32
syms_type_kind_is_record(SYMS_TypeKind kind){
  SYMS_B32 result = (SYMS_TypeKind_Struct <= kind && kind <= SYMS_TypeKind_Union);
  return(result);
}

SYMS_API SYMS_B32
syms_type_kind_is_enum(SYMS_TypeKind kind){
  SYMS_B32 result = (kind == SYMS_TypeKind_Enum);
  return(result);
}

SYMS_API SYMS_B32
syms_type_kind_is_forward(SYMS_TypeKind kind){
  SYMS_B32 result = (SYMS_TypeKind_ForwardStruct <= kind && kind <= SYMS_TypeKind_ForwardEnum);
  return(result);
}

SYMS_API SYMS_SymbolIDArray
syms_sid_array_from_list(SYMS_Arena *arena, SYMS_SymbolIDList *list){
  SYMS_SymbolIDArray result = {0};
  result.count = list->count;
  result.ids = syms_push_array(arena, SYMS_SymbolID, result.count);
  SYMS_SymbolID *id_ptr = result.ids;
  SYMS_SymbolID *id_opl = result.ids + result.count;
  for (SYMS_SymbolIDNode *node = list->first;
       node != 0 && id_ptr < id_opl;
       node = node->next, id_ptr += 1){
    *id_ptr = node->id;
  }
  if (id_ptr < id_opl){
    syms_arena_put_back(arena, (id_opl - id_ptr)*sizeof(*id_ptr));
    result.count = (id_ptr - result.ids);
  }
  return(result);
}

#endif // SYMS_DEBUG_INFO_C
