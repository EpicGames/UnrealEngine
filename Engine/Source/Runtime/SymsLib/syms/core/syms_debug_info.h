// Copyright Epic Games, Inc. All Rights Reserved.
/* date = April 6th 2021 10:18 am */

#ifndef SYMS_DEBUG_INFO_H
#define SYMS_DEBUG_INFO_H

////////////////////////////////
//~ allen: Generated Types

#include "syms/core/generated/syms_meta_debug_info.h"

////////////////////////////////
//~ rjf: Binary Info Types

typedef struct SYMS_SecInfo{
  SYMS_String8 name;
  SYMS_U64Range vrange;
  SYMS_U64Range frange;
} SYMS_SecInfo;

typedef struct SYMS_SecInfoArray{
  SYMS_SecInfo *sec_info;
  SYMS_U64 count;
} SYMS_SecInfoArray;

typedef struct SYMS_BinInfo{
  SYMS_Arch arch;
  SYMS_U64Range range;
} SYMS_BinInfo;

typedef struct SYMS_BinInfoArray{
  SYMS_BinInfo *bin_info;
  SYMS_U64 count;
} SYMS_BinInfoArray;

////////////////////////////////
//~ allen: Binary Imports & Exports

typedef struct SYMS_Import{
  SYMS_String8 name;
  SYMS_String8 library_name;
  SYMS_U16     ordinal;
} SYMS_Import;

typedef struct SYMS_Export{
  SYMS_String8 name;
  SYMS_U64 address;
  SYMS_U32 ordinal;
  SYMS_U32 unused_;
  SYMS_String8 forwarder_library_name;
  SYMS_String8 forwarder_import_name;
} SYMS_Export;

typedef struct SYMS_ImportNode{
  struct SYMS_ImportNode *next;
  SYMS_Import data;
} SYMS_ImportNode;

typedef struct SYMS_ExportNode{
  struct SYMS_ExportNode *next;
  SYMS_Export data;
} SYMS_ExportNode;

typedef struct SYMS_ImportArray{
  SYMS_Import *imports;
  SYMS_U64 count;
} SYMS_ImportArray;

typedef struct SYMS_ExportArray{
  SYMS_Export *exports;
  SYMS_U64 count;
} SYMS_ExportArray;

////////////////////////////////
//~ allen: Compilation Unit Types

typedef SYMS_U64 SYMS_UnitID;

typedef struct SYMS_UnitInfo{
  SYMS_UnitFeatures features;
  SYMS_Language language;
} SYMS_UnitInfo;

typedef struct SYMS_UnitNames{
  SYMS_String8 source_file;
  SYMS_String8 object_file;
  SYMS_String8 archive_file;
  SYMS_String8 compiler;
  SYMS_String8 compile_dir;
} SYMS_UnitNames;

typedef struct SYMS_UnitRange{
  SYMS_U64Range vrange;
  SYMS_UnitID uid;
} SYMS_UnitRange;

typedef struct SYMS_UnitRangeArray{
  SYMS_UnitRange *ranges;
  SYMS_U64 count;
} SYMS_UnitRangeArray;

////////////////////////////////
//~ allen: External File Types

typedef struct SYMS_ExtMatchKey{
  SYMS_U8 v[16];
} SYMS_ExtMatchKey;

typedef struct SYMS_ExtFile{
  SYMS_String8 file_name;
  SYMS_ExtMatchKey match_key;
} SYMS_ExtFile;

typedef struct SYMS_ExtFileNode{
  struct SYMS_ExtFileNode *next;
  SYMS_ExtFile ext_file;
} SYMS_ExtFileNode;

typedef struct SYMS_ExtFileList{
  SYMS_ExtFileNode *first;
  SYMS_ExtFileNode *last;
  SYMS_U64 node_count;
} SYMS_ExtFileList;

////////////////////////////////
//~ allen: Line Info Types

typedef SYMS_U64 SYMS_FileID;

typedef struct SYMS_SrcCoord{
  SYMS_FileID file_id;
  SYMS_U32 line;
  SYMS_U32 col;
} SYMS_SrcCoord;

typedef struct SYMS_Line{
  SYMS_SrcCoord src_coord;
  SYMS_U64 voff;
} SYMS_Line;

typedef struct SYMS_ResolvedLine{
  SYMS_String8 file_name;
  SYMS_U32 line;
  SYMS_U32 col;
  SYMS_U64 voff;
} SYMS_ResolvedLine;

typedef struct SYMS_FileIDArray{
  SYMS_FileID *ids;
  SYMS_U64 count;
} SYMS_FileIDArray;

typedef struct SYMS_LineTable{
  // sequence_index_array ranges from [0,sequence_count] inclusive so that:
  // for-all i in [0,sequence_count):
  //   (sequence_index_array[i + 1] - sequence_index_array[i]) == # of lines in sequence i
  SYMS_U64 *sequence_index_array;
  SYMS_U64 sequence_count;
  
  SYMS_Line *line_array;
  SYMS_U64 line_count;
} SYMS_LineTable;

typedef struct SYMS_LineParseOut{
  SYMS_LineTable line_table;
  SYMS_String8Array file_table;
  SYMS_FileIDArray file_id_array;
} SYMS_LineParseOut;

////////////////////////////////
//~ allen: Type Info Types

// TODO(allen): go to the mc file
// TODO(allen): just yanked right from PDB
// more interpretation should be applied to these name choices
typedef enum SYMS_CallConvention{
  SYMS_CallConvention_NULL,
  SYMS_CallConvention_NEAR_C,
  SYMS_CallConvention_FAR_C,
  SYMS_CallConvention_NEAR_PASCAL,
  SYMS_CallConvention_FAR_PASCAL,
  SYMS_CallConvention_NEAR_FAST,
  SYMS_CallConvention_FAR_FAST,
  SYMS_CallConvention_NEAR_STD,
  SYMS_CallConvention_FAR_STD,
  SYMS_CallConvention_NEAR_SYS,
  SYMS_CallConvention_FAR_SYS,
  SYMS_CallConvention_THISCALL,
  SYMS_CallConvention_MIPSCALL,
  SYMS_CallConvention_GENERIC,
  SYMS_CallConvention_ALPHACALL,
  SYMS_CallConvention_PPCCALL,
  SYMS_CallConvention_SHCALL,
  SYMS_CallConvention_ARMCALL,
  SYMS_CallConvention_AM33CALL,
  SYMS_CallConvention_TRICALL,
  SYMS_CallConvention_SH5CALL,
  SYMS_CallConvention_M32RCALL,
  SYMS_CallConvention_CLRCALL,
  SYMS_CallConvention_INLINE,
  SYMS_CallConvention_NEAR_VECTOR,
  SYMS_CallConvention_COUNT
} SYMS_CallConvention;

typedef enum SYMS_SizeInterpretation{
  SYMS_SizeInterpretation_Null,
  SYMS_SizeInterpretation_ByteCount,
  SYMS_SizeInterpretation_Multiplier,
  SYMS_SizeInterpretation_ResolveForwardReference,
} SYMS_SizeInterpretation;

////////////////////////////////
//~ allen: Symbol API Types

typedef SYMS_U64 SYMS_SymbolID;

typedef struct SYMS_SymbolIDNode{
  struct SYMS_SymbolIDNode *next;
  SYMS_SymbolID id;
} SYMS_SymbolIDNode;

typedef struct SYMS_SymbolIDList{
  SYMS_SymbolIDNode *first;
  SYMS_SymbolIDNode *last;
  SYMS_U64 count;
} SYMS_SymbolIDList;

typedef struct SYMS_SymbolIDArray{
  SYMS_SymbolID *ids;
  SYMS_U64 count;
} SYMS_SymbolIDArray;

// unit and symbol ID
typedef struct SYMS_USID{
  SYMS_UnitID uid;
  SYMS_SymbolID sid;
} SYMS_USID;

typedef struct SYMS_USIDNode{
  struct SYMS_USIDNode *next;
  SYMS_USID usid;
} SYMS_USIDNode;

typedef struct SYMS_USIDList{
  SYMS_USIDNode *first;
  SYMS_USIDNode *last;
  SYMS_U64 count;
} SYMS_USIDList;

typedef struct SYMS_USIDArray{
  SYMS_USID *usid;
  SYMS_U64 count;
} SYMS_USIDArray;

typedef struct SYMS_SigHandle{
  SYMS_U64 v;
} SYMS_SigHandle;

typedef struct SYMS_UnitIDAndSig{
  SYMS_UnitID uid;
  SYMS_SigHandle sig;
} SYMS_UnitIDAndSig;

typedef struct SYMS_TypeInfo{
  SYMS_TypeKind kind;
  
  // 'mods' gives modifier information applied to this type after inferring the
  // rest of it's structure. When (kind == SYMS_TypeKind_Modifier) this type symbol
  // only exists to apply modifiers to another type symbol, but in certain cases a
  // non-modifier type symbol contains it's own modifiers.
  SYMS_TypeModifiers mods;
  
  // 'reported_size_interp', & 'reported_size' together describe the size of the type.
  SYMS_SizeInterpretation reported_size_interp;
  SYMS_U64 reported_size;
  
  // filled for any type that has definition location information.
  SYMS_SrcCoord src_coord;
  
  // 'direct_type' meaning depends on kind.
  //  SYMS_TypeKind_Bitfield   -> the underlying type of the bitfield
  //  SYMS_TypeKind_Enum       -> the underlying type of the enum
  //  SYMS_TypeKind_Typedef    -> the underlying type of the typedef
  //  SYMS_TypeKind_Modifier   -> the modified type
  //  SYMS_TypeKind_Ptr        -> the type pointed to
  //  SYMS_TypeKind_*Reference -> the type referenced
  //  SYMS_TypeKind_MemberPtr  -> the type pointed to
  //  SYMS_TypeKind_Array      -> the type of an element in the array
  //  SYMS_TypeKind_Proc       -> the return type of the procedure
  SYMS_USID direct_type;
  
  union{
    // kind == SYMS_TypeKind_Proc
    SYMS_CallConvention call_convention;
    // kind == SYMS_TypeKind_MemberPtr
    SYMS_USID containing_type;
  };
} SYMS_TypeInfo;

typedef struct SYMS_ConstInfo{
  // TODO(allen): look at more complex constants
  SYMS_TypeKind kind;
  SYMS_U64 val;
} SYMS_ConstInfo;

typedef enum SYMS_MemKind{
  SYMS_MemKind_Null,
  SYMS_MemKind_DataField,
  SYMS_MemKind_StaticData,
  SYMS_MemKind_Method,
  SYMS_MemKind_StaticMethod,
  SYMS_MemKind_VTablePtr,
  SYMS_MemKind_BaseClass,
  SYMS_MemKind_VBaseClassPtr,
  SYMS_MemKind_NestedType,
} SYMS_MemKind;

typedef SYMS_U32 SYMS_MemFlags;
enum{
  SYMS_MemFlag_Virtual     = (1 << 0),
  SYMS_MemFlag_Constructor = (1 << 1),
  SYMS_MemFlag_Destructor  = (1 << 2),
};

typedef struct SYMS_MemInfo{
  SYMS_MemKind kind;
  SYMS_MemVisibility visibility;
  SYMS_MemFlags flags;
  SYMS_String8 name;
  SYMS_U32 off;
  SYMS_U32 virtual_off;
} SYMS_MemInfo;

typedef struct SYMS_SigInfo{
  SYMS_UnitID uid;
  SYMS_SymbolIDArray param_type_ids;
  SYMS_SymbolID return_type_id;
  SYMS_SymbolID this_type_id;
} SYMS_SigInfo;

typedef struct SYMS_EnumMember{
  SYMS_String8 name;
  SYMS_U64 val;
} SYMS_EnumMember;

typedef struct SYMS_EnumMemberArray{
  SYMS_EnumMember *enum_members;
  SYMS_U64 count;
} SYMS_EnumMemberArray;

////////////////////////////////
//~ allen: Location Types

typedef SYMS_U64 SYMS_LocID;

typedef struct SYMS_LocRange{
  SYMS_U64Range vrange;
  SYMS_LocID loc_id;
} SYMS_LocRange;

typedef struct SYMS_LocRangeArray{
  SYMS_LocRange *loc_ranges;
  SYMS_U64 count;
} SYMS_LocRangeArray;

typedef struct SYMS_LocRangeNode{
  struct SYMS_LocRangeNode *next;
  SYMS_LocRange loc_range;
} SYMS_LocRangeNode;

typedef struct SYMS_LocRangeList{
  SYMS_LocRangeNode *first;
  SYMS_LocRangeNode *last;
  SYMS_U64 count;
} SYMS_LocRangeList;

typedef enum SYMS_ProcLoc{
  SYMS_ProcLoc_FrameBase,
  SYMS_ProcLoc_ReturnAddress,
} SYMS_ProcLoc;

////////////////////////////////
//~ allen: Link Name Types

typedef struct SYMS_LinkNameRec{
  SYMS_String8 name;
  SYMS_U64 voff;
} SYMS_LinkNameRec;

typedef struct SYMS_LinkNameRecArray{
  SYMS_LinkNameRec *recs;
  SYMS_U64 count;
} SYMS_LinkNameRecArray;

////////////////////////////////
//~ allen: Nil For Accelerators

SYMS_READ_ONLY SYMS_GLOBAL SYMS_FileFormat syms_format_nil = SYMS_FileFormat_Null;

////////////////////////////////
//~ allen: Symbol Helper Functions

SYMS_C_LINKAGE_BEGIN

SYMS_API SYMS_B32     syms_ext_match_key_match(SYMS_ExtMatchKey *a, SYMS_ExtMatchKey *b);

SYMS_API SYMS_USID    syms_make_usid(SYMS_UnitID uid, SYMS_SymbolID sid);

SYMS_API SYMS_TypeKind syms_type_kind_fwd_from_main(SYMS_TypeKind type_kind);
SYMS_API SYMS_TypeKind syms_type_kind_main_from_fwd(SYMS_TypeKind type_kind);
SYMS_API SYMS_B32      syms_type_kind_is_basic(SYMS_TypeKind kind);
SYMS_API SYMS_B32      syms_type_kind_is_basic_or_enum(SYMS_TypeKind kind);
SYMS_API SYMS_B32      syms_type_kind_is_integer(SYMS_TypeKind kind);
SYMS_API SYMS_B32      syms_type_kind_is_signed(SYMS_TypeKind kind);
SYMS_API SYMS_B32      syms_type_kind_is_complex(SYMS_TypeKind kind);
SYMS_API SYMS_B32      syms_type_kind_is_user_defined(SYMS_TypeKind kind);
SYMS_API SYMS_B32      syms_type_kind_is_record(SYMS_TypeKind kind);
SYMS_API SYMS_B32      syms_type_kind_is_enum(SYMS_TypeKind kind);
SYMS_API SYMS_B32      syms_type_kind_is_forward(SYMS_TypeKind kind);

SYMS_API SYMS_SymbolIDArray syms_sid_array_from_list(SYMS_Arena *arena, SYMS_SymbolIDList *list);

SYMS_C_LINKAGE_END

#endif // SYMS_DEBUG_INFO_H
