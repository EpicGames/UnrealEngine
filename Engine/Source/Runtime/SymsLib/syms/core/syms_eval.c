// Copyright Epic Games, Inc. All Rights Reserved.

#ifndef SYMS_EVAL_C
#define SYMS_EVAL_C

////////////////////////////////
//~ allen: Eval Meta Code

#include "syms/core/generated/syms_meta_eval.c"

// TODO(allen): this should be part of the optional extended data tables stuff
#include "syms/core/generated/syms_meta_eval_ext.c"

////////////////////////////////
//~ allen: Eval Bytecode Helper Functions

SYMS_API SYMS_EvalOpParams
syms_eval_op_params(SYMS_U64 p){
  SYMS_EvalOpParams result = {0};
  result.u64[0] = p;
  return(result);
}

SYMS_API SYMS_EvalOpParams
syms_eval_op_params_2u8(SYMS_U8 p1, SYMS_U8 p2){
  SYMS_EvalOpParams result = {0};
  result.u8[0] = p1;
  result.u8[1] = p2;
  return(result);
}

SYMS_API SYMS_EvalOpParams
syms_eval_op_params_2u16(SYMS_U16 p1, SYMS_U16 p2){
  SYMS_EvalOpParams result = {0};
  result.u16[0] = p1;
  result.u16[1] = p2;
  return(result);
}

SYMS_API void
syms_eval_op_push(SYMS_Arena *arena, SYMS_EvalOpList *list, SYMS_EvalOp op, SYMS_EvalOpParams params){
  SYMS_ASSERT(op < SYMS_EvalOp_COUNT);
  
  //- setup new op
  SYMS_EvalOpNode *loose_op = syms_push_array(arena, SYMS_EvalOpNode, 1);
  loose_op->op = op;
  loose_op->params = params;
  
  //- main op push
  // determine encoded byte count
  SYMS_U8 ctrlbits = syms_eval_opcode_ctrlbits[op];
  SYMS_U64 extra_byte_count = (ctrlbits >> SYMS_EvalOpCtrlBits_DecodeShft)&SYMS_EvalOpCtrlBits_DecodeMask;
  
  // push onto list
  SYMS_QueuePush(list->first, list->last, loose_op);
  SYMS_U64 encoded_byte_count = 1 + extra_byte_count;
  list->byte_count += encoded_byte_count;
}

SYMS_API void
syms_eval_op_push_bytecode(SYMS_Arena *arena, SYMS_EvalOpList *list, SYMS_String8 bytecode){
  //- setup new op
  SYMS_EvalOpNode *loose_op = syms_push_array(arena, SYMS_EvalOpNode, 1);
  loose_op->op = SYMS_EvalIRExtKind_Bytecode;
  loose_op->params.data = bytecode;
  
  // push onto list
  SYMS_QueuePush(list->first, list->last, loose_op);
  list->byte_count += bytecode.size;
}

SYMS_API void
syms_eval_op_list_concat_in_place(SYMS_EvalOpList *left_dst, SYMS_EvalOpList *right_destroyed){
  left_dst->byte_count += right_destroyed->byte_count;
  if (left_dst->last != 0){
    if (right_destroyed->last != 0){
      left_dst->last->next = right_destroyed->first;
      left_dst->last = right_destroyed->last;
    }
  }
  else{
    left_dst->first = right_destroyed->first;
    left_dst->last = right_destroyed->last;
  }
  syms_memzero_struct(right_destroyed);
}

SYMS_API void
syms_eval_op_encode_u(SYMS_Arena *arena, SYMS_EvalOpList *list, SYMS_U64 u){
  SYMS_EvalOp op = SYMS_EvalOp_ConstU64;
  if (u <= SYMS_U8_MAX){
    op = SYMS_EvalOp_ConstU8;
  }
  else if (u <= SYMS_U16_MAX){
    op = SYMS_EvalOp_ConstU16;
  }
  else if (u <= SYMS_U32_MAX){
    op = SYMS_EvalOp_ConstU32;
  }
  syms_eval_op_push(arena, list, op, syms_eval_op_params(u));
}

SYMS_API void
syms_eval_op_encode_s(SYMS_Arena *arena, SYMS_EvalOpList *list, SYMS_S64 s){
  SYMS_U64 size = 64;
  SYMS_EvalOp op = SYMS_EvalOp_ConstU64;
  if (SYMS_S8_MIN <= s && s <= SYMS_S8_MAX){
    op = SYMS_EvalOp_ConstU8;
    size = 8;
  }
  else if (SYMS_S16_MIN <= s && s <= SYMS_S16_MAX){
    op = SYMS_EvalOp_ConstU16;
    size = 16;
  }
  else if (SYMS_S32_MIN <= s && s <= SYMS_S32_MAX){
    op = SYMS_EvalOp_ConstU32;
    size = 32;
  }
  syms_eval_op_push(arena, list, op, syms_eval_op_params((SYMS_U64)s));
  if (size < 64){
    syms_eval_op_push(arena, list, SYMS_EvalOp_TruncSigned, syms_eval_op_params(size));
  }
}

SYMS_API void
syms_eval_op_encode_reg_section(SYMS_Arena *arena, SYMS_EvalOpList *list, SYMS_RegSection sec){
  SYMS_U64 u64 = sec.off | sec.size << 16;
  syms_eval_op_encode_u(arena, list, u64);
}

SYMS_API void
syms_eval_op_encode_reg(SYMS_Arena *arena, SYMS_EvalOpList *list, SYMS_Arch arch, SYMS_RegID reg_id){
  SYMS_RegSection sec = syms_reg_section_from_reg_id(arch, reg_id);
  syms_eval_op_encode_reg_section(arena, list, sec);
}

//- bytecode analysis functions

SYMS_API SYMS_B32
syms_eval_opcode_type_compatible(SYMS_EvalOp op, SYMS_EvalTypeGroup group){
  SYMS_B32 result = syms_false;
  switch (op){
    case SYMS_EvalOp_Neg:case SYMS_EvalOp_Add:case SYMS_EvalOp_Sub:
    case SYMS_EvalOp_Mul:case SYMS_EvalOp_Div:
    case SYMS_EvalOp_EqEq:case SYMS_EvalOp_NtEq:
    case SYMS_EvalOp_LsEq:case SYMS_EvalOp_GrEq:case SYMS_EvalOp_Less:case SYMS_EvalOp_Grtr:
    {
      if (group != SYMS_EvalTypeGroup_Other){
        result = syms_true;
      }
    }break;
    case SYMS_EvalOp_Mod:case SYMS_EvalOp_LShift:case SYMS_EvalOp_RShift:
    case SYMS_EvalOp_BitNot:case SYMS_EvalOp_BitAnd:case SYMS_EvalOp_BitXor:case SYMS_EvalOp_BitOr:
    case SYMS_EvalOp_LogNot:case SYMS_EvalOp_LogAnd:case SYMS_EvalOp_LogOr: 
    {
      if (group == SYMS_EvalTypeGroup_S || group == SYMS_EvalTypeGroup_U){
        result = syms_true;
      }
    }break;
  }
  return(result);
}

//- bytecode encoder

SYMS_API SYMS_String8
syms_eval_bytecode_from_op_list(SYMS_Arena *arena, SYMS_EvalOpList *list){
  // allocate output
  SYMS_String8 result = {0};
  result.size = list->byte_count;
  result.str = syms_push_array(arena, SYMS_U8, result.size);
  
  // iterate loose op nodes
  SYMS_U8 *ptr = result.str;
  SYMS_U8 *opl = result.str + result.size;
  for (SYMS_EvalOpNode *node = list->first;
       node != 0;
       node = node->next){
    SYMS_U32 op = node->op;
    
    switch (op){
      default:
      {
        // compute bytecode advance
        SYMS_U8 ctrlbits = syms_eval_opcode_ctrlbits[op];
        SYMS_U64 extra_byte_count = (ctrlbits >> SYMS_EvalOpCtrlBits_DecodeShft)&SYMS_EvalOpCtrlBits_DecodeMask;
        
        SYMS_U8 *next_ptr = ptr + 1 + extra_byte_count;
        SYMS_ASSERT(next_ptr <= opl);
        
        // fill bytecode
        ptr[0] = op;
        syms_memmove(ptr + 1, &node->params, extra_byte_count);
        
        // advance output pointer
        ptr = next_ptr;
      }break;
      
      case SYMS_EvalIRExtKind_Bytecode:
      {
        // compute bytecode advance
        SYMS_U64 size = node->params.data.size;
        SYMS_U8 *next_ptr = ptr + size;
        SYMS_ASSERT(next_ptr <= opl);
        
        // fill bytecode
        syms_memmove(ptr, node->params.data.str, size);
        
        // advance output pointer
        ptr = next_ptr;
      }break;
    }
  }
  
  return(result);
}

//- ir tree helpers functions

SYMS_API SYMS_EvalIRTree*
syms_eval_ir_tree_const_u(SYMS_Arena *arena, SYMS_U64 v){
  // choose encoding op
  SYMS_EvalOp op = SYMS_EvalOp_ConstU64;
  if (v < 0x100){
    op = SYMS_EvalOp_ConstU8;
  }
  else if (v < 0x10000){
    op = SYMS_EvalOp_ConstU16;
  }
  else if (v < 0x100000000){
    op = SYMS_EvalOp_ConstU32;
  }
  
  // make the tree node
  SYMS_EvalIRTree *result = syms_push_array_zero(arena, SYMS_EvalIRTree, 1);
  result->op = op;
  result->params = syms_eval_op_params(v);
  return(result);
}

SYMS_API SYMS_EvalIRTree*
syms_eval_ir_tree_unary_op(SYMS_Arena *arena, SYMS_EvalOp op, SYMS_EvalTypeGroup group, SYMS_EvalIRTree *c){
  SYMS_EvalIRTree *result = syms_push_array_zero(arena, SYMS_EvalIRTree, 1);
  result->op = op;
  result->params = syms_eval_op_params(group);
  result->children[0] = c;
  return(result);
}

SYMS_API SYMS_EvalIRTree*
syms_eval_ir_tree_binary_op(SYMS_Arena *arena, SYMS_EvalOp op, SYMS_EvalTypeGroup group,
                            SYMS_EvalIRTree *l, SYMS_EvalIRTree *r){
  SYMS_EvalIRTree *result = syms_push_array_zero(arena, SYMS_EvalIRTree, 1);
  result->op = op;
  result->params = syms_eval_op_params(group);
  result->children[0] = l;
  result->children[1] = r;
  return(result);
}

SYMS_API SYMS_EvalIRTree*
syms_eval_ir_tree_binary_op_u(SYMS_Arena *arena, SYMS_EvalOp op, SYMS_EvalIRTree *l, SYMS_EvalIRTree *r){
  SYMS_EvalIRTree *result = syms_eval_ir_tree_binary_op(arena, op, SYMS_EvalTypeGroup_U, l, r);
  return(result);
}

SYMS_API SYMS_EvalIRTree*
syms_eval_ir_tree_conditional(SYMS_Arena *arena, SYMS_EvalIRTree *c, SYMS_EvalIRTree *l, SYMS_EvalIRTree *r){
  SYMS_EvalIRTree *result = syms_push_array_zero(arena, SYMS_EvalIRTree, 1);
  result->op = SYMS_EvalOp_Cond;
  result->children[0] = c;
  result->children[1] = l;
  result->children[2] = r;
  return(result);
}

SYMS_API SYMS_EvalIRTree*
syms_eval_ir_tree_bytecode_no_copy(SYMS_Arena *arena, SYMS_String8 bytecode){
  SYMS_EvalIRTree *result = syms_push_array_zero(arena, SYMS_EvalIRTree, 1);
  result->op = SYMS_EvalIRExtKind_Bytecode;
  result->params.data = bytecode;
  return(result);
}

#endif //SYMS_EVAL_C

