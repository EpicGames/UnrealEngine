// Copyright Epic Games, Inc. All Rights Reserved.
/* date = December 8th 2021 11:42 am */

#ifndef SYMS_EVAL_H
#define SYMS_EVAL_H

////////////////////////////////
//~ allen: Syms Eval Meta Code

#include "syms/core/generated/syms_meta_eval.h"

////////////////////////////////
//~ allen: Syms Eval Bytecode Helper Types

typedef enum SYMS_EvalMode{
  SYMS_EvalMode_Nil,
  SYMS_EvalMode_Value,
  SYMS_EvalMode_Address,
  SYMS_EvalMode_Register,
} SYMS_EvalMode;

enum{
  // apears in IR tree
  SYMS_EvalIRExtKind_Bytecode = SYMS_EvalOp_COUNT,
  
  // apears in IR graph
  SYMS_EvalIRExtKind_Noop,
  
  SYMS_EvalIRExtKind_COUNT,
};

typedef union SYMS_EvalOpParams{
  SYMS_U8  u8[8];
  SYMS_U16 u16[4];
  SYMS_U32 u32[2];
  SYMS_U64 u64[1];
  SYMS_String8 data;
} SYMS_EvalOpParams;

typedef struct SYMS_EvalIRTree{
  SYMS_U32 op;
  union{
    struct{
      struct SYMS_EvalIRTree *children[3];
      SYMS_EvalOpParams params;
    };
  };
} SYMS_EvalIRTree;

typedef struct SYMS_EvalOpNode{
  struct SYMS_EvalOpNode *next;
  SYMS_U8 op;
  SYMS_EvalOpParams params;
} SYMS_EvalOpNode;

typedef struct SYMS_EvalOpList{
  SYMS_EvalOpNode *first;
  SYMS_EvalOpNode *last;
  SYMS_U64 byte_count;
} SYMS_EvalOpList;

typedef struct SYMS_Location{
  SYMS_EvalOpList op_list;
  SYMS_EvalMode mode;
  SYMS_B32 is_parameter;
} SYMS_Location;

////////////////////////////////
//~ allen: Syms Eval Nil Values

SYMS_READ_ONLY SYMS_GLOBAL SYMS_EvalIRTree syms_eval_ir_tree_nil = {0};

////////////////////////////////
//~ allen: Syms Eval Functions

//- bytecode helper functions

SYMS_API SYMS_EvalOpParams syms_eval_op_params(SYMS_U64 p);
SYMS_API SYMS_EvalOpParams syms_eval_op_params_2u8(SYMS_U8 p1, SYMS_U8 p2);
SYMS_API SYMS_EvalOpParams syms_eval_op_params_2u16(SYMS_U16 p1, SYMS_U16 p2);

SYMS_API void syms_eval_op_push(SYMS_Arena *arena, SYMS_EvalOpList *list, SYMS_EvalOp op,
                                SYMS_EvalOpParams params);
SYMS_API void syms_eval_op_push_bytecode(SYMS_Arena *arena, SYMS_EvalOpList *list, SYMS_String8 bytecode);
SYMS_API void syms_eval_op_list_concat_in_place(SYMS_EvalOpList *left_dst, SYMS_EvalOpList *right_destroyed);

SYMS_API void syms_eval_op_encode_u(SYMS_Arena *arena, SYMS_EvalOpList *list, SYMS_U64 u);
SYMS_API void syms_eval_op_encode_s(SYMS_Arena *arena, SYMS_EvalOpList *list, SYMS_S64 s);

SYMS_API void syms_eval_op_encode_reg_section(SYMS_Arena *arena, SYMS_EvalOpList *list, SYMS_RegSection sec);
SYMS_API void syms_eval_op_encode_reg(SYMS_Arena *arena, SYMS_EvalOpList *list, SYMS_Arch arch,
                                      SYMS_RegID reg_id);

//- bytecode analysis functions

SYMS_API SYMS_B32 syms_eval_opcode_type_compatible(SYMS_EvalOp op, SYMS_EvalTypeGroup group);

//- bytecode encoder

SYMS_API SYMS_String8 syms_eval_bytecode_from_op_list(SYMS_Arena *arena, SYMS_EvalOpList *list);

//- ir tree helpers functions

SYMS_API SYMS_EvalIRTree* syms_eval_ir_tree_const_u(SYMS_Arena *arena, SYMS_U64 v);
SYMS_API SYMS_EvalIRTree* syms_eval_ir_tree_unary_op(SYMS_Arena *arena, SYMS_EvalOp op,
                                                     SYMS_EvalTypeGroup group, SYMS_EvalIRTree *c);
SYMS_API SYMS_EvalIRTree* syms_eval_ir_tree_binary_op(SYMS_Arena *arena, SYMS_EvalOp op,
                                                      SYMS_EvalTypeGroup group,
                                                      SYMS_EvalIRTree *l, SYMS_EvalIRTree *r);
SYMS_API SYMS_EvalIRTree* syms_eval_ir_tree_binary_op_u(SYMS_Arena *arena, SYMS_EvalOp op, SYMS_EvalIRTree *l,
                                                        SYMS_EvalIRTree *r);
SYMS_API SYMS_EvalIRTree* syms_eval_ir_tree_conditional(SYMS_Arena *arena, SYMS_EvalIRTree *c,
                                                        SYMS_EvalIRTree *l, SYMS_EvalIRTree *r);
SYMS_API SYMS_EvalIRTree* syms_eval_ir_tree_bytecode_no_copy(SYMS_Arena *arena, SYMS_String8 bytecode);

#endif //SYMS_EVAL_H
