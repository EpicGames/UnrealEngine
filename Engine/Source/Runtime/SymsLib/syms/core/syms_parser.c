// Copyright Epic Games, Inc. All Rights Reserved.

#ifndef SYMS_PARSER_C
#define SYMS_PARSER_C

////////////////////////////////
//~ NOTE(allen): General File Analysis

SYMS_API SYMS_FileAccel*
syms_file_accel_from_data(SYMS_Arena *arena, SYMS_String8 data){
  SYMS_FileAccel *result = (SYMS_FileAccel*)&syms_format_nil;
  
  // easy to recognize
  if (result->format == SYMS_FileFormat_Null){
    result = (SYMS_FileAccel*)syms_pe_file_accel_from_data(arena, data);
  }
  if (result->format == SYMS_FileFormat_Null){
    result = (SYMS_FileAccel*)syms_pdb_file_accel_from_data(arena, data);
  }
  if (result->format == SYMS_FileFormat_Null){
    result = (SYMS_FileAccel*)syms_elf_file_accel_from_data(arena, data);
  }
  if (result->format == SYMS_FileFormat_Null){
    result = (SYMS_FileAccel*)syms_mach_file_accel_from_data(arena, data);
  }
  
  SYMS_ASSERT_PARANOID(syms_parser_api_invariants());
  return(result);
}

SYMS_API SYMS_FileFormat
syms_file_format_from_file(SYMS_FileAccel *accel){
  SYMS_FileFormat result = accel->format;
  return(result);
}

////////////////////////////////
//~ NOTE(allen): Bin File

SYMS_API SYMS_B32
syms_file_is_bin(SYMS_FileAccel *file){
  SYMS_B32 result = syms_false;
  switch (file->format){
    case SYMS_FileFormat_PE:
    case SYMS_FileFormat_ELF:
    {
      result = syms_true;
    }break;
    
    case SYMS_FileFormat_MACH:
    {
      result = syms_mach_file_is_bin(&file->mach_accel);
    }break;
  }
  SYMS_ASSERT_PARANOID(syms_parser_api_invariants());
  return(result);
}

SYMS_API SYMS_BinAccel*
syms_bin_accel_from_file(SYMS_Arena *arena, SYMS_String8 data, SYMS_FileAccel *file){
  SYMS_ProfBegin("syms_bin_accel_from_file");
  SYMS_BinAccel *result = (SYMS_BinAccel*)&syms_format_nil;
  switch (file->format){
    case SYMS_FileFormat_PE:
    {
      result = (SYMS_BinAccel*)syms_pe_bin_accel_from_file(arena, data, &file->pe_accel);
    }break;
    
    case SYMS_FileFormat_ELF:
    {
      result = (SYMS_BinAccel*)syms_elf_bin_accel_from_file(arena, data, &file->elf_accel);
    }break;
    
    case SYMS_FileFormat_MACH:
    {
      result = (SYMS_BinAccel*)syms_mach_bin_accel_from_file(arena, data, &file->mach_accel);
    }break;
  }
  SYMS_ASSERT_PARANOID(syms_parser_api_invariants());
  SYMS_ProfEnd();
  return(result);
}

// arch
SYMS_API SYMS_Arch
syms_arch_from_bin(SYMS_BinAccel *bin){
  SYMS_ProfBegin("syms_arch_from_bin");
  SYMS_Arch result = SYMS_Arch_Null;
  switch (bin->format){
    case SYMS_FileFormat_PE:
    {
      result = syms_pe_arch_from_bin(&bin->pe_accel);
    }break;
    
    case SYMS_FileFormat_ELF:
    {
      result = syms_elf_arch_from_bin(&bin->elf_accel);
    }break;
    
    case SYMS_FileFormat_MACH:
    {
      result = syms_mach_arch_from_bin(&bin->mach_accel);
    }break;
  }
  SYMS_ASSERT_PARANOID(syms_parser_api_invariants());
  SYMS_ProfEnd();
  return(result);
}

// external info
SYMS_API SYMS_ExtFileList
syms_ext_file_list_from_bin(SYMS_Arena *arena, SYMS_String8 data, SYMS_BinAccel *accel){
  SYMS_ProfBegin("syms_ext_file_list_from_bin");
  SYMS_ExtFileList result = {0};
  switch (accel->format){
    case SYMS_FileFormat_PE:
    {
      result = syms_pe_ext_file_list_from_bin(arena, data, &accel->pe_accel);
    }break;
    
    case SYMS_FileFormat_ELF:
    {
      result = syms_elf_ext_file_list_from_bin(arena, data, &accel->elf_accel);
    }break;
  }
  SYMS_ASSERT_PARANOID(syms_parser_api_invariants());
  SYMS_ProfEnd();
  return(result);
}

// binary sections
SYMS_API SYMS_SecInfoArray
syms_sec_info_array_from_bin(SYMS_Arena *arena, SYMS_String8 data, SYMS_BinAccel *bin){
  SYMS_ProfBegin("syms_sec_info_array_from_bin");
  SYMS_SecInfoArray result = {0};
  switch (bin->format){
    case SYMS_FileFormat_PE:
    {
      result = syms_pe_sec_info_array_from_bin(arena, data, &bin->pe_accel);
    }break;
    
    case SYMS_FileFormat_ELF:
    {
      result = syms_elf_sec_info_array_from_bin(arena, data, &bin->elf_accel);
    }break;
    
    case SYMS_FileFormat_MACH:
    {
      result = syms_mach_sec_info_array_from_bin(arena, data, &bin->mach_accel);
    }break;
  }
  SYMS_ASSERT_PARANOID(syms_parser_api_invariants());
  SYMS_ProfEnd();
  return(result);
}


// default vbase
SYMS_API SYMS_U64
syms_default_vbase_from_bin(SYMS_BinAccel *bin){
  SYMS_ProfBegin("syms_default_vbase_from_bin");
  SYMS_U64 result = 0;
  switch (bin->format){
    case SYMS_FileFormat_PE:
    {
      // this always has value 0
    }break;
    
    case SYMS_FileFormat_ELF:
    {
      result = syms_elf_default_vbase_from_bin(&bin->elf_accel);
    }break;
    
    case SYMS_FileFormat_MACH:
    {
      // TODO(allen): ?
    }break;
  }
  SYMS_ASSERT_PARANOID(syms_parser_api_invariants());
  SYMS_ProfEnd();
  return(result);
}

// entry point
SYMS_API SYMS_U64
syms_entry_point_voff_from_bin(SYMS_BinAccel *bin){
  SYMS_ProfBegin("syms_entry_point_voff_from_bin");
  SYMS_U64 result = 0;
  switch (bin->format){
    case SYMS_FileFormat_PE:
    {
      result = syms_pe_entry_point_voff_from_bin(&bin->pe_accel);
    }break;
    
    case SYMS_FileFormat_ELF:
    {
      result = syms_elf_entry_point_voff_from_bin(&bin->elf_accel);
    }break;
    
    case SYMS_FileFormat_MACH:
    {
      // TODO(allen): ?
    }break;
  }
  SYMS_ASSERT_PARANOID(syms_parser_api_invariants());
  SYMS_ProfEnd();
  return(result);
}

// imports & exports
SYMS_API SYMS_ImportArray
syms_imports_from_bin(SYMS_Arena *arena, SYMS_String8 data, SYMS_BinAccel *bin){
  SYMS_ProfBegin("syms_imports_from_bin");
  SYMS_ImportArray result = {0};
  switch (bin->format){
    case SYMS_FileFormat_PE:
    {
      result = syms_pe_imports_from_bin(arena, data, &bin->pe_accel);
    }break;
    
    case SYMS_FileFormat_ELF:
    {
      result = syms_elf_imports_from_bin(arena, data, &bin->elf_accel);
    }break;
    
    case SYMS_FileFormat_MACH:
    {
      result = syms_mach_imports_from_bin(arena, data, &bin->mach_accel);
    }break;
  }
  SYMS_ASSERT_PARANOID(syms_parser_api_invariants());
  SYMS_ProfEnd();
  return(result);
}

SYMS_API SYMS_ExportArray
syms_exports_from_bin(SYMS_Arena *arena, SYMS_String8 data, SYMS_BinAccel *bin){
  SYMS_ProfBegin("syms_exports_from_bin");
  SYMS_ExportArray result = {0};
  switch (bin->format){
    case SYMS_FileFormat_PE:
    {
      result = syms_pe_exports_from_bin(arena, data, &bin->pe_accel);
    }break;
    
    case SYMS_FileFormat_ELF:
    {
      result = syms_elf_exports_from_bin(arena, data, &bin->elf_accel);
    }break;
    
    case SYMS_FileFormat_MACH:
    {
      result = syms_mach_exports_from_bin(arena, data, &bin->mach_accel);
    }break;
  }
  SYMS_ASSERT_PARANOID(syms_parser_api_invariants());
  SYMS_ProfEnd();
  return(result);
}


////////////////////////////////
//~ NOTE(nick): Bin List

SYMS_API SYMS_B32
syms_file_is_bin_list(SYMS_FileAccel *file){
  SYMS_B32 result = syms_false;
  switch (file->format){
    case SYMS_FileFormat_MACH: 
    {
      result = syms_mach_file_is_bin_list((SYMS_MachFileAccel*)file);
    }break;
  }
  return(result);
}

SYMS_API SYMS_BinListAccel*
syms_bin_list_from_file(SYMS_Arena *arena, SYMS_String8 data, SYMS_FileAccel *file){
  SYMS_ProfBegin("syms_bin_list_from_file");
  SYMS_BinListAccel *result = (SYMS_BinListAccel*)&syms_format_nil;
  switch (file->format){
    case SYMS_FileFormat_MACH: 
    {
      result = (SYMS_BinListAccel*)syms_mach_bin_list_accel_from_file(arena, data, (SYMS_MachFileAccel*)file);
    }break;
  }
  SYMS_ASSERT_PARANOID(syms_parser_api_invariants());
  SYMS_ProfEnd();
  return(result);
}

SYMS_API SYMS_BinInfoArray
syms_bin_info_array_from_bin_list(SYMS_Arena *arena, SYMS_BinListAccel *list){
  SYMS_ProfBegin("syms_bin_info_array_from_bin_list");
  SYMS_BinInfoArray result; syms_memzero_struct(&result);
  switch (list->format){
    case SYMS_FileFormat_MACH:
    {
      result = syms_mach_bin_info_array_from_bin_list(arena, (SYMS_MachBinListAccel*)list);
    }break;
  }
  SYMS_ASSERT_PARANOID(syms_parser_api_invariants());
  SYMS_ProfEnd();
  return(result);
}

SYMS_API SYMS_BinAccel*
syms_bin_accel_from_bin_list_number(SYMS_Arena *arena, SYMS_String8 data, SYMS_BinListAccel *list, SYMS_U64 n){
  SYMS_ProfBegin("syms_bin_accel_from_bin_list_number");
  SYMS_BinAccel *result = (SYMS_BinAccel*)&syms_format_nil;
  switch (list->format){
    case SYMS_FileFormat_MACH:
    {
      result = (SYMS_BinAccel*)syms_mach_bin_accel_from_bin_list_number(arena, data, &list->mach_accel, n);
    }break;
  }
  SYMS_ASSERT_PARANOID(syms_parser_api_invariants());
  SYMS_ProfEnd();
  return(result);
}

////////////////////////////////
//~ NOTE(rjf): Dbg File

SYMS_API SYMS_B32
syms_file_is_dbg(SYMS_FileAccel *accel){
  SYMS_ProfBegin("syms_file_is_dbg");
  SYMS_B32 result = syms_false;
  switch(accel->format){
    case SYMS_FileFormat_PDB:
    {
      result = syms_true;
    }break;
  }
  SYMS_ASSERT_PARANOID(syms_parser_api_invariants());
  SYMS_ProfEnd();
  return(result);
}

SYMS_API SYMS_DbgAccel*
syms_dbg_accel_from_file(SYMS_Arena *arena, SYMS_String8 data, SYMS_FileAccel *accel){
  SYMS_ProfBegin("syms_dbg_accel_from_file");
  SYMS_DbgAccel *result = (SYMS_DbgAccel*)&syms_format_nil;
  switch(accel->format){
    case SYMS_FileFormat_PDB:
    {
      result = (SYMS_DbgAccel*)syms_pdb_dbg_accel_from_file(arena, data, &accel->pdb_accel);
    }break;
  }
  SYMS_ASSERT_PARANOID(syms_parser_api_invariants());
  SYMS_ProfEnd();
  return(result);
}

SYMS_API SYMS_B32
syms_bin_is_dbg(SYMS_BinAccel *bin){
  SYMS_ProfBegin("syms_bin_is_dbg");
  SYMS_B32 result = syms_false;
  switch(bin->format){
    case SYMS_FileFormat_ELF:
    {
      result = syms_dw_elf_bin_accel_is_dbg(&bin->elf_accel);
    }break;
    case SYMS_FileFormat_MACH:
    {
      result = syms_dw_mach_bin_accel_is_dbg(&bin->mach_accel);
    }break;
  }
  SYMS_ASSERT_PARANOID(syms_parser_api_invariants());
  SYMS_ProfEnd();
  return(result);
}

SYMS_API SYMS_DbgAccel*
syms_dbg_accel_from_bin(SYMS_Arena *arena, SYMS_String8 data, SYMS_BinAccel *bin){
  SYMS_ProfBegin("syms_dbg_accel_from_bin");
  SYMS_DbgAccel *result = (SYMS_DbgAccel*)&syms_format_nil;
  switch(bin->format){
    case SYMS_FileFormat_ELF:
    {
      result = (SYMS_DbgAccel*)syms_dw_dbg_accel_from_elf_bin(arena, data, &bin->elf_accel);
    }break;
    
    case SYMS_FileFormat_MACH:
    {
      result = (SYMS_DbgAccel*)syms_dw_dbg_accel_from_mach_bin(arena, data, &bin->mach_accel);
    }break;
  }
  SYMS_ASSERT_PARANOID(syms_parser_api_invariants());
  SYMS_ProfEnd();
  return(result);
}

// arch
SYMS_API SYMS_Arch
syms_arch_from_dbg(SYMS_DbgAccel *dbg){
  SYMS_ProfBegin("syms_arch_from_dbg");
  SYMS_Arch result = SYMS_Arch_Null;
  switch (dbg->format){
    case SYMS_FileFormat_PDB:
    {
      result = syms_pdb_arch_from_dbg(&dbg->pdb_accel);
    }break;
    
    case SYMS_FileFormat_DWARF:
    {
      result = ((SYMS_DwDbgAccel *)dbg)->arch;
    }break;
  }
  SYMS_ASSERT_PARANOID(syms_parser_api_invariants());
  SYMS_ProfEnd();
  return(result);
}

// external info
SYMS_API SYMS_ExtFileList
syms_ext_file_list_from_dbg(SYMS_Arena *arena, SYMS_String8 data, SYMS_DbgAccel *dbg){
  SYMS_ProfBegin("syms_ext_file_list_from_dbg");
  SYMS_ExtFileList result = {0};
  switch (dbg->format){
    case SYMS_FileFormat_PDB:
    {
      // PDB has no external files, do nothing.
    }break;
    
    case SYMS_FileFormat_DWARF:
    {
      result = syms_dw_ext_file_list_from_dbg(arena, data, &dbg->dw_accel);
    }break;
  }
  SYMS_ASSERT_PARANOID(syms_parser_api_invariants());
  SYMS_ProfEnd();
  return(result);
}

// match key
SYMS_API SYMS_ExtMatchKey
syms_ext_match_key_from_dbg(SYMS_String8 data, SYMS_DbgAccel *dbg){
  SYMS_ProfBegin("syms_ext_match_key_from_dbg");
  SYMS_ExtMatchKey result = {0};
  switch (dbg->format){
    case SYMS_FileFormat_PDB:
    {
      result = syms_pdb_ext_match_key_from_dbg(data, &dbg->pdb_accel);
    }break;
    
    case SYMS_FileFormat_DWARF:
    {
      result = syms_dw_ext_match_key_from_dbg(data, &dbg->dw_accel);
    }break;
  }
  SYMS_ASSERT_PARANOID(syms_parser_api_invariants());
  SYMS_ProfEnd();
  return(result);
}

// binary secs
SYMS_API SYMS_SecInfoArray
syms_sec_info_array_from_dbg(SYMS_Arena *arena, SYMS_String8 data, SYMS_DbgAccel *dbg){
  SYMS_ProfBegin("syms_sec_info_array_from_bin");
  SYMS_SecInfoArray result = {0};
  switch (dbg->format){
    case SYMS_FileFormat_PDB:
    {
      result = syms_pdb_sec_info_array_from_dbg(arena, data, &dbg->pdb_accel);
    }break;
    
    case SYMS_FileFormat_DWARF:
    {
      result = syms_dw_sec_info_array_from_dbg(arena, data, &dbg->dw_accel);
    }break;
  }
  SYMS_ASSERT_PARANOID(syms_parser_api_invariants());
  SYMS_ProfEnd();
  return(result);
}

// default vbase
SYMS_API SYMS_U64
syms_default_vbase_from_dbg(SYMS_DbgAccel *dbg){
  SYMS_ProfBegin("syms_sec_info_array_from_bin");
  SYMS_U64 result = 0;
  switch (dbg->format){
    case SYMS_FileFormat_PDB:
    {
      // always has value 0
    }break;
    
    case SYMS_FileFormat_DWARF:
    {
      result = syms_dw_default_vbase_from_dbg(&dbg->dw_accel);
    }break;
  }
  SYMS_ASSERT_PARANOID(syms_parser_api_invariants());
  SYMS_ProfEnd();
  return(result);
}


// compilation units
SYMS_API SYMS_UnitSetAccel*
syms_unit_set_accel_from_dbg(SYMS_Arena *arena, SYMS_String8 data, SYMS_DbgAccel *accel){
  SYMS_ProfBegin("syms_unit_set_accel_from_dbg");
  SYMS_UnitSetAccel *result = (SYMS_UnitSetAccel*)&syms_format_nil;
  switch (accel->format){
    case SYMS_FileFormat_PDB:
    {
      result = (SYMS_UnitSetAccel*)syms_pdb_unit_set_accel_from_dbg(arena, data, &accel->pdb_accel);
    }break;
    
    case SYMS_FileFormat_DWARF:
    {
      result = (SYMS_UnitSetAccel*)syms_dw_unit_set_accel_from_dbg(arena, data, &accel->dw_accel);
    }break;
  }
  SYMS_ASSERT_PARANOID(syms_parser_api_invariants());
  SYMS_ProfEnd();
  return(result);
}

SYMS_API SYMS_U64
syms_unit_count_from_set(SYMS_UnitSetAccel *accel){
  SYMS_ProfBegin("syms_unit_count_from_set");
  SYMS_U64 result = 0;
  switch (accel->format){
    case SYMS_FileFormat_PDB:
    {
      result = syms_pdb_unit_count_from_set(&accel->pdb_accel);
    }break;
    
    case SYMS_FileFormat_DWARF:
    {
      result = syms_dw_unit_count_from_set(&accel->dw_accel);
    }break;
  }
  SYMS_ASSERT_PARANOID(syms_parser_api_invariants());
  SYMS_ProfEnd();
  return(result);
}

SYMS_API SYMS_UnitInfo
syms_unit_info_from_uid(SYMS_UnitSetAccel *unit_set, SYMS_UnitID uid){
  SYMS_ProfBegin("syms_unit_info_from_uid");
  SYMS_UnitInfo result = {0};
  switch (unit_set->format){
    case SYMS_FileFormat_PDB:
    {
      result = syms_pdb_unit_info_from_uid(&unit_set->pdb_accel, uid);
    }break;
    
    case SYMS_FileFormat_DWARF:
    {
      result = syms_dw_unit_info_from_uid(&unit_set->dw_accel, uid);
    }break;
  }
  SYMS_ASSERT_PARANOID(syms_parser_api_invariants());
  SYMS_ProfEnd();
  return(result);
}

SYMS_API SYMS_UnitNames
syms_unit_names_from_uid(SYMS_Arena *arena, SYMS_UnitSetAccel *unit_set, SYMS_UnitID uid){
  SYMS_ProfBegin("syms_unit_names_from_uid");
  SYMS_UnitNames result = {0};
  switch (unit_set->format){
    case SYMS_FileFormat_PDB:
    {
      result = syms_pdb_unit_names_from_uid(arena, &unit_set->pdb_accel, uid);
    }break;
    
    case SYMS_FileFormat_DWARF:
    {
      result = syms_dw_unit_names_from_uid(arena, &unit_set->dw_accel, uid);
    }break;
  }
  SYMS_ASSERT_PARANOID(syms_parser_api_invariants());
  SYMS_ProfEnd();
  return(result);
}

SYMS_API SYMS_UnitRangeArray
syms_unit_ranges_from_set(SYMS_Arena *arena, SYMS_String8 data, SYMS_DbgAccel *dbg, SYMS_UnitSetAccel *unit_set){
  SYMS_ProfBegin("syms_unit_ranges_from_set");
  SYMS_UnitRangeArray result = {0};
  switch (unit_set->format){
    case SYMS_FileFormat_PDB:
    {
      result = syms_pdb_unit_ranges_from_set(arena, data, &dbg->pdb_accel, &unit_set->pdb_accel);
    }break;
    
    case SYMS_FileFormat_DWARF:
    {
      result = syms_dw_unit_ranges_from_set(arena, data, &dbg->dw_accel, &unit_set->dw_accel);
    }break;
  }
  SYMS_ASSERT_PARANOID(syms_parser_api_invariants());
  SYMS_ASSERT_PARANOID(syms_unit_ranges_low_level_invariants(&result));
  SYMS_ASSERT_PARANOID(syms_unit_ranges_high_level_invariants(&result, unit_set));
  SYMS_ProfEnd();
  return(result);
}

SYMS_API SYMS_UnitID
syms_uid_collated_types_from_set(SYMS_UnitSetAccel *unit_set){
  SYMS_ProfBegin("syms_uid_collated_types_from_set");
  SYMS_UnitID result = 0;
  switch (unit_set->format){
    case SYMS_FileFormat_PDB:
    {
      result = SYMS_PdbPseudoUnit_TPI;
    }break;
    case SYMS_FileFormat_DWARF:
    {
      result = 0;
    }break;
  }
  SYMS_ASSERT_PARANOID(syms_parser_api_invariants());
  SYMS_ProfEnd();
  return(result);
}

SYMS_API SYMS_UnitID
syms_uid_collated_public_symbols_from_set(SYMS_UnitSetAccel *unit_set){
  SYMS_ProfBegin("syms_uid_collated_public_symbols_from_set");
  SYMS_UnitID result = 0;
  switch (unit_set->format){
    case SYMS_FileFormat_PDB:
    {
      result = SYMS_PdbPseudoUnit_SYM;
    }break;
    case SYMS_FileFormat_DWARF:
    {
      result = 0;
    }break;
  }
  SYMS_ASSERT_PARANOID(syms_parser_api_invariants());
  SYMS_ProfEnd();
  return(result);
}

// symbol parsing
SYMS_API SYMS_UnitAccel*
syms_unit_accel_from_uid(SYMS_Arena *arena, SYMS_String8 data, SYMS_DbgAccel *dbg,
                         SYMS_UnitSetAccel *unit_set, SYMS_UnitID uid){
  SYMS_ProfBegin("syms_unit_accel_from_uid");
  SYMS_UnitAccel *result = (SYMS_UnitAccel*)&syms_format_nil;
  if (unit_set->format == dbg->format){
    switch (unit_set->format){
      case SYMS_FileFormat_PDB:
      {
        result =
        (SYMS_UnitAccel*)syms_pdb_unit_accel_from_uid(arena, data, &dbg->pdb_accel, &unit_set->pdb_accel,
                                                      uid);
      }break;
      
      case SYMS_FileFormat_DWARF:
      {
        result = (SYMS_UnitAccel*)syms_dw_unit_accel_from_uid(arena, data, &dbg->dw_accel, 
                                                              &unit_set->dw_accel, uid);
      }break;
    }
  }
  SYMS_ASSERT_PARANOID(syms_parser_api_invariants());
  SYMS_ProfEnd();
  return(result);
}

SYMS_API SYMS_UnitID
syms_uid_from_unit(SYMS_UnitAccel *unit){
  SYMS_ProfBegin("syms_uid_from_unit");
  SYMS_UnitID result = 0;
  switch (unit->format){
    case SYMS_FileFormat_PDB:
    {
      result = syms_cv_uid_from_accel(&unit->cv_accel);
    }break;
    
    case SYMS_FileFormat_DWARF:
    {
      result = syms_dw_uid_from_accel(&unit->dw_accel);
    }break;
  }
  SYMS_ASSERT_PARANOID(syms_parser_api_invariants());
  SYMS_ProfEnd();
  return(result);
}

SYMS_API SYMS_SymbolIDArray
syms_proc_sid_array_from_unit(SYMS_Arena *arena, SYMS_UnitAccel *unit){
  SYMS_ProfBegin("syms_proc_sid_array_from_unit");
  SYMS_SymbolIDArray result = {0};
  switch (unit->format){
    case SYMS_FileFormat_PDB:
    {
      result = syms_cv_proc_sid_array_from_unit(arena, &unit->cv_accel);
    }break;
    
    case SYMS_FileFormat_DWARF:
    {
      result = syms_dw_proc_sid_array_from_unit(arena, &unit->dw_accel);
    }break;
  }
  SYMS_ASSERT_PARANOID(syms_parser_api_invariants());
  SYMS_ProfEnd();
  return(result);
}

SYMS_API SYMS_SymbolIDArray
syms_var_sid_array_from_unit(SYMS_Arena *arena, SYMS_UnitAccel *unit){
  SYMS_ProfBegin("syms_var_sid_array_from_unit");
  SYMS_SymbolIDArray result = {0};
  switch (unit->format){
    case SYMS_FileFormat_PDB:
    {
      result = syms_cv_var_sid_array_from_unit(arena, &unit->cv_accel);
    }break;
    
    case SYMS_FileFormat_DWARF:
    {
      result = syms_dw_var_sid_array_from_unit(arena, &unit->dw_accel);
    }break;
  }
  SYMS_ASSERT_PARANOID(syms_parser_api_invariants());
  SYMS_ProfEnd();
  return(result);
}

SYMS_API SYMS_SymbolIDArray
syms_type_sid_array_from_unit(SYMS_Arena *arena, SYMS_UnitAccel *unit){
  SYMS_ProfBegin("syms_type_sid_array_from_unit");
  SYMS_SymbolIDArray result = {0};
  switch (unit->format){
    case SYMS_FileFormat_PDB:
    {
      result = syms_cv_type_sid_array_from_unit(arena, &unit->cv_accel);
    }break;
    
    case SYMS_FileFormat_DWARF:
    {
      result = syms_dw_type_sid_array_from_unit(arena, &unit->dw_accel);
    }break;
  }
  SYMS_ASSERT_PARANOID(syms_parser_api_invariants());
  SYMS_ProfEnd();
  return(result);
}

SYMS_API SYMS_SymbolKind
syms_symbol_kind_from_sid(SYMS_String8 data, SYMS_DbgAccel *dbg, SYMS_UnitAccel *unit, SYMS_SymbolID sid){
  SYMS_ProfBegin("syms_symbol_kind_from_sid");
  SYMS_SymbolKind result = SYMS_SymbolKind_Null;
  if (unit->format == dbg->format){
    switch (unit->format){
      case SYMS_FileFormat_PDB:
      {
        result = syms_pdb_symbol_kind_from_sid(data, &dbg->pdb_accel, &unit->cv_accel, sid);
      }break;
      
      case SYMS_FileFormat_DWARF:
      {
        result = syms_dw_symbol_kind_from_sid(data, &dbg->dw_accel, &unit->dw_accel, sid);
      }break;
    }
  }
  SYMS_ASSERT_PARANOID(syms_parser_api_invariants());
  SYMS_ProfEnd();
  return(result);
}

SYMS_API SYMS_String8
syms_symbol_name_from_sid(SYMS_Arena *arena, SYMS_String8 data, SYMS_DbgAccel *dbg,
                          SYMS_UnitAccel *unit, SYMS_SymbolID sid){
  SYMS_ProfBegin("syms_symbol_kind_from_sid");
  SYMS_String8 result = {0};
  if (unit->format == dbg->format){
    switch (unit->format){
      case SYMS_FileFormat_PDB:
      {
        result = syms_cv_symbol_name_from_sid(arena, &unit->cv_accel, sid);
      }break;
      
      case SYMS_FileFormat_DWARF:
      {
        result = syms_dw_symbol_name_from_sid(arena, data, &dbg->dw_accel, &unit->dw_accel, sid);
      }break;
    }
  }
  SYMS_ASSERT_PARANOID(syms_parser_api_invariants());
  SYMS_ProfEnd();
  return(result);
}

SYMS_API SYMS_TypeInfo
syms_type_info_from_sid(SYMS_String8 data, SYMS_DbgAccel *dbg, SYMS_UnitAccel *unit, SYMS_SymbolID id){
  SYMS_ProfBegin("syms_type_info_from_sid");
  SYMS_TypeInfo result = {SYMS_TypeKind_Null};
  if (unit->format == dbg->format){
    switch (unit->format){
      case SYMS_FileFormat_PDB:
      {
        result = syms_pdb_type_info_from_sid(data, &dbg->pdb_accel, &unit->cv_accel, id);
      }break;
      
      case SYMS_FileFormat_DWARF:
      {
        result = syms_dw_type_info_from_sid(data, &dbg->dw_accel, &unit->dw_accel, id);
      }break;
    }
  }
  SYMS_ASSERT_PARANOID(syms_parser_api_invariants());
  SYMS_ProfEnd();
  return(result);
}

SYMS_API SYMS_ConstInfo
syms_const_info_from_sid(SYMS_String8 data, SYMS_DbgAccel *dbg, SYMS_UnitAccel *unit, SYMS_SymbolID id){
  SYMS_ProfBegin("syms_const_info_from_sid");
  SYMS_ConstInfo result = {SYMS_TypeKind_Null};
  if (unit->format == dbg->format){
    switch (unit->format){
      case SYMS_FileFormat_PDB:
      {
        result = syms_pdb_const_info_from_id(data, &dbg->pdb_accel, &unit->cv_accel, id);
      }break;
      
      case SYMS_FileFormat_DWARF:
      {
        result = syms_dw_const_info_from_sid(data, &dbg->dw_accel, &unit->dw_accel, id);
      }break;
    }
  }
  SYMS_ASSERT_PARANOID(syms_parser_api_invariants());
  SYMS_ProfEnd();
  return(result);
}

// variable info

SYMS_API SYMS_USID
syms_type_from_var_sid(SYMS_String8 data, SYMS_DbgAccel *dbg, SYMS_UnitAccel *unit, SYMS_SymbolID id){
  SYMS_ProfBegin("syms_type_from_var_sid");
  SYMS_USID result = {0};
  if (unit->format == dbg->format){
    switch (unit->format){
      case SYMS_FileFormat_PDB:
      {
        result = syms_pdb_type_from_var_id(data, &dbg->pdb_accel, &unit->cv_accel, id);
      }break;
      
      case SYMS_FileFormat_DWARF:
      {
        result = syms_dw_type_from_var_sid(data, &dbg->dw_accel, &unit->dw_accel, id);
      }break;
    }
  }
  SYMS_ASSERT_PARANOID(syms_parser_api_invariants());
  SYMS_ProfEnd();
  return(result);
}

SYMS_API SYMS_U64
syms_voff_from_var_sid(SYMS_String8 data, SYMS_DbgAccel *dbg, SYMS_UnitAccel *unit, SYMS_SymbolID id){
  SYMS_ProfBegin("syms_voff_from_var_sid");
  SYMS_U64 result = 0;
  if (unit->format == dbg->format){
    switch (unit->format){
      case SYMS_FileFormat_PDB:
      {
        result = syms_pdb_voff_from_var_sid(data, &dbg->pdb_accel, &unit->cv_accel, id);
      }break;
      
      case SYMS_FileFormat_DWARF:
      {
        result = syms_dw_voff_from_var_sid(data, &dbg->dw_accel, &unit->dw_accel, id);
      }break;
    }
  }
  SYMS_ASSERT_PARANOID(syms_parser_api_invariants());
  SYMS_ProfEnd();
  return(result);
}

SYMS_API SYMS_Location
syms_location_from_var_sid(SYMS_Arena *arena, SYMS_String8 data, SYMS_DbgAccel *dbg,
                           SYMS_UnitAccel *unit, SYMS_SymbolID sid){
  SYMS_ProfBegin("syms_location_from_var_sid");
  SYMS_Location result = {0};
  if (unit->format == dbg->format){
    switch (unit->format){
      case SYMS_FileFormat_PDB:
      {
        result = syms_pdb_location_from_var_sid(arena, data, &dbg->pdb_accel, &unit->cv_accel, sid);
      }break;
      
      case SYMS_FileFormat_DWARF:
      {
        result = syms_dw_location_from_var_sid(arena, data, &dbg->dw_accel, &unit->dw_accel, sid);
      }break;
    }
  }
  SYMS_ASSERT_PARANOID(syms_parser_api_invariants());
  SYMS_ProfEnd();
  return(result);
}

SYMS_API SYMS_LocRangeArray
syms_location_ranges_from_var_sid(SYMS_Arena *arena, SYMS_String8 data, SYMS_DbgAccel *dbg,
                                  SYMS_UnitAccel *unit, SYMS_SymbolID sid){
  SYMS_ProfBegin("syms_location_ranges_from_var_sid");
  SYMS_LocRangeArray result = {0};
  if (unit->format == dbg->format){
    switch (unit->format){
      case SYMS_FileFormat_PDB:
      {
        result = syms_pdb_location_ranges_from_var_sid(arena, data, &dbg->pdb_accel,
                                                       &unit->cv_accel, sid);
      }break;
      
      case SYMS_FileFormat_DWARF:
      {
        result = syms_dw_location_ranges_from_var_sid(arena, data, &dbg->dw_accel, &unit->dw_accel, sid);
      }break;
    }
  }
  SYMS_ASSERT_PARANOID(syms_parser_api_invariants());
  SYMS_ProfEnd();
  return(result);
}

SYMS_API SYMS_Location
syms_location_from_id(SYMS_Arena *arena, SYMS_String8 data, SYMS_DbgAccel *dbg,
                      SYMS_UnitAccel *unit, SYMS_LocID loc_id){
  SYMS_ProfBegin("syms_location_from_id");
  SYMS_Location result = {0};
  if (unit->format == dbg->format){
    switch (unit->format){
      case SYMS_FileFormat_PDB:
      {
        result = syms_pdb_location_from_id(arena, data, &dbg->pdb_accel, &unit->cv_accel, loc_id);
      }break;
      
      case SYMS_FileFormat_DWARF:
      {
        result = syms_dw_location_from_id(arena, data, &dbg->dw_accel, &unit->dw_accel, loc_id);
      }break;
    }
  }
  SYMS_ASSERT_PARANOID(syms_parser_api_invariants());
  SYMS_ProfEnd();
  return(result);
}


// member info

SYMS_API SYMS_MemsAccel*
syms_mems_accel_from_sid(SYMS_Arena *arena, SYMS_String8 data, SYMS_DbgAccel *dbg,
                         SYMS_UnitAccel *unit, SYMS_SymbolID id){
  SYMS_ProfBegin("syms_mems_accel_from_sid");
  SYMS_MemsAccel *result = (SYMS_MemsAccel *)&syms_format_nil;
  if (dbg->format == unit->format){
    switch (unit->format){
      case SYMS_FileFormat_PDB:
      {
        result = (SYMS_MemsAccel*)syms_pdb_mems_accel_from_sid(arena, data, &dbg->pdb_accel, &unit->cv_accel, id);
      }break;
      
      case SYMS_FileFormat_DWARF:
      {
        result = (SYMS_MemsAccel *)syms_dw_mems_accel_from_sid(arena, data, &dbg->dw_accel, &unit->dw_accel, id);
      }break;
    }
  }
  SYMS_ASSERT_PARANOID(syms_parser_api_invariants());
  SYMS_ProfEnd();
  return(result);
}

SYMS_API SYMS_U64
syms_mem_count_from_mems(SYMS_MemsAccel *mems){
  SYMS_ProfBegin("syms_mem_count_from_mems");
  SYMS_U64 result = 0;
  switch (mems->format){
    case SYMS_FileFormat_PDB:
    {
      result = syms_pdb_mem_count_from_mems(&mems->cv_accel);
    }break;
    
    case SYMS_FileFormat_DWARF:
    {
      result = syms_dw_mem_count_from_mems(&mems->dw_accel);
    }break;
  }
  SYMS_ASSERT_PARANOID(syms_parser_api_invariants());
  SYMS_ProfEnd();
  return(result);
}

SYMS_API SYMS_MemInfo
syms_mem_info_from_number(SYMS_Arena *arena, SYMS_String8 data, SYMS_DbgAccel *dbg,
                          SYMS_UnitAccel *unit, SYMS_MemsAccel *mems, SYMS_U64 n){
  SYMS_ProfBegin("syms_mem_info_from_number");
  SYMS_MemInfo result = {SYMS_MemKind_Null};
  if (dbg->format == unit->format && unit->format == mems->format){
    switch (mems->format){
      case SYMS_FileFormat_PDB:
      {
        result = syms_pdb_mem_info_from_number(arena, data, &dbg->pdb_accel, &unit->cv_accel, &mems->cv_accel, n);
      }break;
      
      case SYMS_FileFormat_DWARF:
      {
        result = syms_dw_mem_info_from_number(arena, data, &dbg->dw_accel, &unit->dw_accel, &mems->dw_accel, n);
      }break;
    }
  }
  SYMS_ASSERT_PARANOID(syms_parser_api_invariants());
  SYMS_ProfEnd();
  return(result);
}

SYMS_API SYMS_USID
syms_type_from_mem_number(SYMS_String8 data, SYMS_DbgAccel *dbg, SYMS_UnitAccel *unit,
                          SYMS_MemsAccel *mems, SYMS_U64 n){
  SYMS_ProfBegin("syms_type_from_mem_number");
  SYMS_USID result = {0};
  if (dbg->format == unit->format && unit->format == mems->format){
    switch (mems->format){
      case SYMS_FileFormat_PDB:
      {
        result = syms_pdb_type_from_mem_number(data, &dbg->pdb_accel, &unit->cv_accel, &mems->cv_accel, n);
      }break;
      
      case SYMS_FileFormat_DWARF:
      {
        result = syms_dw_type_from_mem_number(data, &dbg->dw_accel, &unit->dw_accel, &mems->dw_accel, n);
      }break;
    }
  }
  SYMS_ASSERT_PARANOID(syms_parser_api_invariants());
  SYMS_ProfEnd();
  return(result);
}

SYMS_API SYMS_SigInfo
syms_sig_info_from_mem_number(SYMS_Arena *arena, SYMS_String8 data, SYMS_DbgAccel *dbg,
                              SYMS_UnitAccel *unit, SYMS_MemsAccel *mems, SYMS_U64 n){
  SYMS_ProfBegin("syms_sig_info_from_mem_number");
  SYMS_SigInfo result = {0};
  if (dbg->format == unit->format && unit->format == mems->format){
    switch (mems->format){
      case SYMS_FileFormat_PDB:
      {
        result = syms_pdb_sig_info_from_mem_number(arena, data, &dbg->pdb_accel, &unit->cv_accel,
                                                   &mems->cv_accel, n);
      }break;
      
      case SYMS_FileFormat_DWARF:
      {
        result = syms_dw_sig_info_from_mem_number(arena, data, &dbg->dw_accel, &unit->dw_accel, &mems->dw_accel, n);
      }break;
    }
  }
  SYMS_ASSERT_PARANOID(syms_parser_api_invariants());
  SYMS_ProfEnd();
  return(result);
}

SYMS_API SYMS_USID
syms_symbol_from_mem_number(SYMS_String8 data, SYMS_DbgAccel *dbg, SYMS_UnitAccel *unit, SYMS_MemsAccel *mems,
                            SYMS_U64 n){
  SYMS_ProfBegin("syms_symbol_from_mem_number");
  SYMS_USID result = {0};
  if (dbg->format == unit->format && unit->format == mems->format){
    switch (mems->format){
      case SYMS_FileFormat_PDB:
      {
        // NOTE(allen): not available in PDB
      }break;
      
      case SYMS_FileFormat_DWARF:
      {
        result = syms_dw_symbol_from_mem_number(data, &dbg->dw_accel, &unit->dw_accel, &mems->dw_accel, n);
      }break;
    }
  }
  SYMS_ASSERT_PARANOID(syms_parser_api_invariants());
  SYMS_ProfEnd();
  return(result);
}

SYMS_API SYMS_USID
syms_containing_type_from_sid(SYMS_String8 data, SYMS_DbgAccel *dbg, SYMS_UnitAccel *unit, SYMS_SymbolID sid){
  SYMS_ProfBegin("syms_containing_type_from_sid");
  SYMS_USID result = {0};
  if(dbg->format == unit->format){
    switch(unit->format){
      case SYMS_FileFormat_PDB:
      {
        // NOTE(rjf): not available in PDB
      }break;
      
      case SYMS_FileFormat_DWARF:
      {
        result = syms_dw_containing_type_from_sid(data, &dbg->dw_accel, &unit->dw_accel, sid);
      }break;
    }
  }
  SYMS_ASSERT_PARANOID(syms_parser_api_invariants());
  SYMS_ProfEnd();
  return(result);
}

SYMS_API SYMS_String8
syms_linkage_name_from_sid(SYMS_Arena *arena, SYMS_String8 data, SYMS_DbgAccel *dbg, SYMS_UnitAccel *unit, SYMS_SymbolID sid){
  SYMS_ProfBegin("syms_linkage_name_from_sid");
  SYMS_String8 result = {0};
  if(dbg->format == unit->format){
    switch(unit->format){
      case SYMS_FileFormat_PDB:
      {
        // NOTE(rjf): not available in PDB
      }break;
      
      case SYMS_FileFormat_DWARF:
      {
        result = syms_dw_linkage_name_from_sid(arena, data, &dbg->dw_accel, &unit->dw_accel, sid);
      }break;
    }
  }
  SYMS_ASSERT_PARANOID(syms_parser_api_invariants());
  SYMS_ProfEnd();
  return result;
}

SYMS_API SYMS_EnumMemberArray
syms_enum_member_array_from_sid(SYMS_Arena *arena, SYMS_String8 data, SYMS_DbgAccel *dbg,
                                SYMS_UnitAccel *unit, SYMS_SymbolID sid){
  SYMS_ProfBegin("syms_enum_member_array_from_sid");
  SYMS_EnumMemberArray result = {0};
  if (dbg->format == unit->format){
    switch (unit->format){
      case SYMS_FileFormat_PDB:
      {
        result = syms_pdb_enum_member_array_from_sid(arena, data, &dbg->pdb_accel, &unit->cv_accel, sid);
      }break;
      
      case SYMS_FileFormat_DWARF:
      {
        result = syms_dw_enum_member_array_from_sid(arena, data, &dbg->dw_accel, &unit->dw_accel, sid);
      }break;
    }
  }
  SYMS_ASSERT_PARANOID(syms_parser_api_invariants());
  SYMS_ProfEnd();
  return(result);
}

// symbol info

SYMS_API SYMS_UnitIDAndSig
syms_sig_handle_from_proc_sid(SYMS_String8 data, SYMS_DbgAccel *dbg, SYMS_UnitAccel *unit, SYMS_SymbolID sid){
  SYMS_ProfBegin("syms_sig_handle_from_proc_sid");
  SYMS_UnitIDAndSig result = {0};
  if (unit->format == dbg->format){
    switch (unit->format){
      case SYMS_FileFormat_PDB:
      {
        result = syms_pdb_proc_sig_handle_from_id(data, &dbg->pdb_accel, &unit->cv_accel, sid);
      }break;
      
      case SYMS_FileFormat_DWARF:
      {
        result = syms_dw_proc_sig_handle_from_sid(data, &dbg->dw_accel, &unit->dw_accel, sid);
      }break;
    }
  }
  SYMS_ASSERT_PARANOID(syms_parser_api_invariants());
  SYMS_ProfEnd();
  return(result);
}

SYMS_API SYMS_SigInfo
syms_sig_info_from_handle(SYMS_Arena *arena, SYMS_String8 data, SYMS_DbgAccel *dbg, SYMS_UnitAccel *unit,
                          SYMS_SigHandle handle){
  SYMS_ProfBegin("syms_sig_info_from_handle");
  SYMS_SigInfo result = {0};
  if (unit->format == dbg->format){
    switch (unit->format){
      case SYMS_FileFormat_PDB:
      {
        result = syms_pdb_sig_info_from_handle(arena, data, &dbg->pdb_accel, &unit->cv_accel, handle);
      }break;
      
      case SYMS_FileFormat_DWARF:
      {
        result = syms_dw_sig_info_from_handle(arena, data, &dbg->dw_accel, &unit->dw_accel, handle);
      }break;
    }
  }
  SYMS_ASSERT_PARANOID(syms_parser_api_invariants());
  SYMS_ProfEnd();
  return(result);
}

SYMS_API SYMS_U64RangeArray
syms_scope_vranges_from_sid(SYMS_Arena *arena, SYMS_String8 data, SYMS_DbgAccel *dbg,
                            SYMS_UnitAccel *unit, SYMS_SymbolID id){
  SYMS_ProfBegin("syms_scope_vranges_from_sid");
  SYMS_U64RangeArray result = {0};
  if (unit->format == dbg->format){
    switch (unit->format){
      case SYMS_FileFormat_PDB:
      {
        result = syms_pdb_scope_vranges_from_sid(arena, data, &dbg->pdb_accel, &unit->cv_accel, id);
      }break;
      
      case SYMS_FileFormat_DWARF:
      {
        result = syms_dw_scope_vranges_from_sid(arena, data, &dbg->dw_accel, &unit->dw_accel, id);
      }break;
    }
  }
  SYMS_ASSERT_PARANOID(syms_parser_api_invariants());
  SYMS_ProfEnd();
  return(result);
}

SYMS_API SYMS_SymbolIDArray
syms_scope_children_from_sid(SYMS_Arena *arena, SYMS_String8 data, SYMS_DbgAccel *dbg,
                             SYMS_UnitAccel *unit, SYMS_SymbolID sid){
  SYMS_ProfBegin("syms_scope_children_from_sid");
  SYMS_SymbolIDArray result = {0};
  if (unit->format == dbg->format){
    switch (unit->format){
      case SYMS_FileFormat_PDB:
      {
        result = syms_pdb_scope_children_from_sid(arena, data, &dbg->pdb_accel, &unit->cv_accel, sid);
      }break;
      
      case SYMS_FileFormat_DWARF:
      {
        result = syms_dw_scope_children_from_sid(arena, data, &dbg->dw_accel, &unit->dw_accel, sid);
      }break;
    }
  }
  SYMS_ASSERT_PARANOID(syms_parser_api_invariants());
  SYMS_ProfEnd();
  return(result);
}

SYMS_API SYMS_Location
syms_location_from_proc_sid(SYMS_Arena *arena, SYMS_String8 data, SYMS_DbgAccel *dbg,
                            SYMS_UnitAccel *unit, SYMS_SymbolID sid, SYMS_ProcLoc proc_loc){
  SYMS_ProfBegin("syms_location_from_proc_sid");
  SYMS_Location result = {0};
  if (unit->format == dbg->format){
    switch (unit->format){
      case SYMS_FileFormat_PDB:
      {
        // do nothing
      }break;
      
      case SYMS_FileFormat_DWARF:
      {
        result = syms_dw_location_from_proc_sid(arena, data, &dbg->dw_accel, &unit->dw_accel, sid, proc_loc);
      }break;
    }
  }
  SYMS_ASSERT_PARANOID(syms_parser_api_invariants());
  SYMS_ProfEnd();
  return(result);
}

SYMS_API SYMS_LocRangeArray
syms_location_ranges_from_proc_sid(SYMS_Arena *arena, SYMS_String8 data, SYMS_DbgAccel *dbg,
                                   SYMS_UnitAccel *unit, SYMS_SymbolID sid, SYMS_ProcLoc proc_loc){
  SYMS_ProfBegin("syms_location_ranges_from_proc_sid");
  SYMS_LocRangeArray result = {0};
  if (unit->format == dbg->format){
    switch (unit->format){
      case SYMS_FileFormat_PDB:
      {
        // do nothing
      }break;
      
      case SYMS_FileFormat_DWARF:
      {
        result = syms_dw_location_ranges_from_proc_sid(arena, data, &dbg->dw_accel, &unit->dw_accel, sid, proc_loc);
      }break;
    }
  }
  SYMS_ASSERT_PARANOID(syms_parser_api_invariants());
  SYMS_ProfEnd();
  return(result);
}

// signature info
SYMS_API SYMS_SigInfo
syms_sig_info_from_type_sid(SYMS_Arena *arena, SYMS_String8 data, SYMS_DbgAccel *dbg,
                            SYMS_UnitAccel *unit, SYMS_SymbolID sid){
  SYMS_ProfBegin("syms_sig_info_from_type_sid");
  SYMS_SigInfo result = {0};
  if (unit->format == dbg->format){
    switch (unit->format){
      case SYMS_FileFormat_PDB:
      {
        result = syms_pdb_sig_info_from_id(arena, data, &dbg->pdb_accel, &unit->cv_accel, sid);
      }break;
      
      case SYMS_FileFormat_DWARF:
      {
        result = syms_dw_sig_info_from_sid(arena, data, &dbg->dw_accel, &unit->dw_accel, sid);
      }break;
    }
  }
  SYMS_ASSERT_PARANOID(syms_parser_api_invariants());
  SYMS_ProfEnd();
  return(result);
}

// line info
SYMS_API SYMS_String8
syms_file_name_from_id(SYMS_Arena *arena, SYMS_String8 data, SYMS_DbgAccel *dbg, SYMS_UnitSetAccel *unit_set,
                       SYMS_UnitID uid, SYMS_FileID file_id){
  SYMS_ProfBegin("syms_file_name_from_id");
  SYMS_String8 result = {0};
  if (unit_set->format == dbg->format){
    switch (unit_set->format){
      case SYMS_FileFormat_PDB:
      {
        result = syms_pdb_file_name_from_id(arena, data, &dbg->pdb_accel, &unit_set->pdb_accel, uid, file_id);
      }break;
      
      case SYMS_FileFormat_DWARF:
      {
        result = syms_dw_file_name_from_id(arena, &unit_set->dw_accel, uid, file_id);
      }break;
    }
  }
  SYMS_ASSERT_PARANOID(syms_parser_api_invariants());
  SYMS_ProfEnd();
  return(result);
}

SYMS_API SYMS_String8Array
syms_file_table_from_uid(SYMS_Arena *arena, SYMS_String8 data, SYMS_DbgAccel *dbg,
                         SYMS_UnitSetAccel *unit_set, SYMS_UnitID uid)
{
  SYMS_ProfBegin("syms_file_table_from_uid");
  SYMS_String8Array result = {0};
  if(unit_set->format == dbg->format){
    switch (unit_set->format){
      case SYMS_FileFormat_DWARF:
      {
        result = syms_dw_file_table_from_uid(arena, data, &dbg->dw_accel, &unit_set->dw_accel, uid);
      }break;
    }
  }
  SYMS_ASSERT_PARANOID(syms_parser_api_invariants());
  SYMS_ProfEnd();
  return(result);
}

SYMS_API SYMS_LineParseOut
syms_line_parse_from_uid(SYMS_Arena *arena, SYMS_String8 data, SYMS_DbgAccel *dbg,
                         SYMS_UnitSetAccel *unit_set, SYMS_UnitID uid){
  SYMS_ProfBegin("syms_line_parse_from_uid");
  SYMS_LineParseOut result = {0};
  if (dbg->format == unit_set->format){
    switch (unit_set->format){
      case SYMS_FileFormat_PDB:
      {
        result = syms_pdb_line_parse_from_uid(arena, data, &dbg->pdb_accel, &unit_set->pdb_accel, uid);
      }break;
      
      case SYMS_FileFormat_DWARF:
      {
        result = syms_dw_line_parse_from_uid(arena, data, &dbg->dw_accel, &unit_set->dw_accel, uid);
      }break;
    }
  }
  SYMS_ASSERT_PARANOID(syms_parser_api_invariants());
  SYMS_ASSERT_PARANOID(syms_line_table_low_level_invariants(&result.line_table));
  SYMS_ASSERT_PARANOID(syms_line_table_high_level_invariants(&result.line_table));
  SYMS_ProfEnd();
  return(result);
}

// name maps
SYMS_API SYMS_MapAccel*
syms_type_map_from_dbg(SYMS_Arena *arena, SYMS_String8 data, SYMS_DbgAccel *dbg){
  SYMS_ProfBegin("syms_type_map_from_dbg");
  SYMS_MapAccel *result = (SYMS_MapAccel*)&syms_format_nil;
  switch (dbg->format){
    case SYMS_FileFormat_PDB:
    {
      result = (SYMS_MapAccel*)syms_pdb_type_map_from_dbg(arena, data, &dbg->pdb_accel);
    }break;
    
    case SYMS_FileFormat_DWARF:
    {
      result = (SYMS_MapAccel*)syms_dw_type_map_from_dbg(arena, data, &dbg->dw_accel);
    }break;
  }
  SYMS_ASSERT_PARANOID(syms_parser_api_invariants());
  SYMS_ProfEnd();
  return(result);
}

SYMS_API SYMS_MapAccel*
syms_unmangled_symbol_map_from_dbg(SYMS_Arena *arena, SYMS_String8 data, SYMS_DbgAccel *dbg){
  SYMS_ProfBegin("syms_unmangled_symbol_map_from_dbg");
  SYMS_MapAccel *result = (SYMS_MapAccel*)&syms_format_nil;
  switch (dbg->format){
    case SYMS_FileFormat_PDB:
    {
      result = (SYMS_MapAccel*)syms_pdb_unmangled_symbol_map_from_dbg(arena, data, &dbg->pdb_accel);
    }break;
    
    case SYMS_FileFormat_DWARF:
    {
      result = (SYMS_MapAccel*)syms_dw_image_symbol_map_from_dbg(arena, data, &dbg->dw_accel);
    }break;
  }
  SYMS_ASSERT_PARANOID(syms_parser_api_invariants());
  SYMS_ProfEnd();
  return(result);
}

SYMS_API SYMS_UnitID
syms_partner_uid_from_map(SYMS_MapAccel *map){
  SYMS_ProfBegin("syms_partner_uid_from_map");
  SYMS_UnitID result = 0;
  switch (map->format){
    case SYMS_FileFormat_PDB:
    {
      result = syms_pdb_partner_uid_from_map(&map->pdb_accel);
    }break;
    
    case SYMS_FileFormat_DWARF:
    {
      // NOTE(allen): do nothing
    }break;
  }
  SYMS_ASSERT_PARANOID(syms_parser_api_invariants());
  SYMS_ProfEnd();
  return(result);
}

SYMS_API SYMS_USIDList
syms_usid_list_from_string(SYMS_Arena *arena, SYMS_String8 data, SYMS_DbgAccel *dbg,
                           SYMS_MapAndUnit *map_and_unit, SYMS_String8 string){
  SYMS_ProfBegin("syms_usid_list_from_string");
  SYMS_USIDList result = {0};
  if (map_and_unit->map->format == dbg->format){
    switch (dbg->format){
      case SYMS_FileFormat_PDB:
      {
        result = syms_pdb_usid_list_from_string(arena, data, &dbg->pdb_accel,
                                                &map_and_unit->unit->cv_accel,
                                                &map_and_unit->map->pdb_accel,
                                                string);
      }break;
      
      case SYMS_FileFormat_DWARF:
      {
        result = syms_dw_usid_list_from_string(arena, &map_and_unit->map->dw_accel, string);
      }break;
    }
  }
  SYMS_ASSERT_PARANOID(syms_parser_api_invariants());
  SYMS_ProfEnd();
  return(result);
}

// mangled names (linker names)

SYMS_API SYMS_UnitID
syms_link_names_uid(SYMS_DbgAccel *dbg){
  SYMS_UnitID result = 0;
  switch (dbg->format){
    case SYMS_FileFormat_PDB:
    {
      result = syms_pdb_link_names_uid();
    }break;
    
    case SYMS_FileFormat_DWARF:
    {
      // NOTE(allen): do nothing
    }break;
  }
  return(result);
}

SYMS_API SYMS_B32
syms_link_map_is_complete(SYMS_LinkMapAccel *map){
  SYMS_ProfBegin("syms_link_map_is_complete");
  SYMS_B32 result = syms_false;
  switch (map->format){
    case SYMS_FileFormat_PDB:
    {
      result = syms_true;
    }break;
    case SYMS_FileFormat_DWARF:
    {
      result = syms_false;
    }break;
  }
  SYMS_ProfEnd();
  return(result);
}

SYMS_API SYMS_LinkMapAccel*
syms_link_map_from_dbg(SYMS_Arena *arena, SYMS_String8 data, SYMS_DbgAccel *dbg){
  SYMS_ProfBegin("syms_link_map_from_dbg");
  SYMS_LinkMapAccel *result = (SYMS_LinkMapAccel*)&syms_format_nil;
  switch (dbg->format){
    case SYMS_FileFormat_PDB:
    {
      result = (SYMS_LinkMapAccel*)syms_pdb_link_map_from_dbg(arena, data, (SYMS_PdbDbgAccel*)dbg);
    }break;
    
    case SYMS_FileFormat_DWARF:
    {
      // TODO(allen): 
    }break;
  }
  SYMS_ProfEnd();
  return(result);
}

SYMS_API SYMS_U64
syms_voff_from_link_name(SYMS_String8 data, SYMS_DbgAccel *dbg, SYMS_LinkMapAccel *map,
                         SYMS_UnitAccel *link_unit, SYMS_String8 name){
  SYMS_ProfBegin("syms_voff_from_link_name");
  SYMS_U64 result = 0;
  if (dbg->format == map->format){
    switch (dbg->format){
      case SYMS_FileFormat_PDB:
      {
        result = syms_pdb_voff_from_link_name(data, (SYMS_PdbDbgAccel*)dbg, (SYMS_PdbLinkMapAccel*)map,
                                              (SYMS_CvUnitAccel*)link_unit, name);
      }break;
      
      case SYMS_FileFormat_DWARF:
      {
        // TODO(allen): 
      }break;
    }
  }
  SYMS_ProfEnd();
  return(result);
}

SYMS_API SYMS_LinkNameRecArray
syms_link_name_array_from_unit(SYMS_Arena *arena, SYMS_String8 data, SYMS_DbgAccel *dbg, SYMS_UnitAccel *unit){
  SYMS_ProfBegin("syms_link_name_array_from_unit");
  SYMS_LinkNameRecArray result = {0};
  if (dbg->format == unit->format){
    switch (dbg->format){
      case SYMS_FileFormat_PDB:
      {
        result = syms_pdb_link_name_array_from_unit(arena, data, (SYMS_PdbDbgAccel*)dbg, (SYMS_CvUnitAccel*)unit);
      }break;
      
      case SYMS_FileFormat_DWARF:
      {
        // TODO(allen): 
      }break;
    }
  }
  SYMS_ProfEnd();
  return(result);
}

// thread vars

SYMS_API SYMS_UnitID
syms_tls_var_uid_from_dbg(SYMS_DbgAccel *dbg){
  SYMS_UnitID result = 0;
  switch (dbg->format){
    case SYMS_FileFormat_PDB:
    {
      result = syms_pdb_tls_var_uid_from_dbg(&dbg->pdb_accel);
    }break;
  }
  return(result);
}

SYMS_API SYMS_SymbolIDArray
syms_tls_var_sid_array_from_unit(SYMS_Arena *arena, SYMS_UnitAccel *unit){
  SYMS_ProfBegin("syms_tls_var_sid_array_from_unit");
  SYMS_SymbolIDArray result = {0};
  switch (unit->format){
    case SYMS_FileFormat_PDB:
    {
      result = syms_pdb_tls_var_sid_array_from_unit(arena, &unit->cv_accel);
    }break;
    
    case SYMS_FileFormat_DWARF:
    {
      // TODO(nick): TLS support on dwarf
    }break;
  }
  SYMS_ASSERT_PARANOID(syms_parser_api_invariants());
  SYMS_ProfEnd();
  return(result);
}

#endif //SYMS_PARSER_C
