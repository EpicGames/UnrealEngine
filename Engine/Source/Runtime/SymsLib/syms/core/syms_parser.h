// Copyright Epic Games, Inc. All Rights Reserved.
/* date = June 8th 2021 10:15 am */

#ifndef SYMS_PARSER_H
#define SYMS_PARSER_H

////////////////////////////////
//~ allen: Abstraction Acclerators

typedef union SYMS_FileAccel{
  SYMS_FileFormat format;
  SYMS_ElfFileAccel elf_accel;
  SYMS_PeFileAccel pe_accel;
  SYMS_PdbFileAccel pdb_accel;
  SYMS_MachFileAccel mach_accel;
} SYMS_FileAccel;

typedef union SYMS_BinAccel{
  SYMS_FileFormat format;
  SYMS_PeBinAccel pe_accel;
  SYMS_ElfBinAccel elf_accel;
  SYMS_MachBinAccel mach_accel;
} SYMS_BinAccel;

typedef union SYMS_BinListAccel{
  SYMS_FileFormat format;
  SYMS_MachBinListAccel mach_accel;
} SYMS_BinListAccel;

typedef union SYMS_DbgAccel{
  SYMS_FileFormat format;
  SYMS_DwDbgAccel dw_accel;
  SYMS_PdbDbgAccel pdb_accel;
} SYMS_DbgAccel;

typedef union SYMS_UnitSetAccel{
  SYMS_FileFormat format;
  SYMS_DwUnitSetAccel dw_accel;
  SYMS_PdbUnitSetAccel pdb_accel;
} SYMS_UnitSetAccel;

typedef union SYMS_UnitAccel{
  SYMS_FileFormat format;
  SYMS_DwUnitAccel dw_accel;
  SYMS_CvUnitAccel cv_accel;
} SYMS_UnitAccel;

typedef union SYMS_MemsAccel{
  SYMS_FileFormat format;
  SYMS_DwMemsAccel dw_accel;
  SYMS_CvMemsAccel cv_accel;
} SYMS_MemsAccel;

typedef union SYMS_MapAccel{
  SYMS_FileFormat format;
  SYMS_DwMapAccel dw_accel;
  SYMS_PdbMapAccel pdb_accel;
} SYMS_MapAccel;

typedef union SYMS_LinkMapAccel{
  SYMS_FileFormat format;
  SYMS_DwLinkMapAccel dw_accel;
  SYMS_PdbLinkMapAccel pdb_accel;
} SYMS_LinkMapAccel;

////////////////////////////////
//~ allen: Accelerator Bundle Types

typedef struct SYMS_MapAndUnit{
  SYMS_MapAccel *map;
  SYMS_UnitAccel *unit;
} SYMS_MapAndUnit;

typedef struct SYMS_ParseBundle{
  SYMS_String8 bin_data;
  SYMS_String8 dbg_data;
  SYMS_BinAccel *bin;
  SYMS_DbgAccel *dbg;
} SYMS_ParseBundle;

SYMS_C_LINKAGE_BEGIN


////////////////////////////////
//~ allen: Accel Helpers

#define syms_accel_is_good(a) ((a) != 0 && (a)->format != SYMS_FileFormat_Null)

////////////////////////////////
//~ allen: General File Analysis

SYMS_API SYMS_FileAccel*   syms_file_accel_from_data(SYMS_Arena *arena, SYMS_String8 data);
SYMS_API SYMS_FileFormat   syms_file_format_from_file(SYMS_FileAccel *file);

////////////////////////////////
//~ allen: Bin File

SYMS_API SYMS_B32          syms_file_is_bin(SYMS_FileAccel *file);
SYMS_API SYMS_BinAccel*    syms_bin_accel_from_file(SYMS_Arena *arena, SYMS_String8 data,
                                                    SYMS_FileAccel *file);

// arch
SYMS_API SYMS_Arch         syms_arch_from_bin(SYMS_BinAccel *bin);

// external info
SYMS_API SYMS_ExtFileList  syms_ext_file_list_from_bin(SYMS_Arena *arena, SYMS_String8 data,
                                                       SYMS_BinAccel *bin);

// binary secs
SYMS_API SYMS_SecInfoArray syms_sec_info_array_from_bin(SYMS_Arena *arena, SYMS_String8 data,
                                                        SYMS_BinAccel *bin);

// default vbase
SYMS_API SYMS_U64          syms_default_vbase_from_bin(SYMS_BinAccel *bin);

// entry point
SYMS_API SYMS_U64          syms_entry_point_voff_from_bin(SYMS_BinAccel *bin);

// imports & exports
SYMS_API SYMS_ImportArray  syms_imports_from_bin(SYMS_Arena *arena, SYMS_String8 data,
                                                 SYMS_BinAccel *bin);
SYMS_API SYMS_ExportArray  syms_exports_from_bin(SYMS_Arena *arena, SYMS_String8 data,
                                                 SYMS_BinAccel *bin);

////////////////////////////////
//~ nick: Bin List

SYMS_API SYMS_B32          syms_file_is_bin_list(SYMS_FileAccel *file);
SYMS_API SYMS_BinListAccel*syms_bin_list_from_file(SYMS_Arena *arena, SYMS_String8 data,
                                                   SYMS_FileAccel *file);
SYMS_API SYMS_BinInfoArray syms_bin_info_array_from_bin_list(SYMS_Arena *arena, SYMS_BinListAccel *list);
SYMS_API SYMS_BinAccel*    syms_bin_accel_from_bin_list_number(SYMS_Arena *arena, SYMS_String8 data,
                                                               SYMS_BinListAccel *list, SYMS_U64 n);

////////////////////////////////
//~ rjf: Dbg File

SYMS_API SYMS_B32          syms_file_is_dbg(SYMS_FileAccel *file);
SYMS_API SYMS_DbgAccel*    syms_dbg_accel_from_file(SYMS_Arena *arena, SYMS_String8 data,
                                                    SYMS_FileAccel *file);

SYMS_API SYMS_B32          syms_bin_is_dbg(SYMS_BinAccel *bin);
SYMS_API SYMS_DbgAccel*    syms_dbg_accel_from_bin(SYMS_Arena *arena, SYMS_String8 data,
                                                   SYMS_BinAccel *bin);

// arch
SYMS_API SYMS_Arch         syms_arch_from_dbg(SYMS_DbgAccel *dbg);

// external info
SYMS_API SYMS_ExtFileList  syms_ext_file_list_from_dbg(SYMS_Arena *arena, SYMS_String8 data,
                                                       SYMS_DbgAccel *dbg);

// match key
SYMS_API SYMS_ExtMatchKey  syms_ext_match_key_from_dbg(SYMS_String8 data, SYMS_DbgAccel *dbg);

// binary secs
SYMS_API SYMS_SecInfoArray syms_sec_info_array_from_dbg(SYMS_Arena *arena, SYMS_String8 data,
                                                        SYMS_DbgAccel *dbg);

// default vbase
SYMS_API SYMS_U64          syms_default_vbase_from_dbg(SYMS_DbgAccel *dbg);

// compilation units
SYMS_API SYMS_UnitSetAccel*syms_unit_set_accel_from_dbg(SYMS_Arena *arena, SYMS_String8 data,
                                                        SYMS_DbgAccel *dbg);
SYMS_API SYMS_U64          syms_unit_count_from_set(SYMS_UnitSetAccel *unit_set);
SYMS_API SYMS_UnitInfo     syms_unit_info_from_uid(SYMS_UnitSetAccel *unit_set, SYMS_UnitID uid);
SYMS_API SYMS_UnitNames    syms_unit_names_from_uid(SYMS_Arena *arena, SYMS_UnitSetAccel *unit_set,
                                                    SYMS_UnitID uid);

SYMS_API SYMS_UnitRangeArray syms_unit_ranges_from_set(SYMS_Arena *arena, SYMS_String8 data,
                                                       SYMS_DbgAccel *dbg, SYMS_UnitSetAccel *unit_set);

SYMS_API SYMS_UnitID       syms_uid_collated_types_from_set(SYMS_UnitSetAccel *unit_set);
SYMS_API SYMS_UnitID       syms_uid_collated_public_symbols_from_set(SYMS_UnitSetAccel *unit_set);

// symbol parsing
SYMS_API SYMS_UnitAccel*    syms_unit_accel_from_uid(SYMS_Arena *arena, SYMS_String8 data,
                                                     SYMS_DbgAccel *dbg, SYMS_UnitSetAccel *unit_set,
                                                     SYMS_UnitID uid);

SYMS_API SYMS_UnitID        syms_uid_from_unit(SYMS_UnitAccel *unit);

SYMS_API SYMS_SymbolIDArray syms_proc_sid_array_from_unit(SYMS_Arena *arena, SYMS_UnitAccel *unit);
SYMS_API SYMS_SymbolIDArray syms_var_sid_array_from_unit(SYMS_Arena *arena, SYMS_UnitAccel *unit);
SYMS_API SYMS_SymbolIDArray syms_type_sid_array_from_unit(SYMS_Arena *arena, SYMS_UnitAccel *unit);


SYMS_API SYMS_SymbolKind    syms_symbol_kind_from_sid(SYMS_String8 data, SYMS_DbgAccel *dbg,
                                                      SYMS_UnitAccel *unit, SYMS_SymbolID sid);
SYMS_API SYMS_String8       syms_symbol_name_from_sid(SYMS_Arena *arena, SYMS_String8 data,
                                                      SYMS_DbgAccel *dbg, SYMS_UnitAccel *unit,
                                                      SYMS_SymbolID sid);


SYMS_API SYMS_TypeInfo      syms_type_info_from_sid(SYMS_String8 data, SYMS_DbgAccel *dbg,
                                                    SYMS_UnitAccel *unit, SYMS_SymbolID sid);

SYMS_API SYMS_ConstInfo     syms_const_info_from_sid(SYMS_String8 data, SYMS_DbgAccel *dbg,
                                                     SYMS_UnitAccel *unit, SYMS_SymbolID sid);

// variable info
SYMS_API SYMS_USID         syms_type_from_var_sid(SYMS_String8 data, SYMS_DbgAccel *dbg,
                                                  SYMS_UnitAccel *unit, SYMS_SymbolID sid);

SYMS_API SYMS_U64          syms_voff_from_var_sid(SYMS_String8 data, SYMS_DbgAccel *dbg,
                                                  SYMS_UnitAccel *unit, SYMS_SymbolID sid);

SYMS_API SYMS_Location     syms_location_from_var_sid(SYMS_Arena *arena, SYMS_String8 data,
                                                      SYMS_DbgAccel *dbg, SYMS_UnitAccel *unit,
                                                      SYMS_SymbolID sid);
SYMS_API SYMS_LocRangeArray syms_location_ranges_from_var_sid(SYMS_Arena *arena, SYMS_String8 data,
                                                              SYMS_DbgAccel *dbg, SYMS_UnitAccel *unit,
                                                              SYMS_SymbolID sid);
SYMS_API SYMS_Location     syms_location_from_id(SYMS_Arena *arena, SYMS_String8 data, SYMS_DbgAccel *dbg,
                                                 SYMS_UnitAccel *unit, SYMS_LocID loc_id);

// member info
SYMS_API SYMS_MemsAccel*   syms_mems_accel_from_sid(SYMS_Arena *arena, SYMS_String8 data,
                                                    SYMS_DbgAccel *dbg, SYMS_UnitAccel *unit,
                                                    SYMS_SymbolID sid);

SYMS_API SYMS_U64          syms_mem_count_from_mems(SYMS_MemsAccel *mems);
SYMS_API SYMS_MemInfo      syms_mem_info_from_number(SYMS_Arena *arena, SYMS_String8 data,
                                                     SYMS_DbgAccel *dbg, SYMS_UnitAccel *unit,
                                                     SYMS_MemsAccel *mems, SYMS_U64 n);

SYMS_API SYMS_USID         syms_type_from_mem_number(SYMS_String8 data, SYMS_DbgAccel *dbg,
                                                     SYMS_UnitAccel *unit, SYMS_MemsAccel *mems,
                                                     SYMS_U64 n);
SYMS_API SYMS_SigInfo      syms_sig_info_from_mem_number(SYMS_Arena *arena, SYMS_String8 data,
                                                         SYMS_DbgAccel *dbg, SYMS_UnitAccel *unit,
                                                         SYMS_MemsAccel *mems, SYMS_U64 n);

SYMS_API SYMS_USID         syms_symbol_from_mem_number(SYMS_String8 data, SYMS_DbgAccel *dbg,
                                                       SYMS_UnitAccel *unit, SYMS_MemsAccel *mems,
                                                       SYMS_U64 n);
SYMS_API SYMS_USID         syms_containing_type_from_sid(SYMS_String8 data, SYMS_DbgAccel *dbg,
                                                         SYMS_UnitAccel *unit, SYMS_SymbolID sid);

SYMS_API SYMS_String8      syms_linkage_name_from_sid(SYMS_Arena *arena, SYMS_String8 data,
                                                      SYMS_DbgAccel *dbg, SYMS_UnitAccel *unit,
                                                      SYMS_SymbolID sid);

SYMS_API SYMS_EnumMemberArray syms_enum_member_array_from_sid(SYMS_Arena *arena, SYMS_String8 data,
                                                              SYMS_DbgAccel *dbg, SYMS_UnitAccel *unit,
                                                              SYMS_SymbolID sid);

// procedure info
SYMS_API SYMS_UnitIDAndSig syms_sig_handle_from_proc_sid(SYMS_String8 data, SYMS_DbgAccel *dbg,
                                                         SYMS_UnitAccel *unit, SYMS_SymbolID proc_sid);
SYMS_API SYMS_SigInfo      syms_sig_info_from_handle(SYMS_Arena *arena, SYMS_String8 data,
                                                     SYMS_DbgAccel *dbg, SYMS_UnitAccel *unit,
                                                     SYMS_SigHandle handle);

#define syms_proc_vranges_from_sid syms_scope_vranges_from_sid
SYMS_API SYMS_U64RangeArray syms_scope_vranges_from_sid(SYMS_Arena *arena, SYMS_String8 data,
                                                        SYMS_DbgAccel *dbg, SYMS_UnitAccel *unit,
                                                        SYMS_SymbolID sid);
SYMS_API SYMS_SymbolIDArray syms_scope_children_from_sid(SYMS_Arena *arena, SYMS_String8 data,
                                                         SYMS_DbgAccel *dbg, SYMS_UnitAccel *unit,
                                                         SYMS_SymbolID sid);

SYMS_API SYMS_Location     syms_location_from_proc_sid(SYMS_Arena *arena, SYMS_String8 data,
                                                       SYMS_DbgAccel *dbg, SYMS_UnitAccel *unit,
                                                       SYMS_SymbolID sid, SYMS_ProcLoc proc_loc);
SYMS_API SYMS_LocRangeArray syms_location_ranges_from_proc_sid(SYMS_Arena *arena, SYMS_String8 data,
                                                               SYMS_DbgAccel *dbg, SYMS_UnitAccel *unit,
                                                               SYMS_SymbolID sid, SYMS_ProcLoc proc_loc);

// signature info
SYMS_API SYMS_SigInfo      syms_sig_info_from_type_sid(SYMS_Arena *arena, SYMS_String8 data,
                                                       SYMS_DbgAccel *dbg, SYMS_UnitAccel *unit,
                                                       SYMS_SymbolID sid);

// line info
SYMS_API SYMS_String8      syms_file_name_from_id(SYMS_Arena *arena, SYMS_String8 data,
                                                  SYMS_DbgAccel *dbg, SYMS_UnitSetAccel *unit_set,
                                                  SYMS_UnitID uid, SYMS_FileID file_id);
SYMS_API SYMS_String8Array syms_file_table_from_uid(SYMS_Arena *arena, SYMS_String8 data,
                                                    SYMS_DbgAccel *dbg, SYMS_UnitSetAccel *unit_set,
                                                    SYMS_UnitID uid);
SYMS_API SYMS_LineParseOut syms_line_parse_from_uid(SYMS_Arena *arena, SYMS_String8 data,
                                                    SYMS_DbgAccel *dbg, SYMS_UnitSetAccel *unit_set,
                                                    SYMS_UnitID uid);


// name maps
SYMS_API SYMS_MapAccel*    syms_type_map_from_dbg(SYMS_Arena *arena, SYMS_String8 data,
                                                  SYMS_DbgAccel *dbg);
SYMS_API SYMS_MapAccel*    syms_unmangled_symbol_map_from_dbg(SYMS_Arena *arena,SYMS_String8 data,SYMS_DbgAccel *dbg);
SYMS_API SYMS_UnitID       syms_partner_uid_from_map(SYMS_MapAccel *map);
SYMS_API SYMS_USIDList     syms_usid_list_from_string(SYMS_Arena *arena, SYMS_String8 data,
                                                      SYMS_DbgAccel *dbg, SYMS_MapAndUnit *map_and_unit,
                                                      SYMS_String8 string);

// linker names
SYMS_API SYMS_UnitID       syms_link_names_uid(SYMS_DbgAccel *dbg);

SYMS_API SYMS_LinkMapAccel*syms_link_map_from_dbg(SYMS_Arena *arena, SYMS_String8 data,
                                                  SYMS_DbgAccel *dbg);
SYMS_API SYMS_B32          syms_link_map_is_complete(SYMS_LinkMapAccel *map);
SYMS_API SYMS_U64          syms_voff_from_link_name(SYMS_String8 data, SYMS_DbgAccel *dbg,
                                                    SYMS_LinkMapAccel *map, SYMS_UnitAccel *link_unit,
                                                    SYMS_String8 name);
SYMS_API SYMS_LinkNameRecArray syms_link_name_array_from_unit(SYMS_Arena *arena, SYMS_String8 data,
                                                              SYMS_DbgAccel *dbg, SYMS_UnitAccel *unit);

// thread vars
SYMS_API SYMS_UnitID        syms_tls_var_uid_from_dbg(SYMS_DbgAccel *dbg);
SYMS_API SYMS_SymbolIDArray syms_tls_var_sid_array_from_unit(SYMS_Arena *arena,
                                                             SYMS_UnitAccel *thread_unit);

SYMS_C_LINKAGE_END

#endif //SYMS_PARSER_H
