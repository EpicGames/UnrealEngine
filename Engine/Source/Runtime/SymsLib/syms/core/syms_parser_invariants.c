// Copyright Epic Games, Inc. All Rights Reserved.

////////////////////////////////
//~ NOTE(allen): Parser Invariants

SYMS_API SYMS_B32
syms_parser_api_invariants(void){
  SYMS_B32 result = syms_true;
  
  //  The way we write result pointers for accelerators it is a little bit easy
  // to accidentally modify syms_format_nil, this invariant is here to catch
  // whenever we make that mistake.
  SYMS_INVARIANT(result, syms_format_nil == SYMS_FileFormat_Null);
  
  finish_invariants:;
  return(result);
}

SYMS_API SYMS_B32
syms_unit_ranges_low_level_invariants(SYMS_UnitRangeArray *ranges){
  SYMS_B32 result = syms_true;
  
  SYMS_U64 count = ranges->count;
  SYMS_UnitRange *range_ptr = ranges->ranges;
  for (SYMS_U64 i = 0; i < count; i += 1, range_ptr += 1){
    // no empty ranges
    SYMS_INVARIANT(result, range_ptr->vrange.min < range_ptr->vrange.max);
    // no ranges that refer to the null unit id
    SYMS_INVARIANT(result, range_ptr->uid != 0);
  }
  
  finish_invariants:;
  return(result);
}

SYMS_API SYMS_B32
syms_unit_ranges_high_level_invariants(SYMS_UnitRangeArray *ranges, SYMS_UnitSetAccel *unit_set){
  SYMS_B32 result = syms_true;
  
  SYMS_U64 unit_count = syms_unit_count_from_set(unit_set);
  SYMS_U64 count = ranges->count;
  SYMS_UnitRange *range_ptr = ranges->ranges;
  for (SYMS_U64 i = 0; i < count; i += 1, range_ptr += 1){
    // all unit ids map to an actual unit
    SYMS_U64 n = range_ptr->uid;
    SYMS_INVARIANT(result, 0 < n && n <= unit_count);
  }
  
  finish_invariants:;
  return(result);
}

SYMS_API SYMS_B32
syms_line_table_low_level_invariants(SYMS_LineTable *line_table){
  SYMS_B32 result = syms_true;
  
  SYMS_U64 line_count = line_table->line_count;
  SYMS_U64 seq_count = line_table->sequence_count;
  
  // no sequences and no lines, or at least one of each
  SYMS_INVARIANT(result, (line_count == 0) == (seq_count == 0));
  
  if (seq_count > 0 && line_count > 0){
    // first sequence index is zero
    SYMS_INVARIANT(result, line_table->sequence_index_array[0] == 0);
    // ender sequence index is line_count
    SYMS_INVARIANT(result, line_table->sequence_index_array[seq_count] == line_count);
    
    SYMS_U64 *seq_idx_ptr = line_table->sequence_index_array;
    for (SYMS_U64 i = 0; i < seq_count; i += 1){
      SYMS_U64 first = *seq_idx_ptr;
      seq_idx_ptr += 1;
      SYMS_U64 opl = *seq_idx_ptr;
      // no empty sequences
      SYMS_INVARIANT(result, first + 1 < opl);
    }
  }
  
  finish_invariants:;
  return(result);
}

SYMS_API SYMS_B32
syms_line_table_high_level_invariants(SYMS_LineTable *line_table){
  SYMS_B32 result = syms_true;
  
  SYMS_U64 seq_count = line_table->sequence_count;
  SYMS_U64 *seq_idx_ptr = line_table->sequence_index_array;
  for (SYMS_U64 i = 0; i < seq_count; i += 1){
    SYMS_U64 first = *seq_idx_ptr;
    seq_idx_ptr += 1;
    SYMS_U64 opl = *seq_idx_ptr;
    SYMS_U64 last = opl - 1;
    SYMS_Line *line = line_table->line_array + first;
    for (SYMS_U64 j = first; j < last; j += 1, line += 1){
      // line virtual offsets in non-decreasing order
      SYMS_INVARIANT(result, line->voff <= (line + 1)->voff);
    }
  }
  
  finish_invariants:;
  return(result);
}
