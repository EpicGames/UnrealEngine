// Copyright Epic Games, Inc. All Rights Reserved.
/* date = August 17th 2021 3:47 pm */

#ifndef SYMS_PARSER_INVARIANTS_H
#define SYMS_PARSER_INVARIANTS_H

////////////////////////////////
//~ NOTE(allen): Parser Invariants

// Note on invariant "rankings"
// "low_level"  - only checks invariants that we have control over; does not include invariants
//                that we *expect* to be true of input files, but that could fail with a malformed input
// "high_level" - checks invariants that depend on the input file being well formed; skips checks for
//                invariants that "low_level" will already handle

// invariants that apply to the global state of the syms parser API.
SYMS_API SYMS_B32 syms_parser_api_invariants(void);

// invariants that apply to the unit ranges returned by the syms parser API.
SYMS_API SYMS_B32 syms_unit_ranges_low_level_invariants(SYMS_UnitRangeArray *ranges);
SYMS_API SYMS_B32 syms_unit_ranges_high_level_invariants(SYMS_UnitRangeArray *ranges, SYMS_UnitSetAccel *unit_set);

// invariants that apply to the line tables returned by the syms parser API;
SYMS_API SYMS_B32 syms_line_table_low_level_invariants(SYMS_LineTable *line_table);
SYMS_API SYMS_B32 syms_line_table_high_level_invariants(SYMS_LineTable *line_table);

#endif //SYMS_PARSER_INVARIANTS_H
