// Copyright Epic Games, Inc. All Rights Reserved.

#ifndef SYMS_UNWIND_ELF_X64_C
#define SYMS_UNWIND_ELF_X64_C

////////////////////////////////
// NOTE(allen): Generated Code

#include "syms/core/generated/syms_meta_dwarf_cfi.c"

////////////////////////////////
// NOTE(allen): ELF-x64 Unwind Function

SYMS_API SYMS_UnwindResult
syms_unwind_elf_x64(SYMS_String8 bin_data, SYMS_ElfBinAccel *bin, SYMS_U64 bin_base,
                    SYMS_MemoryView *memview, SYMS_U64 stack_pointer, SYMS_DwRegsX64 *regs){
  SYMS_LogOpen(SYMS_LogFeature_DwarfUnwind, 0, log);
  SYMS_ArenaTemp scratch = syms_get_scratch(0, 0);
  
  syms_unwind_elf_x64__init();
  
  SYMS_UnwindResult result = {0};
  
  //- rebase
  SYMS_U64 default_vbase = syms_elf_default_vbase_from_bin(bin);
  SYMS_U64 rebase_voff_to_vaddr = (bin_base - default_vbase);
  
  //- get ip register values
  SYMS_U64 ip_value = regs->rip;
  SYMS_U64 ip_voff = ip_value - rebase_voff_to_vaddr;
  
  //- get sections
  SYMS_ElfSection *sec_text = syms_elf_sec_from_bin_name__unstable(bin, syms_str8_lit(".text")); // TODO(nick): What if ELF has two sections with instructions and pointer is ecnoded relative to .text2?
  SYMS_ElfSection *sec_frame_info = syms_elf_sec_from_bin_name__unstable(bin, syms_str8_lit(".eh_frame"));
  SYMS_ElfSection *sec_extra_data = syms_elf_sec_from_bin_name__unstable(bin, syms_str8_lit(".eh_frame_hdr"));
  
  //- check sections
  SYMS_B32 has_needed_sections = (sec_text != 0 && sec_frame_info != 0);
  if (!has_needed_sections){
    SYMS_Log("Does not have needed sections\n");
    result.dead = syms_true;
  }
  
  //- get frame info range
  void *frame_base = 0;
  SYMS_U64Range frame_range = {0};
  if (has_needed_sections){
    frame_base = bin_data.str + sec_frame_info->file_range.min;
    frame_range.max = sec_frame_info->file_range.max - sec_frame_info->file_range.min;
  }
  
  //- section vaddrs
  SYMS_U64 text_base_vaddr = 0;
  SYMS_U64 frame_base_voff = 0;
  if (has_needed_sections){
    text_base_vaddr = sec_text->virtual_range.min + rebase_voff_to_vaddr;
    frame_base_voff = sec_frame_info->virtual_range.min;
  }
  SYMS_U64 data_base_vaddr = 0;
  if (sec_extra_data != 0){
    data_base_vaddr = sec_extra_data->virtual_range.min + rebase_voff_to_vaddr;
  }
  
  //- find cfi records
  SYMS_DwCFIRecords cfi_recs = {0};
  if (has_needed_sections){
    SYMS_DwEhPtrCtx ptr_ctx = {0};
    // TODO(allen): name?
    ptr_ctx.raw_base_vaddr = frame_base_voff;
    ptr_ctx.text_vaddr = text_base_vaddr;
    ptr_ctx.data_vaddr = data_base_vaddr;
    ptr_ctx.func_vaddr = 0;
    if (sec_extra_data != 0){
      cfi_recs = syms_unwind_elf_x64__eh_frame_hdr_from_ip(bin_data.str, sec_extra_data->file_range, sec_frame_info->file_range, &ptr_ctx, ip_voff);
    }
    else{
      cfi_recs = syms_unwind_elf_x64__eh_frame_cfi_from_ip__sloppy(frame_base, frame_range, &ptr_ctx, ip_voff);
    }
  }
  
  //- check cfi records
  if (!cfi_recs.valid){
    SYMS_Log("No matching CFI record\n");
    result.dead = syms_true;
  }
  
  //- cfi machine setup
  SYMS_DwCFIMachine machine = {0};
  if (cfi_recs.valid){
    SYMS_DwEhPtrCtx ptr_ctx = {0};
    // TODO(allen): name?
    ptr_ctx.raw_base_vaddr = frame_base_voff;
    ptr_ctx.text_vaddr = text_base_vaddr;
    ptr_ctx.data_vaddr = data_base_vaddr;
    //  NOTE: It's not super clear how to set up this member.
    ptr_ctx.func_vaddr = cfi_recs.fde.ip_voff_range.min + rebase_voff_to_vaddr;
    machine = syms_unwind_elf_x64__machine_make(SYMS_UNWIND_ELF_X64__REG_SLOT_COUNT, &cfi_recs.cie, &ptr_ctx);
  }
  
  //- initial row
  SYMS_DwCFIRow *init_row = 0;
  if (cfi_recs.valid){
    SYMS_U64Range init_cfi_range = cfi_recs.cie.cfi_range;
    SYMS_DwCFIRow *row = syms_unwind_elf_x64__row_alloc(scratch.arena, machine.cells_per_row);
    if (syms_unwind_elf_x64__machine_run_to_ip(frame_base, init_cfi_range, &machine, SYMS_U64_MAX, row)){
      init_row = row;
    }
    if (init_row == 0){
      SYMS_Log("Could not decode initial row\n");
      result.dead = syms_true;
    }
  }
  
  //- main row
  SYMS_DwCFIRow *main_row = 0;
  if (init_row != 0){
    // upgrade machine with new equipment
    syms_unwind_elf_x64__machine_equip_initial_row(&machine, init_row);
    syms_unwind_elf_x64__machine_equip_fde_ip(&machine, cfi_recs.fde.ip_voff_range.min);
    
    // decode main row
    SYMS_U64Range main_cfi_range = cfi_recs.fde.cfi_range;
    SYMS_DwCFIRow *row = syms_unwind_elf_x64__row_alloc(scratch.arena, machine.cells_per_row);
    if (syms_unwind_elf_x64__machine_run_to_ip(frame_base, main_cfi_range, &machine, ip_value, row)){
      main_row = row;
    }
    if (main_row == 0){
      SYMS_Log("Could not decode main row\n");
      result.dead = syms_true;
    }
  }
  
  //- apply main row to modify the registers
  if (main_row != 0){
    result = syms_unwind_elf_x64__apply_rules(bin_data, main_row, text_base_vaddr, memview, stack_pointer, regs);
  }
  
  syms_release_scratch(scratch);
  SYMS_LogClose(log);
  
  return(result);
}

SYMS_API SYMS_UnwindResult
syms_unwind_elf_x64__apply_rules(SYMS_String8 bin_data, SYMS_DwCFIRow *row, SYMS_U64 text_base_vaddr,
                                 SYMS_MemoryView *memview, SYMS_U64 stack_pointer, SYMS_DwRegsX64 *regs){
  SYMS_LogOpen(SYMS_LogFeature_DwarfCFIApply, 0, log);
  
  SYMS_UnwindResult result = {0};
  SYMS_U64 missed_read_addr = 0;
  
  //- setup a dwarf expression machine
  SYMS_DwExprMachineConfig dwexpr_config = {0};
  dwexpr_config.max_step_count = 0xFFFF;
  dwexpr_config.memview = memview;
  dwexpr_config.regs = regs;
  dwexpr_config.text_section_base = &text_base_vaddr;
  
  //- compute cfa
  SYMS_U64 cfa = 0;
  switch (row->cfa_cell.rule){
    case SYMS_DwCFICFARule_REGOFF:
    {
      SYMS_Log("CFA rule: REGOFF(%llu,0x%llx)\n", row->cfa_cell.reg_idx, row->cfa_cell.offset);
      
      // TODO(allen): have we done anything to gaurantee reg_idx here?
      SYMS_U64 reg_idx = row->cfa_cell.reg_idx;
      
      // is this a roll-over CFA?
      SYMS_B32 is_roll_over_cfa = syms_false;
      if (reg_idx == SYMS_DwRegX64_RSP){
        SYMS_DwCFIRegisterRule rule = row->cells[reg_idx].rule;
        if (rule == SYMS_DwCFIRegisterRule_UNDEFINED ||
            rule == SYMS_DwCFIRegisterRule_SAME_VALUE){
          SYMS_Log("CFA rollover\n");
          is_roll_over_cfa = syms_true;
        }
      }
      
      // compute cfa
      if (is_roll_over_cfa){
        cfa = stack_pointer + row->cfa_cell.offset;
      }
      else{
        cfa = regs->r[reg_idx] + row->cfa_cell.offset;
      }
    }break;
    
    case SYMS_DwCFICFARule_EXPR:
    {
      SYMS_Log("CFA rule: EXPR\n");
      SYMS_U64Range expr_range = row->cfa_cell.expr;
      SYMS_DwLocation location = syms_dw_expr__eval(0, bin_data.str, expr_range, &dwexpr_config);
      if (location.non_piece_loc.kind == SYMS_DwSimpleLocKind_Fail &&
          location.non_piece_loc.fail_kind == SYMS_DwLocFailKind_MissingMemory){
        missed_read_addr = location.non_piece_loc.fail_data;
        goto error_out;
      }
      if (location.non_piece_loc.kind == SYMS_DwSimpleLocKind_Address){
        cfa = location.non_piece_loc.addr;
      }
    }break;
  }
  SYMS_Log("CFA value: 0x%llx\n", cfa);
  
  //- compute registers
  {
    SYMS_DwCFICell *cell = row->cells;
    SYMS_DwRegsX64 new_regs = {0};
    for (SYMS_U64 i = 0; i < SYMS_UNWIND_ELF_X64__REG_SLOT_COUNT; i += 1, cell += 1){
      SYMS_Log("REG[%llu] RULE: ", i);
      
      // compute value
      SYMS_U64 v = 0;
      switch (cell->rule){
        default:
        {
          SYMS_Log("UNEXPECTED-RULE\n");
        }break;
        
        case SYMS_DwCFIRegisterRule_UNDEFINED:
        {
          SYMS_Log("UNDEFINED\n");
        }break;
        
        case SYMS_DwCFIRegisterRule_SAME_VALUE:
        {
          SYMS_Log("SAME_VALUE\n");
          v = regs->r[i];
        }break;
        
        case SYMS_DwCFIRegisterRule_OFFSET:
        {
          SYMS_Log("OFFSET [cfa + %llx]\n", cell->n);
          SYMS_U64 addr = cfa + cell->n;
          if (!syms_memory_view_read_struct(memview, addr, &v)){
            missed_read_addr = addr;
            goto error_out;
          }
        }break;
        
        case SYMS_DwCFIRegisterRule_VAL_OFFSET:
        {
          SYMS_Log("VAL_OFFSET (cfa + %llx)\n", cell->n);
          v = cfa + cell->n;
        }break;
        
        case SYMS_DwCFIRegisterRule_REGISTER:
        {
          SYMS_Log("REGISTER r[%llu]\n", cell->n);
          v = regs->r[i];
        }break;
        
        case SYMS_DwCFIRegisterRule_EXPRESSION:
        {
          SYMS_Log("EXPRESSION\n");
          
          SYMS_U64Range expr_range = cell->expr;
          SYMS_U64 addr = 0;
          SYMS_DwLocation location = syms_dw_expr__eval(0, bin_data.str, expr_range, &dwexpr_config);
          if (location.non_piece_loc.kind == SYMS_DwSimpleLocKind_Fail &&
              location.non_piece_loc.fail_kind == SYMS_DwLocFailKind_MissingMemory){
            missed_read_addr = location.non_piece_loc.fail_data;
            goto error_out;
          }
          if (location.non_piece_loc.kind == SYMS_DwSimpleLocKind_Address){
            addr = location.non_piece_loc.addr;
          }
          if (!syms_memory_view_read_struct(memview, addr, &v)){
            missed_read_addr = addr;
            goto error_out;
          }
        }break;
        
        case SYMS_DwCFIRegisterRule_VAL_EXPRESSION:
        {
          SYMS_Log("VAL_EXPRESSION\n");
          
          SYMS_U64Range expr_range = cell->expr;
          SYMS_DwLocation location = syms_dw_expr__eval(0, bin_data.str, expr_range, &dwexpr_config);
          if (location.non_piece_loc.kind == SYMS_DwSimpleLocKind_Fail &&
              location.non_piece_loc.fail_kind == SYMS_DwLocFailKind_MissingMemory){
            missed_read_addr = location.non_piece_loc.fail_data;
            goto error_out;
          }
          if (location.non_piece_loc.kind == SYMS_DwSimpleLocKind_Address){
            v = location.non_piece_loc.addr;
          }
        }break;
      }
      
      // commit value to output slot
      new_regs.r[i] = v;
      
      SYMS_Log("REG[%llu] VAL: 0x%llx\n", i, v);
    }
    
    //- commit all new regs
    syms_memmove(regs, &new_regs, sizeof(new_regs));
  }
  
  //- save new stack pointer
  result.stack_pointer = cfa;
  
  error_out:;
  if (missed_read_addr != 0){
    SYMS_Log("Memory read miss: 0x%llx\n", missed_read_addr);
    syms_unwind_result_missed_read(&result, missed_read_addr);
  }
  
  SYMS_LogClose(log);
  
  return(result);
}


////////////////////////////////
// NOTE(allen): ELF-x64 Unwind Helper Functions

SYMS_API void
syms_unwind_elf_x64__init(void){
  SYMS_LOCAL SYMS_B32 did_init = syms_false;
  
  if (!did_init){
    did_init = syms_true;
    
    // control bits tables
    syms_unwind_elf__cfa_control_bits_kind1[SYMS_DwCFA_NOP              ] = 0x000;
    syms_unwind_elf__cfa_control_bits_kind1[SYMS_DwCFA_SET_LOC          ] = 0x809;
    syms_unwind_elf__cfa_control_bits_kind1[SYMS_DwCFA_ADVANCE_LOC1     ] = 0x801;
    syms_unwind_elf__cfa_control_bits_kind1[SYMS_DwCFA_ADVANCE_LOC2     ] = 0x802;
    syms_unwind_elf__cfa_control_bits_kind1[SYMS_DwCFA_ADVANCE_LOC4     ] = 0x804;
    syms_unwind_elf__cfa_control_bits_kind1[SYMS_DwCFA_OFFSET_EXT       ] = 0x2AA;
    syms_unwind_elf__cfa_control_bits_kind1[SYMS_DwCFA_RESTORE_EXT      ] = 0x20A;
    syms_unwind_elf__cfa_control_bits_kind1[SYMS_DwCFA_UNDEFINED        ] = 0x20A;
    syms_unwind_elf__cfa_control_bits_kind1[SYMS_DwCFA_SAME_VALUE       ] = 0x20A;
    syms_unwind_elf__cfa_control_bits_kind1[SYMS_DwCFA_REGISTER         ] = 0x6AA;
    syms_unwind_elf__cfa_control_bits_kind1[SYMS_DwCFA_REMEMBER_STATE   ] = 0x000;
    syms_unwind_elf__cfa_control_bits_kind1[SYMS_DwCFA_RESTORE_STATE    ] = 0x000;
    syms_unwind_elf__cfa_control_bits_kind1[SYMS_DwCFA_DEF_CFA          ] = 0x2AA;
    syms_unwind_elf__cfa_control_bits_kind1[SYMS_DwCFA_DEF_CFA_REGISTER ] = 0x20A;
    syms_unwind_elf__cfa_control_bits_kind1[SYMS_DwCFA_DEF_CFA_OFFSET   ] = 0x00A;
    syms_unwind_elf__cfa_control_bits_kind1[SYMS_DwCFA_DEF_CFA_EXPR     ] = 0x00A;
    syms_unwind_elf__cfa_control_bits_kind1[SYMS_DwCFA_EXPR             ] = 0x2AA;
    syms_unwind_elf__cfa_control_bits_kind1[SYMS_DwCFA_OFFSET_EXT_SF    ] = 0x2BA;
    syms_unwind_elf__cfa_control_bits_kind1[SYMS_DwCFA_DEF_CFA_SF       ] = 0x2BA;
    syms_unwind_elf__cfa_control_bits_kind1[SYMS_DwCFA_DEF_CFA_OFFSET_SF] = 0x00B;
    syms_unwind_elf__cfa_control_bits_kind1[SYMS_DwCFA_VAL_OFFSET       ] = 0x2AA;
    syms_unwind_elf__cfa_control_bits_kind1[SYMS_DwCFA_VAL_OFFSET_SF    ] = 0x2BA;
    syms_unwind_elf__cfa_control_bits_kind1[SYMS_DwCFA_VAL_EXPR         ] = 0x2AA;
    
    syms_unwind_elf__cfa_control_bits_kind2[SYMS_DwCFA_ADVANCE_LOC >> 6] = 0x800;
    syms_unwind_elf__cfa_control_bits_kind2[SYMS_DwCFA_OFFSET      >> 6] = 0x10A;
    syms_unwind_elf__cfa_control_bits_kind2[SYMS_DwCFA_RESTORE     >> 6] = 0x100;
  }
}

SYMS_API SYMS_U64
syms_unwind_elf_x64__parse_pointer(void *frame_base, SYMS_U64Range frame_range,
                                   SYMS_DwEhPtrCtx *ptr_ctx, SYMS_DwEhPtrEnc encoding,
                                   SYMS_U64 off, SYMS_U64 *ptr_out){
  
  // aligned offset
  SYMS_U64 pointer_off = off;
  if (encoding == SYMS_DwEhPtrEnc_ALIGNED){
    pointer_off = (off + 7) & ~7; // TODO(nick): align to 4 bytes when we parse x86 ELF binary
    encoding = SYMS_DwEhPtrEnc_PTR;
  }
  
  // decode pointer value
  SYMS_U64 size_param = 0;
  SYMS_U64 after_pointer_off = 0;
  SYMS_U64 raw_pointer = 0;
  switch (encoding & SYMS_DwEhPtrEnc_TYPE_MASK){
    default:break;
    
    case SYMS_DwEhPtrEnc_PTR   :size_param = 8; goto ufixed;
    case SYMS_DwEhPtrEnc_UDATA2:size_param = 2; goto ufixed;
    case SYMS_DwEhPtrEnc_UDATA4:size_param = 4; goto ufixed;
    case SYMS_DwEhPtrEnc_UDATA8:size_param = 8; goto ufixed;
    ufixed:
    {
      syms_based_range_read(frame_base, frame_range, pointer_off, size_param, &raw_pointer);
      after_pointer_off = pointer_off + size_param;
    }break;
    
    // TODO(nick): SIGNED is actually just a flag that indicates this int is negavite.
    // There shouldn't be a read when for SIGNED.
    // For instance, (SYMS_Dw_EhPtrEnc_UDATA2 | SYMS_DwEhPtrEnc_SIGNED) == SYMS_DwEhPtrEnc_SDATA etc.
    case SYMS_DwEhPtrEnc_SIGNED:size_param = 8; goto sfixed; 
    
    case SYMS_DwEhPtrEnc_SDATA2:size_param = 2; goto sfixed;
    case SYMS_DwEhPtrEnc_SDATA4:size_param = 4; goto sfixed;
    case SYMS_DwEhPtrEnc_SDATA8:size_param = 8; goto sfixed;
    sfixed:
    {
      syms_based_range_read(frame_base, frame_range, pointer_off, size_param, &raw_pointer);
      after_pointer_off = pointer_off + size_param;
      // sign extension
      SYMS_U64 sign_bit = size_param*8 - 1;
      if ((raw_pointer >> sign_bit) != 0){
        raw_pointer |= (~(1 << sign_bit)) + 1;
      }
    }break;
    
    case SYMS_DwEhPtrEnc_ULEB128:
    {
      SYMS_U64 size = syms_based_range_read_uleb128(frame_base, frame_range, pointer_off, &raw_pointer);
      after_pointer_off = pointer_off + size;
    }break;
    
    case SYMS_DwEhPtrEnc_SLEB128:
    {
      SYMS_U64 size = syms_based_range_read_sleb128(frame_base, frame_range, pointer_off,
                                                    (SYMS_S64*)&raw_pointer);
      after_pointer_off = pointer_off + size;
    }break;
  }
  
  // apply relative bases
  SYMS_U64 pointer = raw_pointer;
  if (pointer != 0){
    switch (encoding & SYMS_DwEhPtrEnc_MODIF_MASK){
      case SYMS_DwEhPtrEnc_PCREL:
      {
        pointer = ptr_ctx->raw_base_vaddr + frame_range.min + off + raw_pointer;
      }break;
      case SYMS_DwEhPtrEnc_TEXTREL:
      {
        pointer = ptr_ctx->text_vaddr + raw_pointer;
      }break;
      case SYMS_DwEhPtrEnc_DATAREL:
      {
        pointer = ptr_ctx->data_vaddr + raw_pointer;
      }break;
      case SYMS_DwEhPtrEnc_FUNCREL:
      {
        // TODO(allen): find some strong indication of how to actually fill func_vaddr
        pointer = ptr_ctx->func_vaddr + raw_pointer;
      }break;
    }
  }
  
  // return
  *ptr_out = pointer;
  SYMS_U64 result = after_pointer_off - off;
  return(result);
}

//- eh_frame parsing

SYMS_API void
syms_unwind_elf_x64__eh_frame_parse_cie(void *base, SYMS_U64Range range, SYMS_DwEhPtrCtx *ptr_ctx,
                                        SYMS_U64 off, SYMS_DwCIEUnpacked *cie_out){
  syms_memzero_struct(cie_out);
  
  // get version
  SYMS_U64 version_off = off;
  SYMS_U8 version = 0;
  syms_based_range_read(base, range, version_off, 1, &version);
  
  // check version
  if (version == 1 || version == 3){
    
    // read augmentation
    SYMS_U64 augmentation_off = version_off + 1;
    SYMS_String8 augmentation = syms_based_range_read_string(base, range, augmentation_off);
    
    // read code align
    SYMS_U64 code_align_factor_off = augmentation_off + augmentation.size + 1;
    SYMS_U64 code_align_factor = 0;
    SYMS_U64 code_align_factor_size = syms_based_range_read_uleb128(base, range,
                                                                    code_align_factor_off, &code_align_factor);
    
    // read data align
    SYMS_U64 data_align_factor_off = code_align_factor_off + code_align_factor_size;
    SYMS_S64 data_align_factor = 0;
    SYMS_U64 data_align_factor_size = syms_based_range_read_sleb128(base, range,
                                                                    data_align_factor_off, &data_align_factor);
    
    // return address register
    SYMS_U64 ret_addr_reg_off = data_align_factor_off + data_align_factor_size;
    SYMS_U64 after_ret_addr_reg_off = 0;
    SYMS_U64 ret_addr_reg = 0;
    if (version == 1){
      syms_based_range_read(base, range, ret_addr_reg_off, 1, &ret_addr_reg);
      after_ret_addr_reg_off = ret_addr_reg_off + 1;
    }
    else{
      SYMS_U64 ret_addr_reg_size = syms_based_range_read_uleb128(base, range,
                                                                 ret_addr_reg_off, &ret_addr_reg);
      after_ret_addr_reg_off = ret_addr_reg_off + ret_addr_reg_size;
    }
    
    // TODO(nick): 
    // Handle "eh" param, it indicates presence of EH Data field.
    // On 32bit arch it is a 4-byte and on 64-bit 8-byte value.
    // Reference: https://refspecs.linuxfoundation.org/LSB_3.0.0/LSB-PDA/LSB-PDA/ehframechpt.html
    // Reference doc doesn't clarify structure for EH Data though
    
    // check for augmentation data
    SYMS_U64 aug_size_off = after_ret_addr_reg_off;
    SYMS_U64 after_aug_size_off = after_ret_addr_reg_off;
    SYMS_B8  has_augmentation_size = syms_false;
    SYMS_U64 augmentation_size = 0;
    if (augmentation.size > 0 && augmentation.str[0] == 'z'){
      has_augmentation_size = syms_true;
      SYMS_U64 aug_size_size = syms_based_range_read_uleb128(base, range, aug_size_off, &augmentation_size);
      after_aug_size_off += aug_size_size;
    }
    
    // read augmentation data
    SYMS_U64 aug_data_off = after_aug_size_off;
    SYMS_U64 after_aug_data_off = after_aug_size_off;
    
    SYMS_DwEhPtrEnc lsda_encoding = SYMS_DwEhPtrEnc_OMIT;
    SYMS_U64 handler_ip = 0;
    SYMS_DwEhPtrEnc addr_encoding = SYMS_DwEhPtrEnc_UDATA8;
    
    if (has_augmentation_size > 0){
      SYMS_U64 aug_data_cursor = aug_data_off;
      for (SYMS_U8 *ptr = augmentation.str + 1, *opl = augmentation.str + augmentation.size;
           ptr < opl;
           ptr += 1){
        switch (*ptr){
          case 'L':
          {
            syms_based_range_read_struct(base, range, aug_data_cursor, &lsda_encoding);
            aug_data_cursor += sizeof(lsda_encoding);
          }break;
          
          case 'P':
          {
            SYMS_DwEhPtrEnc handler_encoding = SYMS_DwEhPtrEnc_OMIT;
            syms_based_range_read_struct(base, range, aug_data_cursor, &handler_encoding);
            
            SYMS_U64 ptr_off = aug_data_cursor + sizeof(handler_encoding);
            SYMS_U64 ptr_size = syms_unwind_elf_x64__parse_pointer(base, range,
                                                                   ptr_ctx, handler_encoding,
                                                                   ptr_off, &handler_ip);
            aug_data_cursor = ptr_off + ptr_size;
          }break;
          
          case 'R':
          {
            syms_based_range_read_struct(base, range, aug_data_cursor, &addr_encoding);
            aug_data_cursor += sizeof(addr_encoding);
            
          }break;
          
          default:
          {
            goto dbl_break_aug;
          }break;
        }
      }
      dbl_break_aug:;
      after_aug_data_off = aug_data_cursor;
    }
    
    // cfi range
    SYMS_U64 cfi_off = range.min + after_aug_data_off;
    SYMS_U64 cfi_size = 0;
    if (range.max > cfi_off){
      cfi_size = range.max - cfi_off;
    }
    
    // commit values to out
    cie_out->version = version;
    cie_out->lsda_encoding = lsda_encoding;
    cie_out->addr_encoding = addr_encoding;
    cie_out->has_augmentation_size = has_augmentation_size;
    cie_out->augmentation_size = augmentation_size;
    cie_out->augmentation = augmentation;
    cie_out->code_align_factor = code_align_factor;
    cie_out->data_align_factor = data_align_factor;
    cie_out->ret_addr_reg = ret_addr_reg;
    cie_out->handler_ip = handler_ip;
    cie_out->cfi_range.min = cfi_off;
    cie_out->cfi_range.max = cfi_off + cfi_size;
  }
}

SYMS_API void
syms_unwind_elf_x64__eh_frame_parse_fde(void *base, SYMS_U64Range range, SYMS_DwEhPtrCtx *ptr_ctx,
                                        SYMS_DwCIEUnpacked *cie, SYMS_U64 off, SYMS_DwFDEUnpacked *fde_out){
  // pull out pointer encoding field
  SYMS_DwEhPtrEnc ptr_enc = cie->addr_encoding;
  
  // ip first
  SYMS_U64 ip_first_off = off;
  SYMS_U64 ip_first = 0;
  SYMS_U64 ip_first_size = syms_unwind_elf_x64__parse_pointer(base, range, ptr_ctx, ptr_enc,
                                                              ip_first_off, &ip_first);
  
  // ip range size
  SYMS_U64 ip_range_size_off = ip_first_off + ip_first_size;
  SYMS_U64 ip_range_size = 0;
  SYMS_U64 ip_range_size_size = syms_unwind_elf_x64__parse_pointer(base, range, ptr_ctx,
                                                                   ptr_enc & SYMS_DwEhPtrEnc_TYPE_MASK,
                                                                   ip_range_size_off, &ip_range_size);
  
  // augmentation data
  SYMS_U64 aug_data_off = ip_range_size_off + ip_range_size_size;
  SYMS_U64 after_aug_data_off = aug_data_off;
  SYMS_U64 lsda_ip = 0;
  
  if (cie->has_augmentation_size){
    // augmentation size
    SYMS_U64 augmentation_size = 0;
    SYMS_U64 aug_size_size = syms_based_range_read_uleb128(base, range, aug_data_off, &augmentation_size);
    SYMS_U64 after_aug_size_off = aug_data_off + aug_size_size;
    
    // extract lsda (only thing that can actually be in FDE's augmentation data as far as we know)
    SYMS_DwEhPtrEnc lsda_encoding = cie->lsda_encoding;
    if (lsda_encoding != SYMS_DwEhPtrEnc_OMIT){
      SYMS_U64 lsda_off = after_aug_size_off;
      syms_unwind_elf_x64__parse_pointer(base, range, ptr_ctx, lsda_encoding, lsda_off, &lsda_ip);
    }
    
    // set offset at end of augmentation data
    after_aug_data_off = after_aug_size_off + augmentation_size;
  }
  
  // cfi range
  SYMS_U64 cfi_off = range.min + after_aug_data_off;
  SYMS_U64 cfi_size = 0;
  if (range.max > cfi_off){
    cfi_size = range.max - cfi_off;
  }
  
  // commit values to out
  fde_out->ip_voff_range.min = ip_first;
  fde_out->ip_voff_range.max = ip_first + ip_range_size;
  fde_out->lsda_ip = lsda_ip;
  fde_out->cfi_range.min = cfi_off;
  fde_out->cfi_range.max = cfi_off + cfi_size;
}

SYMS_API SYMS_DwCFIRecords
syms_unwind_elf_x64__eh_frame_cfi_from_ip__sloppy(void *base, SYMS_U64Range range,
                                                  SYMS_DwEhPtrCtx *ptr_ctx, SYMS_U64 ip_voff){
  SYMS_LogOpen(SYMS_LogFeature_DwarfCFILookup, 0, log);
  
  SYMS_DwCFIRecords result = {0};
  
  SYMS_ArenaTemp scratch = syms_get_scratch(0, 0);
  SYMS_DwCIEUnpackedNode *cie_first = 0;
  SYMS_DwCIEUnpackedNode *cie_last = 0;
  
  SYMS_U64 cursor = 0;
  for (;;){
    // CIE/FDE size
    SYMS_U64 rec_off = cursor;
    SYMS_U64 after_rec_size_off = 0;
    SYMS_U64 rec_size = 0;
    
    {
      syms_based_range_read(base, range, rec_off, 4, &rec_size);
      after_rec_size_off = 4;
      if (rec_size == SYMS_U32_MAX){
        syms_based_range_read(base, range, rec_off + 4, 8, &rec_size);
        after_rec_size_off = 12;
      }
    }
    
    // zero size is the end of the loop
    if (rec_size == 0){
      break;
    }
    
    // compute end offset
    SYMS_U64 rec_opl = rec_off + after_rec_size_off + rec_size;
    
    // sub-range the rest of the reads
    SYMS_U64Range rec_range = {0};
    rec_range.min = range.min + rec_off;
    rec_range.max = range.min + rec_opl;
    
    // discriminator
    SYMS_U64 discrim_off = after_rec_size_off;
    SYMS_U32 discrim = 0;
    syms_based_range_read(base, rec_range, discrim_off, 4, &discrim);
    
    SYMS_U64 after_discrim_off = discrim_off + 4;
    
    // CIE
    if (discrim == 0){
      SYMS_DwCIEUnpackedNode *node = syms_push_array(scratch.arena, SYMS_DwCIEUnpackedNode, 1);
      syms_unwind_elf_x64__eh_frame_parse_cie(base, rec_range, ptr_ctx, after_discrim_off, &node->cie);
      if (node->cie.version != 0){
        SYMS_QueuePush(cie_first, cie_last, node);
        node->offset = range.min + rec_off;
      }
    }
    
    // FDE
    else{
      // compute cie offset
      SYMS_U64 cie_offset = rec_range.min + discrim_off - discrim;
      
      // get cie node
      SYMS_DwCIEUnpackedNode *cie_node = 0;
      for (SYMS_DwCIEUnpackedNode *node = cie_first;
           node != 0;
           node = node->next){
        if (node->offset == cie_offset){
          cie_node = node;
          break;
        }
      }
      
      // parse fde
      SYMS_DwFDEUnpacked fde = {0};
      if (cie_node != 0){
        syms_unwind_elf_x64__eh_frame_parse_fde(base, rec_range, ptr_ctx, &cie_node->cie, after_discrim_off, &fde);
      }
      
      if (fde.ip_voff_range.min <= ip_voff && ip_voff < fde.ip_voff_range.max){
        SYMS_Log("CIE/FDE match {CIE:0x%llx, FDE:0x%llx}\n", cie_offset, rec_range.min);
        result.valid = syms_true;
        result.cie = cie_node->cie;
        result.fde = fde;
        break;
      }
    }
    
    // advance cursor
    cursor = rec_opl;
  }
  
  syms_release_scratch(scratch);
  
  SYMS_LogClose(log);
  
  return(result);
}

SYMS_API SYMS_U64
syms_search_eh_frame_hdr__linear(void *base, SYMS_U64Range range, SYMS_DwEhPtrCtx *ptr_ctx, SYMS_U64 location)
{
  // Table contains only addresses for first instruction in a function and we cannot
  // guarantee that result is FDE that corresponds to the input location. 
  // So input location must be cheked against range from FDE header again.
  
  SYMS_U64 closest_location = SYMS_U64_MAX;
  SYMS_U64 closest_address  = SYMS_U64_MAX;
  
  SYMS_U64 read_offset = 0;
  
  SYMS_U8 version = 0;
  read_offset += syms_based_range_read_struct(base, range, read_offset, &version);
  
  if (version == 1){
#if 0
    SYMS_DwEhPtrCtx ptr_ctx; syms_memzero_struct(&ptr_ctx);
    // Set this to base address of .eh_frame_hdr. Entries are relative
    // to this section for some reason.
    ptr_ctx.data_vaddr = range.min;
    // If input location is VMA then set this to address of .text. 
    // Pointer parsing function will adjust "init_location" to correct VMA.
    ptr_ctx.text_vaddr = 0; 
#endif
    
    SYMS_DwEhPtrEnc eh_frame_ptr_enc = 0, fde_count_enc = 0, table_enc = 0;
    read_offset += syms_based_range_read_struct(base, range, read_offset, &eh_frame_ptr_enc);
    read_offset += syms_based_range_read_struct(base, range, read_offset, &fde_count_enc);
    read_offset += syms_based_range_read_struct(base, range, read_offset, &table_enc);
    
    SYMS_U64 eh_frame_ptr = 0, fde_count = 0;
    read_offset += syms_unwind_elf_x64__parse_pointer(base, range, ptr_ctx, eh_frame_ptr_enc, read_offset, &eh_frame_ptr);
    read_offset += syms_unwind_elf_x64__parse_pointer(base, range, ptr_ctx, fde_count_enc, read_offset, &fde_count);
    
    for (SYMS_U64 fde_idx = 0; fde_idx < fde_count; ++fde_idx){
      SYMS_U64 init_location = 0, address = 0;
      read_offset += syms_unwind_elf_x64__parse_pointer(base, range, ptr_ctx, table_enc, read_offset, &init_location);
      read_offset += syms_unwind_elf_x64__parse_pointer(base, range, ptr_ctx, table_enc, read_offset, &address);
      
      SYMS_S64 current_delta = (SYMS_S64)(location - init_location);
      SYMS_S64 closest_delta = (SYMS_S64)(location - closest_location);
      if (0 <= current_delta && current_delta < closest_delta){
        closest_location = init_location;
        closest_address  = address;
      }
    }
  }
  
  // address where to find corresponding FDE, this is an absolute offset
  // into the image file.
  return closest_address;
}

SYMS_API SYMS_DwCFIRecords
syms_unwind_elf_x64__eh_frame_hdr_from_ip(void *base, SYMS_U64Range eh_frame_hdr_range, SYMS_U64Range eh_frame_range, SYMS_DwEhPtrCtx *ptr_ctx, SYMS_U64 ip_voff)
{
  SYMS_DwCFIRecords result; syms_memzero_struct(&result);
  
  // find FDE offset
  void *eh_frame_hdr = (void*)((SYMS_U8*)base+eh_frame_hdr_range.min);
  SYMS_U64 fde_offset = syms_search_eh_frame_hdr__linear(eh_frame_hdr, syms_make_u64_range(0, syms_u64_range_size(eh_frame_hdr_range)), ptr_ctx, ip_voff);
  
  SYMS_B32 is_fde_offset_valid = (fde_offset != SYMS_U64_MAX);
  if (is_fde_offset_valid){
    SYMS_U64 fde_read_offset = (fde_offset - ptr_ctx->raw_base_vaddr);
    
    // read FDE size
    SYMS_U64 fde_size = 0;
    fde_read_offset += syms_dw_based_range_read_length(base, eh_frame_range, fde_read_offset, &fde_size);
    
    // read FDE discriminator
    SYMS_U32 fde_discrim = 0;
    fde_read_offset += syms_based_range_read_struct(base, eh_frame_range, fde_read_offset, &fde_discrim);
    
    // compute parent CIE offset
    SYMS_U64 cie_read_offset = fde_read_offset - (fde_discrim + sizeof(fde_discrim));
    
    // read CIE size
    SYMS_U64 cie_size = 0;
    cie_read_offset += syms_dw_based_range_read_length(base, eh_frame_range, cie_read_offset, &cie_size);
    
    // read CIE discriminator
    SYMS_U32 cie_discrim = SYMS_U32_MAX;
    cie_read_offset += syms_based_range_read_struct(base, eh_frame_range, cie_read_offset, &cie_discrim);
    
    SYMS_B32 is_fde = (fde_discrim != 0);
    SYMS_B32 is_cie = (cie_discrim == 0);
    if (is_fde && is_cie) {
      void *eh_frame = (void*)((SYMS_U8*)base + eh_frame_range.min);
      SYMS_U64Range cie_range = syms_make_u64_range(0, cie_read_offset + (cie_size - sizeof(cie_discrim)));
      SYMS_U64Range fde_range = syms_make_u64_range(0, fde_read_offset + (fde_size - sizeof(fde_discrim)));
      
      // parse CIE
      SYMS_DwCIEUnpacked cie; syms_memzero_struct(&cie);
      syms_unwind_elf_x64__eh_frame_parse_cie(eh_frame, cie_range, ptr_ctx, cie_read_offset, &cie);
      
      // parse FDE
      SYMS_DwFDEUnpacked fde; syms_memzero_struct(&fde);
      syms_unwind_elf_x64__eh_frame_parse_fde(eh_frame, fde_range, ptr_ctx, &cie, fde_read_offset, &fde);
      
      // range check instruction pointer
      if (fde.ip_voff_range.min <= ip_voff && ip_voff < fde.ip_voff_range.max){
        result.valid = syms_true;
        result.cie = cie;
        result.fde = fde;
      }
    }
  }
  
  return(result);
}

//- cfi machine

SYMS_API SYMS_DwCFIMachine
syms_unwind_elf_x64__machine_make(SYMS_U64 cells_per_row, SYMS_DwCIEUnpacked *cie, SYMS_DwEhPtrCtx *ptr_ctx){
  SYMS_DwCFIMachine result = {0};
  result.cells_per_row = cells_per_row;
  result.cie = cie;
  result.ptr_ctx = ptr_ctx;
  return(result);
}

SYMS_API void
syms_unwind_elf_x64__machine_equip_initial_row(SYMS_DwCFIMachine *machine, SYMS_DwCFIRow *initial_row){
  machine->initial_row = initial_row;
}

SYMS_API void
syms_unwind_elf_x64__machine_equip_fde_ip(SYMS_DwCFIMachine *machine, SYMS_U64 fde_ip){
  machine->fde_ip = fde_ip;
}

SYMS_API SYMS_DwCFIRow*
syms_unwind_elf_x64__row_alloc(SYMS_Arena *arena, SYMS_U64 cells_per_row){
  SYMS_DwCFIRow *result = syms_push_array(arena, SYMS_DwCFIRow, 1);
  result->cells = syms_push_array(arena, SYMS_DwCFICell, cells_per_row);
  return(result);
}

SYMS_API void
syms_unwind_elf_x64__row_zero(SYMS_DwCFIRow *row, SYMS_U64 cells_per_row){
  syms_memset(row->cells, 0, sizeof(*row->cells)*cells_per_row);
  syms_memzero_struct(&row->cfa_cell);
}

SYMS_API void
syms_unwind_elf_x64__row_copy(SYMS_DwCFIRow *dst, SYMS_DwCFIRow *src, SYMS_U64 cells_per_row){
  syms_memmove(dst->cells, src->cells, sizeof(*src->cells)*cells_per_row);
  dst->cfa_cell = src->cfa_cell;
}


SYMS_API SYMS_B32
syms_unwind_elf_x64__machine_run_to_ip(void *base, SYMS_U64Range range, SYMS_DwCFIMachine *machine,
                                       SYMS_U64 target_ip, SYMS_DwCFIRow *row){
  SYMS_LogOpen(SYMS_LogFeature_DwarfCFIDecode, 0, log);
  
  SYMS_B32 result = syms_false;
  
  // pull out machine's equipment
  SYMS_DwCIEUnpacked *cie = machine->cie;
  SYMS_DwEhPtrCtx *ptr_ctx = machine->ptr_ctx;
  SYMS_U64 cells_per_row = machine->cells_per_row;
  SYMS_DwCFIRow *initial_row = machine->initial_row;
  
  // start with an empty stack
  SYMS_ArenaTemp scratch = syms_get_scratch(0, 0);
  SYMS_DwCFIRow *stack = 0;
  SYMS_DwCFIRow *free_rows = 0;
  
  // initialize the row
  if (initial_row != 0){
    syms_unwind_elf_x64__row_copy(row, initial_row, cells_per_row);
  }
  else{
    syms_unwind_elf_x64__row_zero(row, cells_per_row);
  }
  SYMS_U64 table_ip = machine->fde_ip;
  
  // logging
#if SYMS_ENABLE_DEV_LOG
  if (initial_row == 0){
    SYMS_Log("CFA decode INIT:\n");
  }
  else{
    SYMS_Log("CFA decode MAIN:\n");
  }
  SYMS_Log("CFA row ip: %llx\n", table_ip);
#endif
  
  // loop
  SYMS_U64 cfi_off = 0;
  for (;;){
    // op variables
    SYMS_DwCFA opcode = 0;
    SYMS_U64 operand0 = 0;
    SYMS_U64 operand1 = 0;
    SYMS_U64 operand2 = 0;
    SYMS_DwCFAControlBits control_bits = 0;
    
    // decode opcode/operand0
    if (!syms_based_range_read(base, range, cfi_off, 1, &opcode)){
      result = syms_true;
      goto done;
    }
    if ((opcode & SYMS_DwCFAMask_HI_OPCODE) != 0){
      operand0 = (opcode & SYMS_DwCFAMask_OPERAND);
      opcode   = (opcode & SYMS_DwCFAMask_HI_OPCODE);
      control_bits = syms_unwind_elf__cfa_control_bits_kind2[opcode >> 6];
    }
    else{
      if (opcode < SYMS_DwCFADetail_OPL_KIND1){
        control_bits = syms_unwind_elf__cfa_control_bits_kind1[opcode];
      }
    }
    
    // decode operand1/operand2
    SYMS_U64 decode_cursor = cfi_off + 1;
    {
      // setup loop ins/outs
      SYMS_U64 o[2];
      SYMS_DwCFADecode dec[2] = {0};
      dec[0] = (control_bits & 0xF);
      dec[1] = ((control_bits >> 4) & 0xF);
      
      // loop
      SYMS_U64 *out = o;
      for (SYMS_U64 i = 0; i < 2; i += 1, out += 1){
        SYMS_DwCFADecode d = dec[i];
        SYMS_U64 o_size = 0;
        switch (d){
          case 0:
          {
            *out = 0;
          }break;
          default:
          {
            if (d <= 8){
              syms_based_range_read(base, range, decode_cursor, d, out);
              o_size = d;
            }
          }break;
          case SYMS_DwCFADecode_ADDRESS:
          {
            o_size = syms_unwind_elf_x64__parse_pointer(base, range, ptr_ctx, cie->addr_encoding, decode_cursor, out);
          }break;
          case SYMS_DwCFADecode_ULEB128:
          {
            o_size = syms_based_range_read_uleb128(base, range, decode_cursor, out);
          }break;
          case SYMS_DwCFADecode_SLEB128:
          {
            o_size = syms_based_range_read_sleb128(base, range, decode_cursor, (SYMS_S64*)out);
          }break;
        }
        decode_cursor += o_size;
      }
      
      // commit out values
      operand1 = o[0];
      operand2 = o[1];
    }
    SYMS_U64 after_decode_off = decode_cursor;
    
    // register checks
    if (control_bits & SYMS_DwCFAControlBits_IS_REG_0){
      if (operand0 >= cells_per_row){
        goto done;
      }
    }
    if (control_bits & SYMS_DwCFAControlBits_IS_REG_1){
      if (operand1 >= cells_per_row){
        goto done;
      }
    }
    if (control_bits & SYMS_DwCFAControlBits_IS_REG_2){
      if (operand2 >= cells_per_row){
        goto done;
      }
    }
    
    // logging
#if SYMS_ENABLE_DEV_LOG
    {
      SYMS_String8 opstring = syms_string_from_enum_value(SYMS_DwCFA, opcode);
      SYMS_Log("%.*s %llx %llx %llx\n", syms_expand_string(opstring), operand0, operand1, operand2);
    }
#endif
    
    // values for deferred work
    SYMS_U64 new_table_ip = table_ip;
    
    // step
    SYMS_U64 step_cursor = after_decode_off;
    switch (opcode){
      default: goto done;
      case SYMS_DwCFA_NOP:break;
      
      
      //// new row/IP opcodes ////
      
      case SYMS_DwCFA_SET_LOC:
      {
        new_table_ip = operand1;
      }break;
      case SYMS_DwCFA_ADVANCE_LOC:
      {
        new_table_ip = table_ip + operand0*cie->code_align_factor;
      }break;
      case SYMS_DwCFA_ADVANCE_LOC1:case SYMS_DwCFA_ADVANCE_LOC2:case SYMS_DwCFA_ADVANCE_LOC4:
      {
        SYMS_U64 advance = operand1*cie->code_align_factor;
        new_table_ip = table_ip + advance;
      }break;
      
      
      //// change CFA (canonical frame address) opcodes ////
      
      case SYMS_DwCFA_DEF_CFA:
      {
        row->cfa_cell.rule = SYMS_DwCFICFARule_REGOFF;
        row->cfa_cell.reg_idx = operand1;
        row->cfa_cell.offset = operand2;
      }break;
      
      case SYMS_DwCFA_DEF_CFA_SF:
      {
        row->cfa_cell.rule = SYMS_DwCFICFARule_REGOFF;
        row->cfa_cell.reg_idx = operand1;
        row->cfa_cell.offset = ((SYMS_S64)operand2)*cie->data_align_factor;
      }break;
      
      case SYMS_DwCFA_DEF_CFA_REGISTER:
      {
        // check rule
        if (row->cfa_cell.rule != SYMS_DwCFICFARule_REGOFF){
          goto done;
        }
        // commit new cfa
        row->cfa_cell.reg_idx = operand1;
      }break;
      
      case SYMS_DwCFA_DEF_CFA_OFFSET:
      {
        // check rule
        if (row->cfa_cell.rule != SYMS_DwCFICFARule_REGOFF){
          goto done;
        }
        // commit new cfa
        row->cfa_cell.offset = operand1;
      }break;
      
      case SYMS_DwCFA_DEF_CFA_OFFSET_SF:
      {
        // check rule
        if (row->cfa_cell.rule != SYMS_DwCFICFARule_REGOFF){
          goto done;
        }
        // commit new cfa
        row->cfa_cell.offset = ((SYMS_S64)operand1)*cie->data_align_factor;
      }break;
      
      case SYMS_DwCFA_DEF_CFA_EXPR:
      {
        // setup expr range
        SYMS_U64 expr_first = range.min + after_decode_off;
        SYMS_U64 expr_size = operand1;
        step_cursor += expr_size;
        
        // commit new cfa
        row->cfa_cell.rule = SYMS_DwCFICFARule_EXPR;
        row->cfa_cell.expr.min = expr_first;
        row->cfa_cell.expr.max = expr_first + expr_size;
      }break;
      
      
      //// change register rules ////
      
      case SYMS_DwCFA_UNDEFINED:
      {
        row->cells[operand1].rule = SYMS_DwCFIRegisterRule_UNDEFINED;
      }break;
      
      case SYMS_DwCFA_SAME_VALUE:
      {
        row->cells[operand1].rule = SYMS_DwCFIRegisterRule_SAME_VALUE;
      }break;
      
      case SYMS_DwCFA_OFFSET:
      {
        SYMS_DwCFICell *cell = &row->cells[operand0];
        cell->rule = SYMS_DwCFIRegisterRule_OFFSET;
        cell->n = operand1*cie->data_align_factor;
      }break;
      
      case SYMS_DwCFA_OFFSET_EXT:
      {
        SYMS_DwCFICell *cell = &row->cells[operand1];
        cell->rule = SYMS_DwCFIRegisterRule_OFFSET;
        cell->n = operand2*cie->data_align_factor;
      }break;
      
      case SYMS_DwCFA_OFFSET_EXT_SF:
      {
        SYMS_DwCFICell *cell = &row->cells[operand1];
        cell->rule = SYMS_DwCFIRegisterRule_OFFSET;
        cell->n = ((SYMS_S64)operand2)*cie->data_align_factor;
      }break;
      
      case SYMS_DwCFA_VAL_OFFSET:
      {
        SYMS_DwCFICell *cell = &row->cells[operand1];
        cell->rule = SYMS_DwCFIRegisterRule_VAL_OFFSET;
        cell->n = operand2*cie->data_align_factor;
      }break;
      
      case SYMS_DwCFA_VAL_OFFSET_SF:
      {
        SYMS_DwCFICell *cell = &row->cells[operand1];
        cell->rule = SYMS_DwCFIRegisterRule_VAL_OFFSET;
        cell->n = ((SYMS_S64)operand2)*cie->data_align_factor;
      }break;
      
      case SYMS_DwCFA_REGISTER:
      {
        SYMS_DwCFICell *cell = &row->cells[operand1];
        cell->rule = SYMS_DwCFIRegisterRule_REGISTER;
        cell->n = operand2;
      }break;
      
      case SYMS_DwCFA_EXPR:
      {
        // setup expr range
        SYMS_U64 expr_first = range.min + after_decode_off;
        SYMS_U64 expr_size = operand2;
        step_cursor += expr_size;
        
        // commit new rule
        SYMS_DwCFICell *cell = &row->cells[operand1];
        cell->rule = SYMS_DwCFIRegisterRule_EXPRESSION;
        cell->expr.min = expr_first;
        cell->expr.max = expr_first + expr_size;
      }break;
      
      case SYMS_DwCFA_VAL_EXPR:
      {
        // setup expr range
        SYMS_U64 expr_first = range.min + after_decode_off;
        SYMS_U64 expr_size = operand2;
        step_cursor += expr_size;
        
        // commit new rule
        SYMS_DwCFICell *cell = &row->cells[operand1];
        cell->rule = SYMS_DwCFIRegisterRule_VAL_EXPRESSION;
        cell->expr.min = expr_first;
        cell->expr.max = expr_first + expr_size;
      }break;
      
      case SYMS_DwCFA_RESTORE:
      {
        // check initial row
        if (initial_row == 0){
          goto done;
        }
        // commit new rule
        row->cells[operand0] = initial_row->cells[operand0];
      }break;
      
      case SYMS_DwCFA_RESTORE_EXT:
      {
        // check initial row
        if (initial_row == 0){
          goto done;
        }
        // commit new rule
        row->cells[operand1] = initial_row->cells[operand1];
      }break;
      
      
      //// row stack ////
      
      case SYMS_DwCFA_REMEMBER_STATE:
      {
        SYMS_DwCFIRow *stack_row = free_rows;
        if (stack_row != 0){
          SYMS_StackPop(free_rows);
        }
        else{
          stack_row = syms_unwind_elf_x64__row_alloc(scratch.arena, cells_per_row);
        }
        syms_unwind_elf_x64__row_copy(stack_row, row, cells_per_row);
        SYMS_StackPush(stack, stack_row);
      }break;
      
      case SYMS_DwCFA_RESTORE_STATE:
      {
        if (stack != 0){
          SYMS_DwCFIRow *stack_row = stack;
          SYMS_StackPop(stack);
          syms_unwind_elf_x64__row_copy(row, stack_row, cells_per_row);
          SYMS_StackPush(free_rows, stack_row);
        }
        else{
          syms_unwind_elf_x64__row_zero(row, cells_per_row);
        }
      }break;
    }
    
    // apply location change
    if (control_bits & SYMS_DwCFAControlBits_NEW_ROW){
      // new ip should always grow the ip
      if (new_table_ip <= table_ip){
        goto done;
      }
      // stop if this encloses the target ip
      if (table_ip <= target_ip && target_ip < new_table_ip){
        result = syms_true;
        goto done;
      }
      // commit new ip
      table_ip = new_table_ip;
      
      SYMS_Log("CFA row ip: %llx\n", table_ip);
    }
    
    // advance
    cfi_off = step_cursor;
  }
  done:;
  
  // logging
#if SYMS_ENABLE_DEV_LOG
  {
    switch (row->cfa_cell.rule){
      case SYMS_DwCFICFARule_REGOFF:
      {
        SYMS_Log("CFA:(%llu, %lld); ", row->cfa_cell.reg_idx, row->cfa_cell.offset);
      }break;
      case SYMS_DwCFICFARule_EXPR:
      {
        SYMS_Log("CFA:EXPR; ");
      }break;
    }
    SYMS_DwCFICell *cell = row->cells;
    for (SYMS_U64 i = 0; i < cells_per_row; i += 1, cell += 1){
      SYMS_Log("[%llu]:", i);
      switch (cell->rule){
        case SYMS_DwCFIRegisterRule_UNDEFINED:
        {
          SYMS_Log("UNDEFINED ");
        }break;
        case SYMS_DwCFIRegisterRule_SAME_VALUE:
        {
          SYMS_Log("SAME_VALUE ");
        }break;
        case SYMS_DwCFIRegisterRule_OFFSET:
        {
          SYMS_Log("[CFA+0x%llx] ", cell->n);
        }break;
        case SYMS_DwCFIRegisterRule_VAL_OFFSET:
        {
          SYMS_Log("(CFA+0x%llx) ", cell->n);
        }break;
        case SYMS_DwCFIRegisterRule_REGISTER:
        {
          SYMS_Log("R[%lld] ", cell->n);
        }break;
        case SYMS_DwCFIRegisterRule_EXPRESSION:
        {
          SYMS_Log("EXPRESSION ");
        }break;
        case SYMS_DwCFIRegisterRule_VAL_EXPRESSION:
        {
          SYMS_Log("VAL_EXPRESSION ");
        }break;
      }
    }
    SYMS_Log("\n");
  }
#endif
  
  syms_release_scratch(scratch);
  
  SYMS_LogClose(log);
  
  return(result);
}

#endif //SYMS_UNWIND_ELF_X64_C

