// Copyright Epic Games, Inc. All Rights Reserved.
/* date = September 28th 2021 9:35 am */

#ifndef SYMS_UNWIND_ELF_X64_H
#define SYMS_UNWIND_ELF_X64_H

////////////////////////////////
// NOTE(allen): Generated Code

#include "syms/core/generated/syms_meta_dwarf_cfi.h"

////////////////////////////////
// NOTE(allen): ELF-x64 Unwind Types

// EH: Exception Frames

typedef SYMS_U8 SYMS_DwEhPtrEnc;
enum{
  SYMS_DwEhPtrEnc_TYPE_MASK = 0x0F,
  SYMS_DwEhPtrEnc_PTR     = 0x00, // Pointer sized unsigned value
  SYMS_DwEhPtrEnc_ULEB128 = 0x01, // Unsigned LE base-128 value
  SYMS_DwEhPtrEnc_UDATA2  = 0x02, // Unsigned 16-bit value
  SYMS_DwEhPtrEnc_UDATA4  = 0x03, // Unsigned 32-bit value
  SYMS_DwEhPtrEnc_UDATA8  = 0x04, // Unsigned 64-bit value
  SYMS_DwEhPtrEnc_SIGNED  = 0x08, // Signed pointer
  SYMS_DwEhPtrEnc_SLEB128 = 0x09, // Signed LE base-128 value
  SYMS_DwEhPtrEnc_SDATA2  = 0x0A, // Signed 16-bit value
  SYMS_DwEhPtrEnc_SDATA4  = 0x0B, // Signed 32-bit value
  SYMS_DwEhPtrEnc_SDATA8  = 0x0C, // Signed 64-bit value
};
enum{
  SYMS_DwEhPtrEnc_MODIF_MASK = 0x70,
  SYMS_DwEhPtrEnc_PCREL   = 0x10, // Value is relative to the current program counter.
  SYMS_DwEhPtrEnc_TEXTREL = 0x20, // Value is relative to the .text section.
  SYMS_DwEhPtrEnc_DATAREL = 0x30, // Value is relative to the .got or .eh_frame_hdr section.
  SYMS_DwEhPtrEnc_FUNCREL = 0x40, // Value is relative to the function.
  SYMS_DwEhPtrEnc_ALIGNED = 0x50, // Value is aligned to an address unit sized boundary.
};
enum{
  SYMS_DwEhPtrEnc_INDIRECT = 0x80, // This flag indicates that value is stored in virtual memory.

  SYMS_DwEhPtrEnc_OMIT     = 0xFF,
};

typedef struct SYMS_DwEhPtrCtx{
  SYMS_U64 raw_base_vaddr; // address where pointer is being read
  SYMS_U64 text_vaddr;     // base address of section with instructions (used for encoding pointer on SH and IA64)
  SYMS_U64 data_vaddr;     // base address of data section (used for encoding pointer on x86-64)
  SYMS_U64 func_vaddr;     // base address of function where IP is located
} SYMS_DwEhPtrCtx;

// CIE: Common Information Entry
typedef struct SYMS_DwCIEUnpacked{
  SYMS_U8 version;
  SYMS_DwEhPtrEnc lsda_encoding;
  SYMS_DwEhPtrEnc addr_encoding;
  
  SYMS_B8 has_augmentation_size;
  SYMS_U64 augmentation_size;
  SYMS_String8 augmentation;
  
  SYMS_U64 code_align_factor;
  SYMS_S64 data_align_factor;
  SYMS_U64 ret_addr_reg;
  
  SYMS_U64 handler_ip;
  
  SYMS_U64Range cfi_range;
} SYMS_DwCIEUnpacked;

typedef struct SYMS_DwCIEUnpackedNode{
  struct SYMS_DwCIEUnpackedNode *next;
  SYMS_DwCIEUnpacked cie;
  SYMS_U64 offset;
} SYMS_DwCIEUnpackedNode;

// FDE: Frame Description Entry
typedef struct SYMS_DwFDEUnpacked{
  SYMS_U64Range ip_voff_range;
  SYMS_U64 lsda_ip;
  
  SYMS_U64Range cfi_range;
} SYMS_DwFDEUnpacked;

// CFI: Call Frame Information
typedef struct SYMS_DwCFIRecords{
  SYMS_B32 valid;
  SYMS_DwCIEUnpacked cie;
  SYMS_DwFDEUnpacked fde;
} SYMS_DwCFIRecords;

typedef enum SYMS_DwCFICFARule{
  SYMS_DwCFICFARule_REGOFF,
  SYMS_DwCFICFARule_EXPR,
} SYMS_DwCFICFARule;

typedef struct SYMS_DwCFICFACell{
  SYMS_DwCFICFARule rule;
  union{
    struct{
      SYMS_U64 reg_idx;
      SYMS_S64 offset;
    };
    SYMS_U64Range expr;
  };
} SYMS_DwCFICFACell;

typedef enum SYMS_DwCFIRegisterRule{
  SYMS_DwCFIRegisterRule_SAME_VALUE,
  SYMS_DwCFIRegisterRule_UNDEFINED,
  SYMS_DwCFIRegisterRule_OFFSET,
  SYMS_DwCFIRegisterRule_VAL_OFFSET,
  SYMS_DwCFIRegisterRule_REGISTER,
  SYMS_DwCFIRegisterRule_EXPRESSION,
  SYMS_DwCFIRegisterRule_VAL_EXPRESSION,
} SYMS_DwCFIRegisterRule;

typedef struct SYMS_DwCFICell{
  SYMS_DwCFIRegisterRule rule;
  union{
    SYMS_S64 n;
    SYMS_U64Range expr;
  };
} SYMS_DwCFICell;

typedef struct SYMS_DwCFIRow{
  struct SYMS_DwCFIRow *next;
  SYMS_DwCFICell *cells;
  SYMS_DwCFICFACell cfa_cell;
} SYMS_DwCFIRow;

typedef struct SYMS_DwCFIMachine{
  SYMS_U64 cells_per_row;
  SYMS_DwCIEUnpacked *cie;
  SYMS_DwEhPtrCtx *ptr_ctx;
  SYMS_DwCFIRow *initial_row;
  SYMS_U64 fde_ip;
} SYMS_DwCFIMachine;

typedef SYMS_U8 SYMS_DwCFADecode;
enum{
  SYMS_DwCFADecode_NOP     = 0x0,
  // 1,2,4,8 reserved for literal byte sizes
  SYMS_DwCFADecode_ADDRESS = 0x9,
  SYMS_DwCFADecode_ULEB128 = 0xA,
  SYMS_DwCFADecode_SLEB128 = 0xB,
};

typedef SYMS_U16 SYMS_DwCFAControlBits;
enum{
  SYMS_DwCFAControlBits_DEC1_MASK = 0x00F,
  SYMS_DwCFAControlBits_DEC2_MASK = 0x0F0,
  SYMS_DwCFAControlBits_IS_REG_0  = 0x100,
  SYMS_DwCFAControlBits_IS_REG_1  = 0x200,
  SYMS_DwCFAControlBits_IS_REG_2  = 0x400,
  SYMS_DwCFAControlBits_NEW_ROW   = 0x800,
};

SYMS_GLOBAL SYMS_DwCFAControlBits syms_unwind_elf__cfa_control_bits_kind1[SYMS_DwCFADetail_OPL_KIND1 + 1];
SYMS_GLOBAL SYMS_DwCFAControlBits syms_unwind_elf__cfa_control_bits_kind2[SYMS_DwCFADetail_OPL_KIND2 + 1];

// NOTE(allen): register codes for unwinding match the SYMS_DwRegX64 register codes
#define SYMS_UNWIND_ELF_X64__REG_SLOT_COUNT 17

////////////////////////////////
// NOTE(allen): ELF-x64 Unwind Function

SYMS_API SYMS_UnwindResult syms_unwind_elf_x64(SYMS_String8 bin_data, SYMS_ElfBinAccel *bin, SYMS_U64 bin_base,
                                               SYMS_MemoryView *memview, SYMS_U64 stack_pointer,
                                               SYMS_DwRegsX64 *regs_in_out);

SYMS_API SYMS_UnwindResult syms_unwind_elf_x64__apply_rules(SYMS_String8 bin_data, SYMS_DwCFIRow *row,
                                                            SYMS_U64 text_base_vaddr,
                                                            SYMS_MemoryView *memview, SYMS_U64 stack_pointer,
                                                            SYMS_DwRegsX64 *regs_in_out);

////////////////////////////////
// NOTE(allen): ELF-x64 Unwind Helper Functions

SYMS_API void     syms_unwind_elf_x64__init(void);

SYMS_API SYMS_U64 syms_unwind_elf_x64__parse_pointer(void *base, SYMS_U64Range range,
                                                     SYMS_DwEhPtrCtx *ptr_ctx, SYMS_DwEhPtrEnc ptr_enc,
                                                     SYMS_U64 off, SYMS_U64 *ptr_out);

//- eh_frame parsing
SYMS_API void syms_unwind_elf_x64__eh_frame_parse_cie(void *base,SYMS_U64Range range,SYMS_DwEhPtrCtx *ptr_ctx,
                                                      SYMS_U64 off, SYMS_DwCIEUnpacked *cie_out);
SYMS_API void syms_unwind_elf_x64__eh_frame_parse_fde(void *base,SYMS_U64Range range,SYMS_DwEhPtrCtx *ptr_ctx,
                                                      SYMS_DwCIEUnpacked *parent_cie, SYMS_U64 off,
                                                      SYMS_DwFDEUnpacked *fde_out);
SYMS_API SYMS_DwCFIRecords
syms_unwind_elf_x64__eh_frame_cfi_from_ip__sloppy(void *base, SYMS_U64Range range, SYMS_DwEhPtrCtx *ptr_ctx,
                                                  SYMS_U64 ip_voff);
SYMS_API SYMS_DwCFIRecords
syms_unwind_elf_x64__eh_frame_hdr_from_ip(void *base, 
                                          SYMS_U64Range eh_frame_hdr_range, 
                                          SYMS_U64Range eh_frame_range, 
                                          SYMS_DwEhPtrCtx *ptr_ctx, 
                                          SYMS_U64 ip_voff);

//- cfi machine

SYMS_API SYMS_DwCFIMachine  syms_unwind_elf_x64__machine_make(SYMS_U64 cells_per_row, SYMS_DwCIEUnpacked *cie,
                                                              SYMS_DwEhPtrCtx *ptr_ctx);
SYMS_API void               syms_unwind_elf_x64__machine_equip_initial_row(SYMS_DwCFIMachine *machine,
                                                                           SYMS_DwCFIRow *initial_row);
SYMS_API void               syms_unwind_elf_x64__machine_equip_fde_ip(SYMS_DwCFIMachine *machine,
                                                                      SYMS_U64 fde_ip);

SYMS_API SYMS_DwCFIRow*syms_unwind_elf_x64__row_alloc(SYMS_Arena *arena, SYMS_U64 cells_per_row);
SYMS_API void          syms_unwind_elf_x64__row_zero(SYMS_DwCFIRow *row, SYMS_U64 cells_per_row);
SYMS_API void          syms_unwind_elf_x64__row_copy(SYMS_DwCFIRow *dst, SYMS_DwCFIRow *src,
                                                     SYMS_U64 cells_per_row);

SYMS_API SYMS_B32      syms_unwind_elf_x64__machine_run_to_ip(void *base, SYMS_U64Range range,
                                                              SYMS_DwCFIMachine *machine, SYMS_U64 target_ip,
                                                              SYMS_DwCFIRow *row_out);

#endif //SYMS_UNWIND_ELF_X64_H
