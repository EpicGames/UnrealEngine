// Copyright Epic Games, Inc. All Rights Reserved.

#ifndef SYMS_UNWIND_PE_X64_C
#define SYMS_UNWIND_PE_X64_C

////////////////////////////////
// NOTE(allen): PE-x64 Unwind Function

SYMS_API SYMS_UnwindResult
syms_unwind_pe_x64(SYMS_String8 data, SYMS_PeBinAccel *bin, SYMS_U64 bin_base,
                   SYMS_MemoryView *memview, SYMS_RegX64 *regs){
  SYMS_UnwindResult result = {0};
  SYMS_U64 missed_read_addr = 0;
  
  //- grab ip_voff (several places can use this)
  SYMS_U64 ip_voff = regs->rip.u64 - bin_base;
  
  //- get pdata entry from current ip
  SYMS_PeIntelPdata *initial_pdata = 0;
  {
    SYMS_U64 initial_pdata_off = syms_pe_binary_search_intel_pdata(data, bin, ip_voff);
    if (initial_pdata_off != 0){
      initial_pdata = (SYMS_PeIntelPdata*)(data.str + initial_pdata_off);
    }
  }
  
  //- no pdata; unwind by reading stack pointer
  if (initial_pdata == 0){
    // read ip from stack pointer
    SYMS_U64 sp = regs->rsp.u64;
    SYMS_U64 new_ip = 0;
    if (!syms_memory_view_read_struct(memview, sp, &new_ip)){
      missed_read_addr = sp;
      goto error_out;
    }
    
    // advance stack pointer
    SYMS_U64 new_sp = sp + 8;
    
    // commit registers
    regs->rip.u64 = new_ip;
    regs->rsp.u64 = new_sp;
  }
  
  //- got pdata; perform unwinding with exception handling
  else{
    // try epilog unwind
    SYMS_B32 did_epilog_unwind = syms_false;
    if (syms_unwind_pe_x64__voff_is_in_epilog(data, bin, ip_voff, initial_pdata)){
      result = syms_unwind_pe_x64__epilog(data, bin, bin_base, memview, regs);
      did_epilog_unwind = syms_true;
    }
    
    // try xdata unwind
    if (!did_epilog_unwind){
      SYMS_B32 did_machframe = syms_false;
      
      // get frame reg
      SYMS_Reg64 *frame_reg = 0;
      SYMS_U64 frame_off = 0;
      
      {
        SYMS_U64 unwind_info_off = initial_pdata->voff_unwind_info;
        SYMS_UnwindPeInfo *unwind_info = (SYMS_UnwindPeInfo*)(syms_pe_ptr_from_voff(data, bin, unwind_info_off));
        
        SYMS_U32 frame_reg_id = SYMS_UnwindPeInfo_RegFromFrame(unwind_info->frame);
        SYMS_U64 frame_off_val = SYMS_UnwindPeInfo_OffFromFrame(unwind_info->frame);
        
        if (frame_reg_id != 0){
          frame_reg = syms_unwind_pe_x64__gpr_reg(regs, frame_reg_id);
          // TODO(allen): at this point if frame_reg is zero, the exe is corrupted.
        }
        frame_off = frame_off_val;
      }
      
      SYMS_PeIntelPdata *pdata = initial_pdata;
      for (;;){
        //- get unwind info & codes
        SYMS_U64 unwind_info_off = pdata->voff_unwind_info;
        SYMS_UnwindPeInfo *unwind_info = (SYMS_UnwindPeInfo*)(syms_pe_ptr_from_voff(data, bin, unwind_info_off));
        SYMS_UnwindPeCode *unwind_codes = (SYMS_UnwindPeCode*)(unwind_info + 1);
        
        //- get frame base
        SYMS_U64 frame_base = regs->rsp.u64;
        if (frame_reg != 0){
          SYMS_U64 raw_frame_base = frame_reg->u64;
          SYMS_U64 adjusted_frame_base = raw_frame_base - frame_off*16;
          if (adjusted_frame_base < raw_frame_base){
            frame_base = adjusted_frame_base;
          }
          else{
            frame_base = 0;
          }
        }
        
        //- op code interpreter
        SYMS_UnwindPeCode *code_ptr = unwind_codes;
        SYMS_UnwindPeCode *code_opl = unwind_codes + unwind_info->codes_num;
        for (SYMS_UnwindPeCode *next_code_ptr = 0; code_ptr < code_opl; code_ptr = next_code_ptr){
          // extract op code parts
          SYMS_U32 op_code = SYMS_UnwindPeCode_CodeFromFlags(code_ptr->flags);
          SYMS_U32 op_info = SYMS_UnwindPeCode_InfoFromFlags(code_ptr->flags);
          
          // determine number of op code slots
          SYMS_U32 slot_count = syms_unwind_pe_x64__slot_count_from_op_code(op_code);
          if (op_code == SYMS_UnwindPeOpCode_ALLOC_LARGE && op_info == 1){
            slot_count += 1;
          }
          
          // check op code slot count
          if (slot_count == 0 || code_ptr + slot_count > code_opl){
            result.dead = syms_true;
            goto end_xdata_unwind;
          }
          
          // set next op code pointer
          next_code_ptr = code_ptr + slot_count;
          
          // interpret this op code
          SYMS_U64 code_voff = pdata->voff_first + code_ptr->off_in_prolog;
          if (code_voff <= ip_voff){
            switch (op_code){
              case SYMS_UnwindPeOpCode_PUSH_NONVOL:
              {
                // read value from stack pointer
                SYMS_U64 sp = regs->rsp.u64;
                SYMS_U64 value = 0;
                if (!syms_memory_view_read_struct(memview, sp, &value)){
                  missed_read_addr = sp;
                  goto error_out;
                }
                
                // advance stack pointer
                SYMS_U64 new_sp = sp + 8;
                
                // commit registers
                SYMS_Reg64 *reg = syms_unwind_pe_x64__gpr_reg(regs, op_info);
                reg->u64 = value;
                regs->rsp.u64 = new_sp;
              }break;
              
              case SYMS_UnwindPeOpCode_ALLOC_LARGE:
              {
                // read alloc size
                SYMS_U64 size = 0;
                if (op_info == 0){
                  size = code_ptr[1].u16*8;
                }
                else if (op_info == 1){
                  size = code_ptr[1].u16 + ((SYMS_U32)code_ptr[2].u16 << 16);
                }
                else{
                  result.dead = syms_true;
                  goto end_xdata_unwind;
                }
                
                // advance stack pointer
                SYMS_U64 sp = regs->rsp.u64;
                SYMS_U64 new_sp = sp + size;
                
                // advance stack pointer
                regs->rsp.u64 = new_sp;
              }break;
              
              case SYMS_UnwindPeOpCode_ALLOC_SMALL:
              {
                // advance stack pointer
                regs->rsp.u64 += op_info*8 + 8;
              }break;
              
              case SYMS_UnwindPeOpCode_SET_FPREG:
              {
                // put stack pointer back to the frame base
                regs->rsp.u64 = frame_base;
              }break;
              
              case SYMS_UnwindPeOpCode_SAVE_NONVOL:
              {
                // read value from frame base
                SYMS_U64 off = code_ptr[1].u16*8;
                SYMS_U64 addr = frame_base + off;
                SYMS_U64 value = 0;
                if (!syms_memory_view_read_struct(memview, addr, &value)){
                  missed_read_addr = addr;
                  goto error_out;
                }
                
                // commit to register
                SYMS_Reg64 *reg = syms_unwind_pe_x64__gpr_reg(regs, op_info);
                reg->u64 = value;
              }break;
              
              case SYMS_UnwindPeOpCode_SAVE_NONVOL_FAR:
              {
                // read value from frame base
                SYMS_U64 off = code_ptr[1].u16 + ((SYMS_U32)code_ptr[2].u16 << 16);
                SYMS_U64 addr = frame_base + off;
                SYMS_U64 value = 0;
                if (!syms_memory_view_read_struct(memview, addr, &value)){
                  missed_read_addr = addr;
                  goto error_out;
                }
                
                // commit to register
                SYMS_Reg64 *reg = syms_unwind_pe_x64__gpr_reg(regs, op_info);
                reg->u64 = value;
              }break;
              
              case SYMS_UnwindPeOpCode_EPILOG:
              {
                // NOTE(rjf): this was found by stepping through kernel code after an exception was
                // thrown, encountered in the exception_stepping_tests (after the throw) in mule_main
                // SYMS_ASSERT(!"Hit me!");
                result.dead = syms_true;
                // TODO(allen): ???
              }break;
              
              case SYMS_UnwindPeOpCode_SPARE_CODE:
              {
                SYMS_ASSERT(!"Hit me!");
                // TODO(allen): ???
              }break;
              
              case SYMS_UnwindPeOpCode_SAVE_XMM128:
              {
                // read new register values
                SYMS_U8 buf[16];
                SYMS_U64 off = code_ptr[1].u16*16;
                SYMS_U64 addr = frame_base + off;
                if (!syms_memory_view_read(memview, addr, 16, buf)){
                  missed_read_addr = addr;
                  goto error_out;
                }
                
                // commit to register
                void *xmm_reg = (&regs->ymm0) + op_info;
                syms_memmove(xmm_reg, buf, sizeof(buf));
              }break;
              
              case SYMS_UnwindPeOpCode_SAVE_XMM128_FAR:
              {
                // read new register values
                SYMS_U8 buf[16];
                SYMS_U64 off = code_ptr[1].u16 + ((SYMS_U32)code_ptr[2].u16 << 16);
                SYMS_U64 addr = frame_base + off;
                if (!syms_memory_view_read(memview, addr, 16, buf)){
                  missed_read_addr = addr;
                  goto error_out;
                }
                
                // commit to register
                void *xmm_reg = (&regs->ymm0) + op_info;
                syms_memmove(xmm_reg, buf, sizeof(buf));
              }break;
              
              case SYMS_UnwindPeOpCode_PUSH_MACHFRAME:
              {
                // NOTE(rjf): this was found by stepping through kernel code after an exception was
                // thrown, encountered in the exception_stepping_tests (after the throw) in mule_main
                // SYMS_ASSERT(!"Hit me!");
                
                if (op_info > 1){
                  result.dead = syms_true;
                  goto end_xdata_unwind;
                }
                
                // read values
                SYMS_U64 sp_og = regs->rsp.u64;
                SYMS_U64 sp_adj = sp_og;
                if (op_info == 1){
                  sp_adj += 8;
                }
                
                SYMS_U64 ip_value = 0;
                if (!syms_memory_view_read_struct(memview, sp_adj, &ip_value)){
                  missed_read_addr = sp_adj;
                  goto error_out;
                }
                
                SYMS_U64 sp_after_ip = sp_adj + 8;
                SYMS_U16 ss_value = 0;
                if (!syms_memory_view_read_struct(memview, sp_after_ip, &ss_value)){
                  missed_read_addr = sp_after_ip;
                  goto error_out;
                }
                
                SYMS_U64 sp_after_ss = sp_after_ip + 8;
                SYMS_U64 rflags_value = 0;
                if (!syms_memory_view_read_struct(memview, sp_after_ss, &rflags_value)){
                  missed_read_addr = sp_after_ss;
                  goto error_out;
                }
                
                SYMS_U64 sp_after_rflags = sp_after_ss + 8;
                SYMS_U64 sp_value = 0;
                if (!syms_memory_view_read_struct(memview, sp_after_rflags, &sp_value)){
                  missed_read_addr = sp_after_rflags;
                  goto error_out;
                }
                
                // commit registers
                regs->rip.u64 = ip_value;
                regs->ss.u16 = ss_value;
                regs->rflags.u64 = rflags_value;
                regs->rsp.u64 = sp_value;
                
                // mark machine frame
                did_machframe = syms_true;
              }break;
            }
          }
        }
        
        //- iterate pdata chain
        SYMS_U32 flags = SYMS_UnwindPeInfo_FlagsFromHeader(unwind_info->header);
        if (!(flags & SYMS_UnwindPeInfoFlag_CHAINED)){
          break;
        }
        
        SYMS_U64 code_count_rounded = SYMS_AlignPow2(unwind_info->codes_num, 2);
        SYMS_U64 code_size = code_count_rounded*sizeof(SYMS_UnwindPeCode);
        SYMS_U64 chained_pdata_off = unwind_info_off + sizeof(SYMS_UnwindPeInfo) + code_size;
        
        pdata = (SYMS_PeIntelPdata*)syms_pe_ptr_from_voff(data, bin, chained_pdata_off);
      }
      
      if (!did_machframe){
        SYMS_U64 sp = regs->rsp.u64;
        SYMS_U64 new_ip = 0;
        if (!syms_memory_view_read_struct(memview, sp, &new_ip)){
          missed_read_addr = sp;
          goto error_out;
        }
        
        // advance stack pointer
        SYMS_U64 new_sp = sp + 8;
        
        // commit registers
        regs->rip.u64 = new_ip;
        regs->rsp.u64 = new_sp;
      }
      
      end_xdata_unwind:;
    }
  }
  
  error_out:;
  
  if (missed_read_addr != 0){
    SYMS_Log("Memory read miss: 0x%llx\n", missed_read_addr);
    syms_unwind_result_missed_read(&result, missed_read_addr);
  }
  
  if (!result.dead){
    result.stack_pointer = regs->rsp.u64;
  }
  
  return(result);
}

SYMS_API SYMS_UnwindResult
syms_unwind_pe_x64__epilog(SYMS_String8 bin_data, SYMS_PeBinAccel *bin, SYMS_U64 bin_base,
                           SYMS_MemoryView *memview, SYMS_RegX64 *regs){
  SYMS_LogOpen(SYMS_LogFeature_PEEpilog, 0, log);
  
  SYMS_UnwindResult result = {0};
  SYMS_U64 missed_read_addr = 0;
  
  //- setup parsing context
  SYMS_U64 ip_voff = regs->rip.u64 - bin_base;
  SYMS_U64 sec_number = syms_pe_sec_number_from_voff(bin_data, bin, ip_voff);
  SYMS_CoffSectionHeader *sec = syms_pecoff_sec_hdr_from_n(bin_data, bin->section_array_off, sec_number);
  void*         inst_base  = syms_pe_ptr_from_sec_number(bin_data, bin, sec_number);
  SYMS_U64Range inst_range = {0, sec->virt_size};
  
  //- setup parsing variables
  SYMS_B32 keep_parsing = syms_true;
  SYMS_U64 off = ip_voff - sec->virt_off;
  
  //- parsing loop
  for (;keep_parsing;){
    keep_parsing = syms_false;
    
    SYMS_U8 inst_byte = 0;
    syms_based_range_read_struct(inst_base, inst_range, off, &inst_byte);
    off += 1;
    
    SYMS_U8 rex = 0;
    if ((inst_byte & 0xF0) == 0x40){
      rex = inst_byte & 0x0F; // rex prefix
      syms_based_range_read_struct(inst_base, inst_range, off, &inst_byte);
      off += 1;
    }
    
    switch (inst_byte){
      // pop
      case 0x58:
      case 0x59:
      case 0x5A:
      case 0x5B:
      case 0x5C:
      case 0x5D:
      case 0x5E:
      case 0x5F:
      {
        SYMS_U64 sp = regs->rsp.u64;
        SYMS_U64 value = 0;
        if (!syms_memory_view_read_struct(memview, sp, &value)){
          missed_read_addr = sp;
          goto error_out;
        }
        
        // modify register
        SYMS_UnwindPeX64GprReg gpr_reg = (inst_byte - 0x58) + (rex & 1)*8;
        SYMS_Reg64 *reg = syms_unwind_pe_x64__gpr_reg(regs, gpr_reg);
        
        // not a final instruction
        keep_parsing = syms_true;
        
        // commit registers
        reg->u64 = value;
        regs->rsp.u64 = sp + 8;
      }break;
      
      // add $nnnn,%rsp 
      case 0x81:
      {
        // skip one byte (we already know what it is in this scenario)
        off += 1;
        
        // read the 4-byte immediate
        SYMS_S32 imm = 0;
        syms_based_range_read_struct(inst_base, inst_range, off, &imm);
        off += 4;
        
        // not a final instruction
        keep_parsing = syms_true;
        
        // update stack pointer
        regs->rsp.u64 = (SYMS_U64)(regs->rsp.u64 + imm);
      }break;
      
      // add $n,%rsp
      case 0x83:
      {
        // skip one byte (we already know what it is in this scenario)
        off += 1;
        
        // read the 1-byte immediate
        SYMS_S8 imm = 0;
        syms_based_range_read_struct(inst_base, inst_range, off, &imm);
        off += 1;
        
        // update stack pointer
        regs->rsp.u64 = (SYMS_U64)(regs->rsp.u64 + imm);
        keep_parsing = syms_true;
      }break;
      
      // lea imm8/imm32,$rsp
      case 0x8D:
      {
        // read source register
        SYMS_U8 modrm = 0;
        syms_based_range_read_struct(inst_base, inst_range, off, &modrm);
        SYMS_UnwindPeX64GprReg gpr_reg = (modrm & 7) + (rex & 1)*8;
        SYMS_Reg64 *reg = syms_unwind_pe_x64__gpr_reg(regs, gpr_reg);
        SYMS_U64 reg_value = reg->u64;
        
        // advance to the immediate
        off += 1;
        
        SYMS_S32 imm = 0;
        // read 1-byte immediate
        if ((modrm >> 6) == 1){
          SYMS_S8 imm8 = 0;
          syms_based_range_read_struct(inst_base, inst_range, off, &imm8);
          imm = imm8;
          off += 1;
        }
        
        // read 4-byte immediate
        else{
          syms_based_range_read_struct(inst_base, inst_range, off, &imm);
          off += 4;
        }
        
        regs->rsp.u64 = (SYMS_U64)(reg_value + imm);
        keep_parsing = syms_true;
      }break;
      
      // ret $nn
      case 0xC2:
      {
        // read new ip
        SYMS_U64 sp = regs->rsp.u64;
        SYMS_U64 new_ip = 0;
        if (!syms_memory_view_read_struct(memview, sp, &new_ip)){
          missed_read_addr = sp;
          goto error_out;
        }
        
        // read 2-byte immediate & advance stack pointer
        SYMS_U16 imm = 0;
        syms_based_range_read_struct(inst_base, inst_range, off, &imm);
        SYMS_U64 new_sp = sp + 8 + imm;
        
        // commit registers
        regs->rip.u64 = new_ip;
        regs->rsp.u64 = new_sp;
      }break;
      
      // ret / rep; ret
      case 0xF3:
      {
        SYMS_ASSERT(!"Hit me!");
      }
      case 0xC3:
      {
        // read new ip
        SYMS_U64 sp = regs->rsp.u64;
        SYMS_U64 new_ip = 0;
        if (!syms_memory_view_read_struct(memview, sp, &new_ip)){
          missed_read_addr = sp;
          goto error_out;
        }
        
        // advance stack pointer
        SYMS_U64 new_sp = sp + 8;
        
        // commit registers
        regs->rip.u64 = new_ip;
        regs->rsp.u64 = new_sp;
      }break;
      
      // jmp nnnn
      case 0xE9:
      {
        SYMS_ASSERT(!"Hit Me");
        // TODO(allen): general idea: read the immediate, move the ip, leave the sp, done
        // we don't have any cases to exercise this right now. no guess implementation!
      }break;
      
      // jmp n
      case 0xEB:
      {
        SYMS_ASSERT(!"Hit Me");
        // TODO(allen): general idea: read the immediate, move the ip, leave the sp, done
        // we don't have any cases to exercise this right now. no guess implementation!
      }break;
    }
  }
  
  error_out:;
  
  if (missed_read_addr != 0){
    SYMS_Log("Memory read miss: 0x%llx\n", missed_read_addr);
    syms_unwind_result_missed_read(&result, missed_read_addr);
  }
  
  SYMS_LogClose(log);
  
  return(result);
}


////////////////////////////////
// NOTE(allen): PE-x64 Helper Functions

SYMS_API SYMS_U32
syms_unwind_pe_x64__slot_count_from_op_code(SYMS_UnwindPeOpCode op_code){
  SYMS_U32 result = 0;
  switch (op_code){
    case SYMS_UnwindPeOpCode_PUSH_NONVOL:     result = 1; break;
    case SYMS_UnwindPeOpCode_ALLOC_LARGE:     result = 2; break;
    case SYMS_UnwindPeOpCode_ALLOC_SMALL:     result = 1; break;
    case SYMS_UnwindPeOpCode_SET_FPREG:       result = 1; break;
    case SYMS_UnwindPeOpCode_SAVE_NONVOL:     result = 2; break;
    case SYMS_UnwindPeOpCode_SAVE_NONVOL_FAR: result = 3; break;
    case SYMS_UnwindPeOpCode_EPILOG:          result = 2; break;
    case SYMS_UnwindPeOpCode_SPARE_CODE:      result = 3; break;
    case SYMS_UnwindPeOpCode_SAVE_XMM128:     result = 2; break;
    case SYMS_UnwindPeOpCode_SAVE_XMM128_FAR: result = 3; break;
    case SYMS_UnwindPeOpCode_PUSH_MACHFRAME:  result = 1; break;
  }
  return(result);
}

SYMS_API SYMS_B32
syms_unwind_pe_x64__voff_is_in_epilog(SYMS_String8 bin_data, SYMS_PeBinAccel *bin, SYMS_U64 voff,
                                      SYMS_PeIntelPdata *final_pdata){
  // NOTE(allen): There are restrictions placed on how an epilog is allowed
  // to be formed (https://docs.microsoft.com/en-us/cpp/build/prolog-and-epilog?view=msvc-160)
  // Here we interpret machine code directly according to the rules
  // given there to determine if the code we're looking at looks like an epilog.
  
  // TODO(allen): Figure out how to verify this.
  
  //- setup parsing context
  SYMS_U64 sec_number = syms_pe_sec_number_from_voff(bin_data, bin, voff);
  SYMS_CoffSectionHeader *sec = syms_pecoff_sec_hdr_from_n(bin_data, bin->section_array_off, sec_number);
  void*         inst_base  = syms_pe_ptr_from_sec_number(bin_data, bin, sec_number);
  SYMS_U64Range inst_range = {0, sec->virt_size};
  
  //- setup parsing variables
  SYMS_B32 is_epilog = syms_false;
  SYMS_B32 keep_parsing = syms_true;
  SYMS_U64 off = voff - sec->virt_off;
  
  //- check first instruction
  {
    SYMS_U8 inst[4];
    if (!syms_based_range_read(inst_base, inst_range, off, 4, inst)){
      keep_parsing = syms_false;
    }
    else{
      if ((inst[0] & 0xF8) == 0x48){
        switch (inst[1]){
          // add $nnnn,%rsp
          case 0x81:
          {
            if (inst[0] == 0x48 && inst[2] == 0xC4){
              off += 7;
            }
            else{
              keep_parsing = syms_false;
            }
          }break;
          
          // add $n,%rsp
          case 0x83:
          {
            if (inst[0] == 0x48 && inst[2] == 0xC4){
              off += 4;
            }
            else{
              keep_parsing = syms_false;
            }
          }break;
          
          // lea n(reg),%rsp
          case 0x8D:
          {
            if ((inst[0] & 0x06) == 0 &&
                ((inst[2] >> 3) & 0x07) == 0x04 &&
                (inst[2] & 0x07) != 0x04){
              SYMS_U8 imm_size = (inst[2] >> 6);
              // 1-byte immediate
              if (imm_size == 1){
                off += 4;
              }
              // 4-byte immediate
              else if (imm_size == 2){
                off += 7;
              }
              else{
                keep_parsing = syms_false;
              }
            }
            else{
              keep_parsing = syms_false;
            }
          }break;
        }
      }
    }
  }
  
  //- parsing loop
  if (keep_parsing){
    for (;;){
      // read inst
      SYMS_U8 inst_byte = 0;
      if (!syms_based_range_read_struct(inst_base, inst_range, off, &inst_byte)){
        goto loop_break;
      }
      
      // when (... I don't know ...) rely on the next byte
      SYMS_U64 check_off = off;
      SYMS_U8 check_inst_byte = inst_byte;
      if ((inst_byte & 0xF0) == 0x40){
        check_off = off + 1;
        if (!syms_based_range_read_struct(inst_base, inst_range, check_off, &check_inst_byte)){
          goto loop_break;
        }
      }
      
      switch (check_inst_byte){
        // pop
        case 0x58:case 0x59:case 0x5A:case 0x5B:
        case 0x5C:case 0x5D:case 0x5E:case 0x5F:
        {
          off = check_off + 1;
        }break;
        
        // ret
        case 0xC2:case 0xC3:
        { 
          is_epilog = syms_true;
          goto loop_break;
        }break;
        
        // jmp nnnn
        case 0xE9:
        {
          SYMS_U64 imm_off = check_off + 1;
          SYMS_S32 imm = 0;
          if (!syms_based_range_read_struct(inst_base, inst_range, imm_off, &imm)){
            goto loop_break;
          }
          
          SYMS_U64 next_off = (SYMS_U64)(imm_off + sizeof(imm) + imm);
          if (!(final_pdata->voff_first <= next_off && next_off < final_pdata->voff_one_past_last)){
            goto loop_break;
          }
          
          off = next_off;
          // TODO(allen): why isn't this just the end of the epilog?
        }break;
        
        // rep; ret (for amd64 prediction bug)
        case 0xF3:
        {
          SYMS_U8 next_inst_byte = 0;
          syms_based_range_read_struct(inst_base, inst_range, off, &next_inst_byte);
          is_epilog = (next_inst_byte == 0xC3);
          goto loop_break;
        }break;
        
        default: goto loop_break;
      }
    }
    
    loop_break:;
  }
  
  //- fill result
  SYMS_B32 result = is_epilog;
  return(result);
}

SYMS_API SYMS_Reg64*
syms_unwind_pe_x64__gpr_reg(SYMS_RegX64 *regs, SYMS_UnwindPeX64GprReg reg_id){
  SYMS_LOCAL SYMS_Reg64 dummy = {0};
  SYMS_Reg64 *result = &dummy;
  switch (reg_id){
    case SYMS_UnwindPeX64GprReg_RAX: result = &regs->rax; break;
    case SYMS_UnwindPeX64GprReg_RCX: result = &regs->rcx; break;
    case SYMS_UnwindPeX64GprReg_RDX: result = &regs->rdx; break;
    case SYMS_UnwindPeX64GprReg_RBX: result = &regs->rbx; break;
    case SYMS_UnwindPeX64GprReg_RSP: result = &regs->rsp; break;
    case SYMS_UnwindPeX64GprReg_RBP: result = &regs->rbp; break;
    case SYMS_UnwindPeX64GprReg_RSI: result = &regs->rsi; break;
    case SYMS_UnwindPeX64GprReg_RDI: result = &regs->rdi; break;
    case SYMS_UnwindPeX64GprReg_R8 : result = &regs->r8 ; break;
    case SYMS_UnwindPeX64GprReg_R9 : result = &regs->r9 ; break;
    case SYMS_UnwindPeX64GprReg_R10: result = &regs->r10; break;
    case SYMS_UnwindPeX64GprReg_R11: result = &regs->r11; break;
    case SYMS_UnwindPeX64GprReg_R12: result = &regs->r12; break;
    case SYMS_UnwindPeX64GprReg_R13: result = &regs->r13; break;
    case SYMS_UnwindPeX64GprReg_R14: result = &regs->r14; break;
    case SYMS_UnwindPeX64GprReg_R15: result = &regs->r15; break;
  }
  return(result);
}

#endif //SYMS_UNWIND_PE_X64_C
