// Copyright Epic Games, Inc. All Rights Reserved.
/* date = September 7th 2021 0:14 pm */

#ifndef SYMS_UNWIND_PE_X64_H
#define SYMS_UNWIND_PE_X64_H

////////////////////////////////
// NOTE(allen): PE Unwind Types

typedef SYMS_U32 SYMS_UnwindPeOpCode;
enum{
  SYMS_UnwindPeOpCode_PUSH_NONVOL = 0,
  SYMS_UnwindPeOpCode_ALLOC_LARGE = 1,
  SYMS_UnwindPeOpCode_ALLOC_SMALL = 2,
  SYMS_UnwindPeOpCode_SET_FPREG   = 3,
  SYMS_UnwindPeOpCode_SAVE_NONVOL = 4,
  SYMS_UnwindPeOpCode_SAVE_NONVOL_FAR = 5,
  SYMS_UnwindPeOpCode_EPILOG      = 6,
  SYMS_UnwindPeOpCode_SPARE_CODE  = 7,
  SYMS_UnwindPeOpCode_SAVE_XMM128 = 8,
  SYMS_UnwindPeOpCode_SAVE_XMM128_FAR = 9,
  SYMS_UnwindPeOpCode_PUSH_MACHFRAME = 10,
};

#define SYMS_UnwindPeCode_CodeFromFlags(x)  ((x) & 0xf)
#define SYMS_UnwindPeCode_InfoFromFlags(x) (((x) & 0xf0) >> 4)

typedef union SYMS_UnwindPeCode{
  struct{
    SYMS_U8 off_in_prolog;
    SYMS_U8 flags;
  };
  SYMS_U16 u16;
} SYMS_UnwindPeCode;

typedef SYMS_U8 SYMS_UnwindPeInfoFlags;
enum{
  SYMS_UnwindPeInfoFlag_EHANDLER = (1 << 0),
  SYMS_UnwindPeInfoFlag_UHANDLER = (1 << 1),
  SYMS_UnwindPeInfoFlag_FHANDLER = SYMS_UnwindPeInfoFlag_EHANDLER|SYMS_UnwindPeInfoFlag_UHANDLER,
  SYMS_UnwindPeInfoFlag_CHAINED  = (1 << 2),
};

#define SYMS_UnwindPeInfo_VersionFromHeader(x) ((x) & 0x07)
#define SYMS_UnwindPeInfo_FlagsFromHeader(x)  (((x) & 0xf8) >> 3)
#define SYMS_UnwindPeInfo_RegFromFrame(x)      ((x) & 0x0f)
#define SYMS_UnwindPeInfo_OffFromFrame(x)     (((x) & 0xf0) >> 4)

typedef struct SYMS_UnwindPeInfo{
  SYMS_U8 header;
  SYMS_U8 prolog_size;
  SYMS_U8 codes_num;
  SYMS_U8 frame;
} SYMS_UnwindPeInfo;

////////////////////////////////
// NOTE(allen): PE-x64 Unwind Types

typedef SYMS_U8 SYMS_UnwindPeX64GprReg;
enum{
  SYMS_UnwindPeX64GprReg_RAX = 0,
  SYMS_UnwindPeX64GprReg_RCX = 1,
  SYMS_UnwindPeX64GprReg_RDX = 2,
  SYMS_UnwindPeX64GprReg_RBX = 3,
  SYMS_UnwindPeX64GprReg_RSP = 4,
  SYMS_UnwindPeX64GprReg_RBP = 5,
  SYMS_UnwindPeX64GprReg_RSI = 6,
  SYMS_UnwindPeX64GprReg_RDI = 7,
  SYMS_UnwindPeX64GprReg_R8  = 8,
  SYMS_UnwindPeX64GprReg_R9  = 9,
  SYMS_UnwindPeX64GprReg_R10 = 10,
  SYMS_UnwindPeX64GprReg_R11 = 11,
  SYMS_UnwindPeX64GprReg_R12 = 12,
  SYMS_UnwindPeX64GprReg_R13 = 13,
  SYMS_UnwindPeX64GprReg_R14 = 14,
  SYMS_UnwindPeX64GprReg_R15 = 15,
};

SYMS_C_LINKAGE_BEGIN

////////////////////////////////
// NOTE(allen): PE-x64 Unwind Function

SYMS_API SYMS_UnwindResult syms_unwind_pe_x64(SYMS_String8 bin_data, SYMS_PeBinAccel *bin, SYMS_U64 bin_base,
                                              SYMS_MemoryView *memview, SYMS_RegX64 *regs_in_out);

SYMS_API SYMS_UnwindResult syms_unwind_pe_x64__epilog(SYMS_String8 bin_data, SYMS_PeBinAccel *bin, SYMS_U64 base,
                                                      SYMS_MemoryView *memview, SYMS_RegX64 *regs_in_out);


////////////////////////////////
// NOTE(allen): PE-x64 Helper Functions

SYMS_API SYMS_U32 syms_unwind_pe_x64__slot_count_from_op_code(SYMS_UnwindPeOpCode op_code);
SYMS_API SYMS_B32 syms_unwind_pe_x64__voff_is_in_epilog(SYMS_String8 bin_data, SYMS_PeBinAccel *bin, SYMS_U64 voff,
                                                        SYMS_PeIntelPdata *final_pdata);

SYMS_API SYMS_Reg64* syms_unwind_pe_x64__gpr_reg(SYMS_RegX64 *regs, SYMS_UnwindPeX64GprReg reg_id);

SYMS_C_LINKAGE_END

#endif //SYMS_UNWIND_PE_X64_H
