// Copyright Epic Games, Inc. All Rights Reserved.

#ifndef SYMS_CRT_OVERRIDES_C
#define SYMS_CRT_OVERRIDES_C

////////////////////////////////
//~ allen: use default helpers
#include "syms_default_helpers.c"

////////////////////////////////
//~ allen: use default arena
#include "syms_default_arena.c"

////////////////////////////////
//~ allen: use default scrtach
#include "syms_default_scratch.c"

#endif // SYMS_CRT_OVERRIDES_C
