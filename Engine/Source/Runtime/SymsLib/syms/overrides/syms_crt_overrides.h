// Copyright Epic Games, Inc. All Rights Reserved.

#ifndef SYMS_CRT_DEFAULT_OVERRIDES_H
#define SYMS_CRT_DEFAULT_OVERRIDES_H

#if !defined(SYMS_OVERRIDE_FUNC)
# define SYMS_OVERRIDE_FUNC
#endif

// NOTE(allen): basic types

#ifdef _MSC_VER

typedef __int8 SYMS_S8;
typedef __int16 SYMS_S16;
typedef __int32 SYMS_S32;
typedef __int64 SYMS_S64;
typedef unsigned __int8 SYMS_U8;
typedef unsigned __int16 SYMS_U16;
typedef unsigned __int32 SYMS_U32;
typedef unsigned __int64 SYMS_U64;

#define SYMS_PRIu64 "I64u"
#define SYMS_PRId64 "I64d"
#define SYMS_PRIx64 "I64x"

#else

#include <stdint.h>
#include <inttypes.h>

typedef int8_t SYMS_S8;
typedef int16_t SYMS_S16;
typedef int32_t SYMS_S32;
typedef int64_t SYMS_S64;
typedef uint8_t SYMS_U8;
typedef uint16_t SYMS_U16;
typedef uint32_t SYMS_U32;
typedef uint64_t SYMS_U64;

#define SYMS_PRIu64 PRIu64
#define SYMS_PRId64 PRId64
#define SYMS_PRIx64 PRIx64

#endif

#define SYMS_S8 SYMS_S8
#define SYMS_S16 SYMS_S16
#define SYMS_S32 SYMS_S32
#define SYMS_S64 SYMS_S64
#define SYMS_U8 SYMS_U8
#define SYMS_U16 SYMS_U16
#define SYMS_U32 SYMS_U32
#define SYMS_U64 SYMS_U64

// NOTE(allen): asserts
#include <assert.h>
#define SYMS_ASSERT_BREAK(m) assert(!(#m))

////////////////////////////////
//~ allen: memory helpers
#include <stdlib.h>
#include <string.h>

#define syms_memmove memmove
#define syms_memset memset
#define syms_memcmp memcmp
#define syms_strlen strlen
#define syms_memisnull(ptr,sz) (memchr(ptr,0,sz)!=NULL)

////////////////////////////////
//~ allen: memory
#define syms_mem_reserve(s)    malloc(s)
#define syms_mem_commit(p,s)   (1)
#define syms_mem_decommit(p,s) ((void)0)
#define syms_mem_release(p,s)  free(p)

// @notes Since we are relying on malloc & free, and not using commit/decommit,
//  we want a smaller reserve chunk size than the default, because malloc
//  automatically reserves and commits everything it allocates. And wasting too
//  much commit isn't very great. We also want to set the commit size equal to
//  the reserve size to avoid wasting time on commit logic that won't matter.
#define SYMS_ARENA_RESERVE_SIZE (1 << 20)
#define SYMS_ARENA_COMMIT_SIZE  (1 << 20)

////////////////////////////////
//~ allen: use default arena
#include "syms_default_arena.h"

////////////////////////////////
//~ allen: use default scrtach
#include "syms_default_scratch.h"

#endif // SYMS_CRT_DEFAULT_OVERRIDES_H
