// Copyright Epic Games, Inc. All Rights Reserved.

#ifndef SYMS_DEFAULT_ARENA_C
#define SYMS_DEFAULT_ARENA_C

SYMS_OVERRIDE_FUNC SYMS_DefArena*
syms_arena_def_alloc__sized(SYMS_U64 res, SYMS_U64 cmt){
  SYMS_ASSERT(SYMS_ARENA_HEADER_SIZE < cmt && cmt <= res);
  SYMS_Arena *result = 0;
  void *memory = syms_mem_reserve(res);
  if (memory != 0){
    if (syms_mem_commit(memory, cmt)){
      result = (SYMS_Arena*)memory;
      result->prev = 0;
      result->current = result;
      result->base_pos = 0;
      result->pos = SYMS_ARENA_HEADER_SIZE;
      result->cmt = cmt;
      result->cap = res;
      result->align = 8;
      result->dev = 0;
    }
    else{
      syms_mem_release(memory, res);
    }
  }
  return(result);
}

SYMS_OVERRIDE_FUNC SYMS_DefArena*
syms_arena_def_alloc(void){
  SYMS_DefArena *result = syms_arena_def_alloc__sized(SYMS_ARENA_RESERVE_SIZE, SYMS_ARENA_COMMIT_SIZE);
  return(result);
}

SYMS_OVERRIDE_FUNC void
syms_arena_def_release(SYMS_DefArena *arena){
  for (SYMS_DefArena *node = arena->current, *prev = 0;
       node != 0;
       node = prev){
    prev = node->prev;
    syms_mem_release(node, node->cap);
  }
}

#define SYMS_ARENA_VERY_LARGE ((SYMS_ARENA_RESERVE_SIZE - SYMS_ARENA_HEADER_SIZE)/2) + 1

SYMS_OVERRIDE_FUNC void*
syms_arena_def_push(SYMS_DefArena *arena, SYMS_U64 size){
  void *result = 0;
  
  // get new pos
  SYMS_Arena *current = arena->current;
  SYMS_U64 pos_aligned = SYMS_AlignPow2(current->pos, arena->align);
  SYMS_U64 pos_new = pos_aligned + size;
  
  // check cap
  if (current->cap < pos_new){
    
    // normal growth path
    SYMS_Arena *new_block = 0;
    if (size < SYMS_ARENA_VERY_LARGE){
      new_block = syms_arena_def_alloc();
    }
    // "very large" growth path
    else{
      SYMS_U64 size_aligned = SYMS_AlignPow2(size + SYMS_ARENA_HEADER_SIZE, (4 << 10));
      new_block = syms_arena_def_alloc__sized(size_aligned, size_aligned);
    }
    
    // connect block to chain
    if (new_block != 0){
      new_block->base_pos = current->base_pos + current->cap;
      SYMS_StackPush_N(arena->current, new_block, prev);
      
      // recompute the new pos
      current = new_block;
      pos_aligned = SYMS_AlignPow2(current->pos, arena->align);
      pos_new = pos_aligned + size;
    }
  }
  
  // cap big enough?
  if (current->cap >= pos_new){
    
    // cmt too small?
    if (current->cmt < pos_new){
      
      // get new cmt
      SYMS_U64 cmt_new_unclamped = SYMS_AlignPow2(pos_new, SYMS_ARENA_COMMIT_SIZE);
      SYMS_U64 cmt_new = SYMS_ClampTop(cmt_new_unclamped, current->cap);
      SYMS_U64 cmt_size = cmt_new - current->cmt;
      
      // try commit
      if (syms_mem_commit((SYMS_U8*)current + current->cmt, cmt_size)){
        current->cmt = cmt_new;
      }
    }
    
    // cmt big enough?
    if (current->cmt >= pos_new){
      
      // get result & advance pos
      result = (SYMS_U8*)current + pos_aligned;
      current->pos = pos_new;
    }
  }
  
  SYMS_ASSERT(arena->current != 0);
  
  // dev
#if SYMS_ENABLE_DEV_ARENA
  syms_arena_dev_push__impl(arena, size, result);
#endif
  
  return(result);
}

SYMS_OVERRIDE_FUNC SYMS_U64
syms_arena_def_pos(SYMS_Arena *arena){
  SYMS_Arena *current = arena->current;
  SYMS_U64 result = current->base_pos + current->pos;
  return(result);
}

SYMS_OVERRIDE_FUNC void
syms_arena_def_pop_to(SYMS_DefArena *arena, SYMS_U64 pos_unclamped){
  SYMS_U64 big_pos = SYMS_ClampBot(SYMS_ARENA_HEADER_SIZE, pos_unclamped);
  
  // unroll the chain
  SYMS_Arena *current = arena->current;
  for (SYMS_Arena *prev = 0;
       current->base_pos >= big_pos;
       current = prev){
    prev = current->prev;
    syms_mem_release(current, current->cap);
  }
  arena->current = current;
  
  // fix the pos
  {
    SYMS_U64 pos_unclamped = big_pos - current->base_pos;
    SYMS_U64 pos = SYMS_ClampBot(SYMS_ARENA_HEADER_SIZE, pos_unclamped);
    if (pos < current->pos){
      current->pos = pos;
    }
  }
  
  SYMS_ASSERT(arena->current != 0);
  
  // dev
#if SYMS_ENABLE_DEV_ARENA
  syms_arena_dev_pop_to__impl(arena, pos_unclamped);
#endif
}

SYMS_OVERRIDE_FUNC void
syms_arena_def_set_auto_align(SYMS_DefArena *arena, SYMS_U64 pow2_align){
  arena->align = pow2_align;
}

SYMS_OVERRIDE_FUNC void
syms_arena_def_absorb(SYMS_DefArena *arena, SYMS_DefArena *sub){
  // base adjustment
  SYMS_DefArena *current = arena->current;
  SYMS_U64 base_adjust = current->base_pos + current->cap;
  for (SYMS_DefArena *node = sub->current;
       node != 0;
       node = node->prev){
    node->base_pos += base_adjust;
  }
  
  // attach sub to arena
  sub->prev = arena->current;
  arena->current = sub->current;
  sub->current = sub;
  
  // dev
#if SYMS_ENABLE_DEV_ARENA
  syms_arena_dev_absorb__impl(arena, sub);
#endif
}

#endif //SYMS_DEFAULT_ARENA_C
