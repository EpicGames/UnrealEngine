// Copyright Epic Games, Inc. All Rights Reserved.

#ifndef SYMS_DEFAULT_ARENA_H
#define SYMS_DEFAULT_ARENA_H

// NOTE(allen): Statically "constructs" an implementation for syms's arena override macros from
// reserve/commit/decommit/release macros

#if !defined(syms_mem_reserve)
# error SYMS: syms_mem_reserve was not #define'd.
#endif
#if !defined(syms_mem_commit)
# error SYMS: syms_mem_commit was not #define'd.
#endif
#if !defined(syms_mem_decommit)
# error SYMS: syms_mem_decommit was not #define'd.
#endif
#if !defined(syms_mem_release)
# error SYMS: syms_mem_release was not #define'd.
#endif

#if !defined(SYMS_ENABLE_DEV_ARENA)
# define SYMS_ENABLE_DEV_ARENA 0
#endif

////////////////////////////////
// NOTE(allen): Arena Definition

#define SYMS_ARENA_HEADER_SIZE 128

// @usage @important When overriding these macros the following should hold
// SYMS_ARENA_HEADER_SIZE < SYMS_ARENA_COMMIT_SIZE
// SYMS_ARENA_COMMIT_SIZE <= SYMS_ARENA_RESERVE_SIZE

#if !defined(SYMS_ARENA_RESERVE_SIZE)
# define SYMS_ARENA_RESERVE_SIZE (64 << 20)
#endif
#if !defined(SYMS_ARENA_COMMIT_SIZE)
# define SYMS_ARENA_COMMIT_SIZE (64 << 10)
#endif

typedef struct SYMS_DefArena{
  struct SYMS_DefArena *prev;
  struct SYMS_DefArena *current;
  SYMS_U64 base_pos;
  SYMS_U64 pos;
  SYMS_U64 cmt;
  SYMS_U64 cap;
  SYMS_U64 align;
  struct SYMS_ArenaDev *dev;
} SYMS_DefArena;

SYMS_OVERRIDE_FUNC SYMS_DefArena*   syms_arena_def_alloc__sized(SYMS_U64 init_res, SYMS_U64 init_cmt);
SYMS_OVERRIDE_FUNC SYMS_DefArena*   syms_arena_def_alloc(void);
SYMS_OVERRIDE_FUNC void             syms_arena_def_release(SYMS_DefArena *arena);
SYMS_OVERRIDE_FUNC void*            syms_arena_def_push(SYMS_DefArena *arena, SYMS_U64 size);
SYMS_OVERRIDE_FUNC void             syms_arena_def_pop_to(SYMS_DefArena *arena, SYMS_U64 pos);
SYMS_OVERRIDE_FUNC void             syms_arena_def_set_auto_align(SYMS_DefArena *arena, SYMS_U64 pow2_align);
SYMS_OVERRIDE_FUNC void             syms_arena_def_absorb(SYMS_DefArena *arena, SYMS_DefArena *sub);

#define SYMS_Arena SYMS_DefArena
#define syms_arena_alloc__impl          syms_arena_def_alloc
#define syms_arena_release__impl        syms_arena_def_release
#define syms_arena_get_pos__impl        syms_arena_def_pos
#define syms_arena_push__impl           syms_arena_def_push
#define syms_arena_pop_to__impl         syms_arena_def_pop_to
#define syms_arena_set_auto_align__impl syms_arena_def_set_auto_align
#define syms_arena_absorb__impl         syms_arena_def_absorb
#define syms_arena_tidy__impl(a)        ((void)(a))

#endif //SYMS_DEFAULT_ARENA_H
