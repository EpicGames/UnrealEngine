////////////////////////////////
//~ allen: memory helpers

static SYMS_U32
syms_override_memisnull(void *ptr, SYMS_U64 size){
  SYMS_U32 result = 1;
  
  // break down size
  SYMS_U64 extra = (size&0x7);
  SYMS_U64 count8 = (size >> 3);

  SYMS_U8 *p8;

  // check with 8-byte stride
  SYMS_U64 *p64 = (SYMS_U64*)ptr;
  for (SYMS_U64 i = 0; i < count8; i += 1, p64 += 1){
    if (*p64 != 0){
      result = 0;
      goto done;
    }
  }
  
  // check extra
  p8 = (SYMS_U8*)p64;
  for (SYMS_U64 i = 0; i < extra; i += 1, p8 += 1){
    if (*p8 != 0){
      result = 0;
      goto done;
    }
  }
  
  done:;
  return(result);
}
