// Copyright Epic Games, Inc. All Rights Reserved.

#ifndef SYMS_DEFAULT_HELPERS_H
#define SYMS_DEFAULT_HELPERS_H

////////////////////////////////
//~ allen: memory helpers

static SYMS_U32 syms_override_memisnull(void*ptr, SYMS_U64 size);

#endif //SYMS_DEFAULT_HELPERS_H
