// Copyright Epic Games, Inc. All Rights Reserved.

#ifndef SYMS_DEFAULT_OVERRIDES_C
#define SYMS_DEFAULT_OVERRIDES_C

// NOTE(allen): get os specific override implementations
#if SYMS_OS_WINDOWS
# include "syms_win32_overrides.c"
#elif SYMS_OS_LINUX
# include "syms_unix_overrides.c"
#else
# error No 'default' implementation for this OS
#endif

#endif
