// Copyright Epic Games, Inc. All Rights Reserved.

#ifndef SYMS_DEFAULT_SCRATCH_C
#define SYMS_DEFAULT_SCRATCH_C

////////////////////////////////
// NOTE(allen): Scratch

SYMS_THREAD_LOCAL SYMS_DefaultScratchPool syms_scratch_pool = {0};

SYMS_OVERRIDE_FUNC SYMS_Arena*
syms_default_get_implicit_thread_arena(SYMS_Arena **conflicts, SYMS_U64 conflict_count){
  // init pool if first time
  if (syms_scratch_pool.arenas[0] == 0){
    for (SYMS_U64 i = 0; i < SYMS_ARRAY_SIZE(syms_scratch_pool.arenas); i += 1){
      syms_scratch_pool.arenas[i] = syms_arena_alloc();
    }
  }
  
  // grab local pointer
  SYMS_DefaultScratchPool *tctx = &syms_scratch_pool;
  
  // get compatible arena
  SYMS_Arena *result = 0;
  {
    SYMS_Arena **arena_ptr = tctx->arenas;
    for (SYMS_U64 i = 0; i < SYMS_ARRAY_SIZE(tctx->arenas); i += 1, arena_ptr += 1){
      SYMS_B32 has_conflict = 0;
      SYMS_Arena **conflict_ptr = conflicts;
      for (SYMS_U64 j = 0; j < conflict_count; j += 1, conflict_ptr += 1){
        if (*conflict_ptr == *arena_ptr){
          has_conflict = 1;
          break;
        }
      }
      if (!has_conflict){
        result = *arena_ptr;
        break;
      }
    }
  }
  
  return(result);
}

#endif //SYMS_DEFAULT_SCRATCH_C
