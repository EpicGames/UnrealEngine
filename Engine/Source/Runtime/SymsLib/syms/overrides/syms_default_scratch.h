// Copyright Epic Games, Inc. All Rights Reserved.

#ifndef SYMS_DEFAULT_SCRATCH_H
#define SYMS_DEFAULT_SCRATCH_H

////////////////////////////////
// NOTE(allen): Implicit Thread Context Type

typedef struct SYMS_DefaultScratchPool{
  SYMS_Arena *arenas[2];
} SYMS_DefaultScratchPool;

////////////////////////////////
// NOTE(allen): Scratch Override

SYMS_OVERRIDE_FUNC SYMS_Arena* syms_default_get_implicit_thread_arena(SYMS_Arena **conflicts, SYMS_U64 conflict_count);
#define syms_get_implicit_thread_arena__impl syms_default_get_implicit_thread_arena
#define syms_scratch_pool_tidy__impl()       ((void)0)

#endif //SYMS_DEFAULT_SCRATCH_H
