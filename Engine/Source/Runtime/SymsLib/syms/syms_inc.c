// Copyright Epic Games, Inc. All Rights Reserved.

#ifndef SYMS_INC_C
#define SYMS_INC_C

////////////////////////////////
//~ NOTE(allen): Include the whole core Syms library

// base & common type definitions
#include "syms/core/base/syms_base.c"
#include "syms/core/syms_debug_info.c"

// eval
#include "syms/core/syms_eval.c"

// "windows related" formats
#include "syms/core/pe/syms_coff.c"
#include "syms/core/pe/syms_pe.c"
#include "syms/core/pdb/syms_cv.c"
#include "syms/core/pdb/syms_pdb.c"

// "linux related" foramts
#include "syms/core/elf/syms_elf.c"
#include "syms/core/dwarf/syms_dwarf.c"

// "mach-o related" formats
#include "syms/core/mach/syms_mach.c"

// "windows related" parsers
#include "syms/core/pe/syms_pecoff_helpers.c"
#include "syms/core/pe/syms_pe_parser.c"
#include "syms/core/pdb/syms_msf_parser.c"
#include "syms/core/pdb/syms_cv_helpers.c"
#include "syms/core/pdb/syms_pdb_parser.c"

// "mach-o related" parsers
#include "syms/core/mach/syms_mach_parser.c"

// "linux related" parsers
#include "syms/core/elf/syms_elf_parser.c"
#include "syms/core/dwarf/syms_dwarf_expr.c"
#include "syms/core/dwarf/syms_dwarf_parser.c"
#include "syms/core/dwarf/syms_dwarf_transpiler.c"

// parser abstraction
#include "syms/core/syms_parser.c"
#include "syms/core/syms_parser_invariants.c"
#include "syms/core/data_structures/syms_data_structures.c"
#include "syms/core/group/syms_type_graph.c"
#include "syms/core/group/syms_functions.c"
#include "syms/core/group/syms_group.c"
#include "syms/core/file_inf/syms_file_inf.c"

// regs
#include "syms/core/regs/syms_regs.c"
#include "syms/core/regs/syms_regs_helpers.c"

// depends on "syms_dwarf_expr" and "regs"
#include "syms/core/regs/syms_dwarf_regs_helper.c"

// unwinders
#include "syms/core/unwind/syms_unwind_pe_x64.c"
#include "syms/core/unwind/syms_unwind_elf_x64.c"

// serialized type information
#include "syms/syms_serial_inc.c"

// extras
#include "syms/core/extras/syms_aggregate_proc_map.c"

#endif // SYMS_INC_C
