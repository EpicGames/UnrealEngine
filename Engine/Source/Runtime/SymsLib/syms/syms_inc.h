// Copyright Epic Games, Inc. All Rights Reserved.
/* date = March 31st 2021 1:37 pm */

#ifndef SYMS_INC_H
#define SYMS_INC_H

////////////////////////////////
//~ NOTE(allen): Include the whole core Syms library

// base & common type definitions
#include "syms/core/base/syms_base.h"
#include "syms/core/syms_debug_info.h"

// regs
#include "syms/core/regs/syms_regs.h"
#pragma pack(push, 1)
#include "syms/core/generated/syms_meta_regs_x86.h"
#include "syms/core/generated/syms_meta_regs_x64.h"
#pragma pack(pop)
#include "syms/core/regs/syms_regs_helpers.h"

// eval
#include "syms/core/syms_eval.h"

// "windows related" formats
#include "syms/core/pe/syms_coff.h"
#include "syms/core/pe/syms_pe.h"
#include "syms/core/pdb/syms_cv.h"
#include "syms/core/pdb/syms_msf.h"
#include "syms/core/pdb/syms_pdb.h"

// "linux related" formats
#include "syms/core/elf/syms_elf.h"
#include "syms/core/dwarf/syms_dwarf.h"

// "mach-o related" formats
#include "syms/core/mach/syms_mach.h"

// "windows related" parsers
#include "syms/core/pe/syms_pecoff_helpers.h"
#include "syms/core/pe/syms_pe_parser.h"
#include "syms/core/pdb/syms_msf_parser.h"
#include "syms/core/pdb/syms_cv_helpers.h"
#include "syms/core/pdb/syms_pdb_parser.h"

// "mach-o related" parsers
#include "syms/core/mach/syms_mach_parser.h"

// "linux related" parsers
#include "syms/core/elf/syms_elf_parser.h"
#include "syms/core/dwarf/syms_dwarf_expr.h"
#include "syms/core/dwarf/syms_dwarf_parser.h"
#include "syms/core/dwarf/syms_dwarf_transpiler.h"

// parser abstraction
#include "syms/core/syms_parser.h"
#include "syms/core/syms_parser_invariants.h"
#include "syms/core/data_structures/syms_data_structures.h"
#include "syms/core/group/syms_type_graph.h"
#include "syms/core/group/syms_functions.h"
#include "syms/core/group/syms_group.h"
#include "syms/core/file_inf/syms_file_inf.h"

#include "syms/core/regs/syms_regs_x64.h"
// depends on "syms_dwarf_expr" and "regs"
#include "syms/core/regs/syms_dwarf_regs_helper.h"

// unwinders
#include "syms/core/unwind/syms_unwind_pe_x64.h"
#include "syms/core/unwind/syms_unwind_elf_x64.h"

// serialized type information
#include "syms/syms_serial_inc.h"

// extras
#include "syms/core/extras/syms_aggregate_proc_map.h"

#endif // SYMS_INC_H
