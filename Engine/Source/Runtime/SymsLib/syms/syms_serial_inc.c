// Copyright Epic Games, Inc. All Rights Reserved.

#ifndef SYMS_SERIAL_INC_C
#define SYMS_SERIAL_INC_C

////////////////////////////////
// NOTE(allen): Extended Serial Info
#include "syms/core/base/syms_serial.c"

#endif // SYMS_SERIAL_INC_C
