// Copyright Epic Games, Inc. All Rights Reserved.
// symslib //
#pragma once

#define SYMS_API

#ifdef __cplusplus
extern "C" {
#endif

#define SYMS_READ_ONLY
#include "syms/overrides/syms_crt_overrides.h"
#include "syms/syms_inc.h"

#ifdef __cplusplus
}
#endif
